<?xml version='1.0' ?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../../../Build/Graph/Schema.xsd" >

	<Option Name="PreflightChange" DefaultValue="" Description="Preflight changelist number"/>
	
	<Option Name="SubmitToStream" DefaultValue="" Description="Stream to submit binaries to"/>
	<Option Name="SubmitToBranch" DefaultValue="" Description="Branch to submit binaries to"/>
	<Option Name="SubmitPath" DefaultValue="" Description="Relative path within the stream/branch to submit binaries"/>
	<Option Name="IncludeInstaller" DefaultValue="false" Description="Whether to include the launcher installer in the regular client distribution"/>
	<Option Name="HordeSettings" DefaultValue="" Description="Path to a JSON file containing settings for publishing to Horde"/>
	<Option Name="HordeSettingsMac" DefaultValue="" Description="Path to a JSON file containing settings for publishing to Horde"/>
	<Option Name="SignBinaries" DefaultValue="false" Description="Include code signing step for binaries and installer"/>

	<Property Name="UgsDir" Value="$(RootDir)/Engine/Source/Programs/UnrealGameSync"/>
	<Property Name="UgsOutputDir" Value="$(RootDir)/LocalBuilds/UnrealGameSync"/>
	<Property Name="UgsZipOutputDir" Value="$(RootDir)/LocalBuilds/UnrealGameSyncZip"/>
	<Property Name="UgsCmdOutputDir" Value="$(RootDir)/LocalBuilds/UnrealGameSyncCmd"/>
	<Property Name="UgsCmdMacX64ZipFile" Value="$(UgsCmdOutputDir)/ugs-mac-x64.zip"/>
	<Property Name="UgsCmdMacX64OutputDir" Value="$(UgsCmdOutputDir)/Mac-X64"/>
	<Property Name="UgsCmdMacArm64ZipFile" Value="$(UgsCmdOutputDir)/ugs-mac-arm64.zip"/>
	<Property Name="UgsCmdMacArm64OutputDir" Value="$(UgsCmdOutputDir)/Mac-Arm64"/>
	<Property Name="UgsCmdLinuxZipFile" Value="$(UgsCmdOutputDir)/ugs-linux.zip"/>
	<Property Name="UgsCmdLinuxOutputDir" Value="$(UgsCmdOutputDir)/Linux"/>
	<Property Name="WixDir" Value="$(RootDir)/Engine/Source/ThirdParty/WiX/3.8"/>

	<Property Name="Version" Value="$(EngineMajorVersion).$(EngineMinorVersion).$(EnginePatchVersion)"/>
	<Property Name="InformationalVersion" Value="$(EscapedBranch)-CL-$(Change)"/>
	<Property Name="InformationalVersion" Value="$(InformationalVersion)-PF-$(PreflightChange)" If="'$(PreflightChange)' != ''"/>
	<Property Name="VersionArguments" Value="/p:Version=$(Version).0 /p:InformationalVersion=$(InformationalVersion)"/>
	
	<Agent Name="UnrealGameSync Agent" Type="Win64">
		<Property Name="UgsInstallerDir" Value="$(RootDir)/Engine/Source/Programs/UnrealGameSync/Installer"/>
		<Property Name="UgsInstallerPublishDir" Value="$(RootDir)/LocalBuilds/UnrealGameSyncInstaller"/>

		<Property Name="UgsPublishDir" Value="$(UgsOutputDir)"/>
		<Property Name="UgsPublishDir" Value="$(UgsPublishDir)/$(SubmitPath)" If="'$(SubmitPath)' != ''"/>

		<Property Name="UgsZipPublishDir" Value="$(UgsZipOutputDir)"/>
		<Property Name="UgsZipPublishDir" Value="$(UgsZipPublishDir)/$(SubmitPath)" If="'$(SubmitPath)' != ''"/>

		<Node Name="Build UnrealGameSync">
			<Delete Files="$(UgsOutputDir)/..."/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSyncCmd/UnrealGameSyncCmd.csproj&quot; --output=&quot;$(UgsPublishDir)&quot; --no-self-contained -c Release -r osx-x64 $(VersionArguments)"/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSyncCmd/UnrealGameSyncCmd.csproj&quot; --output=&quot;$(UgsPublishDir)&quot; --no-self-contained -c Release -r linux-x64 $(VersionArguments)"/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSyncCmd/UnrealGameSyncCmd.csproj&quot; --output=&quot;$(UgsPublishDir)&quot; --self-contained -c Release -r win-x64 $(VersionArguments)"/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSync/UnrealGameSync.csproj&quot; --output=&quot;$(UgsPublishDir)&quot; --self-contained -c ReleaseAutoUpdate -r win-x64 $(VersionArguments)"/>

			<Sign Files="$(UgsPublishDir)/..." If="$(IsBuildMachine) and $(SignBinaries)" />
		</Node>
		<Node Name="Build UnrealGameSync Installer" Requires="Build UnrealGameSync">
			<Property Name="WixDir" Value="$(RootDir)/Engine/Source/ThirdParty/WiX/3.8"/>
			
			<Spawn Exe="$(WixDir)/heat.exe" Arguments="dir &quot;$(UgsPublishDir)&quot; -cg UGSLauncher_Project -dr INSTALLFOLDER -scom -sreg -srd -var var.BasePath -gg -sfrag -out obj/Release/UGSLauncher.wxs" WorkingDir="$(UgsInstallerDir)"/>
			<Spawn Exe="$(WixDir)/candle.exe" Arguments="-dBasePath=&quot;$(UgsPublishDir)&quot; -out obj\Release\ -dConfiguration=Release -dPlatform=x64 -arch x86 -ext ..\..\..\ThirdParty\WiX\3.8\WixUtilExtension.dll Product.wxs obj/Release/UGSLauncher.wxs" WorkingDir="$(UgsInstallerDir)"/>
			<Spawn Exe="$(WixDir)/light.exe" Arguments="-out &quot;bin/Release/UnrealGameSync.msi&quot; -pdbout &quot;bin/Release/UnrealGameSync.wixpdb&quot; -cultures:null -ext ..\..\..\ThirdParty\WiX\3.8\WixUtilExtension.dll -sice:ICE69 &quot;obj\Release\Product.wixobj&quot; &quot;obj\Release\UGSLauncher.wixobj&quot;" WorkingDir="$(UgsInstallerDir)"/>

			<Copy From="$(UgsInstallerDir)/bin/Release/UnrealGameSync.msi" To="$(UgsInstallerPublishDir)/UnrealGameSync.msi"/>
			
			<Sign Files="$(UgsInstallerPublishDir)/UnrealGameSync.msi" If="$(IsBuildMachine) and $(SignBinaries)" />
		</Node>
		<Node Name="Publish UnrealGameSync Installer Horde" Requires="Build UnrealGameSync Installer">
			<Error Message="Missing -HordeSettings=... argument" If="'$(HordeSettings)' == ''"/>
			<Zip FromDir="$(UgsInstallerPublishDir)" Files="$(UgsInstallerPublishDir)/UnrealGameSync.msi" ZipFile="$(UgsZipPublishDir)/UgsInstaller.zip"/>
			<DeployTool Id="ugs" Version="$(InformationalVersion)" Settings="$(HordeSettings)" File="$(UgsZipPublishDir)/UgsInstaller.zip"/>
		</Node>

		<Property Name="InstallerDependency" Value=""/>
		<Property Name="InstallerDependency" Value="Build UnrealGameSync Installer" If="$(IncludeInstaller)"/>

		<Node Name="Build UnrealGameSync Zip" Requires="Build UnrealGameSync;$(InstallerDependency)">
			<Copy From="$(UgsInstallerDir)/bin/Release/UnrealGameSync.msi" To="$(UgsPublishDir)/UnrealGameSync.msi" If="$(IncludeInstaller)"/>
			<Zip FromDir="$(UgsPublishDir)" ZipFile="$(UgsZipPublishDir)/Ugs.zip"/>
		</Node>

		<!-- Windows (Preview) -->
		<Node Name="Publish Unstable" Requires="Build UnrealGameSync">
			<Property Name="UnstableBranch" Value="" />
			<Property Name="UnstableBranch" Value="$(SubmitToBranch)/UnstableRelease" If="'$(SubmitToBranch)' != ''" />

			<Submit Description="Updating UGS from CL $(Change)" Workspace="Temp" Files="$(UgsPublishDir)/..." RootDir="$(UgsOutputDir)" Stream="$(SubmitToStream)" Branch="$(UnstableBranch)"/>
		</Node>
		<Node Name="Publish Unstable Zip" Requires="Build UnrealGameSync Zip">
			<Copy From="$(UgsZipPublishDir)/Ugs.zip" To="$(UgsZipPublishDir)/UnstableRelease.zip"/>
			<Submit Description="Updating UGS from CL $(Change) (Zip)" Workspace="Temp" Files="$(UgsZipPublishDir)/UnstableRelease.zip" RootDir="$(UgsZipOutputDir)" Stream="$(SubmitToStream)" Branch="$(SubmitToBranch)"/>
		</Node>

		<!-- Windows (Regular) -->
		<Node Name="Publish Live" Requires="Build UnrealGameSync">
			<Property Name="ReleaseBranch" Value="" />
			<Property Name="ReleaseBranch" Value="$(SubmitToBranch)/Release" If="'$(SubmitToBranch)' != ''" />

			<Submit Description="Updating UGS from CL $(Change)" Workspace="Temp" Files="$(UgsPublishDir)/..." RootDir="$(UgsOutputDir)" Stream="$(SubmitToStream)" Branch="$(ReleaseBranch)"/>
		</Node>
		<Node Name="Publish Live Zip" Requires="Build UnrealGameSync Zip">
			<Copy From="$(UgsZipPublishDir)/Ugs.zip" To="$(UgsZipPublishDir)/Release.zip"/>
			<Submit Description="Updating UGS from CL $(Change) (Zip)" Workspace="Temp" Files="$(UgsZipPublishDir)/Release.zip" RootDir="$(UgsZipOutputDir)" Stream="$(SubmitToStream)" Branch="$(SubmitToBranch)"/>
		</Node>
		<Node Name="Publish Live Horde" Requires="Build UnrealGameSync Zip">
			<Error Message="Missing -HordeSettings=... argument" If="'$(HordeSettings)' == ''"/>
			<DeployTool Id="ugs-win" Version="$(InformationalVersion)" Settings="$(HordeSettings)" File="$(UgsZipPublishDir)/Ugs.zip"/>
		</Node>
		
		<!-- Linux -->
		<Node Name="Build UnrealGameSyncCmd Linux">
			<Delete Files="$(UgsCmdLinuxOutputDir)/..."/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSyncCmd/UnrealGameSyncCmd.csproj&quot; --output=&quot;$(UgsCmdLinuxOutputDir)&quot; --self-contained -c Release -r linux-x64 $(VersionArguments)"/>
			<Zip ZipFile="$(UgsCmdLinuxZipFile)" FromDir="$(UgsCmdLinuxOutputDir)" ExecutableFiles="ugs;*.so;*.sh"/>
		</Node>
		<Node Name="Publish UnrealGameSyncCmd Linux" Requires="Build UnrealGameSyncCmd Linux">
			<Error Message="Missing -HordeSettings=... argument" If="'$(HordeSettings)' == ''"/>
			<DeployTool Id="ugs-linux" Version="$(InformationalVersion)" Settings="$(HordeSettings)" File="$(UgsCmdLinuxZipFile)"/>
		</Node>
	</Agent>

	<Agent Name="UnrealGameSync Mac Agent" Type="Mac">

		<!-- Mac -->
		<Node Name="Build UnrealGameSyncCmd Mac-X64">
			<Delete Files="$(UgsCmdMacX64OutputDir)/..."/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSyncCmd/UnrealGameSyncCmd.csproj&quot; --output=&quot;$(UgsCmdMacX64OutputDir)&quot; --self-contained -c Release -r osx-x64 $(VersionArguments)"/>
			<Sign Files="$(UgsCmdMacX64OutputDir)/ugs"/>
			<Zip ZipFile="$(UgsCmdMacX64ZipFile)" FromDir="$(UgsCmdMacX64OutputDir)" ExecutableFiles="ugs;*.dylib;*.sh"/>
		</Node>
		<Node Name="Build UnrealGameSyncCmd Mac-Arm64">
			<Delete Files="$(UgsCmdMacArm64OutputDir)/..."/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSyncCmd/UnrealGameSyncCmd.csproj&quot; --output=&quot;$(UgsCmdMacArm64OutputDir)&quot; --self-contained -c Release -r osx-arm64 $(VersionArguments)"/>
			<Sign Files="$(UgsCmdMacArm64OutputDir)/ugs"/>
			<Zip ZipFile="$(UgsCmdMacArm64ZipFile)" FromDir="$(UgsCmdMacArm64OutputDir)" ExecutableFiles="ugs;*.dylib;*.sh"/>
		</Node>
		<Node Name="Publish UnrealGameSyncCmd Mac" Requires="Build UnrealGameSyncCmd Mac-X64;Build UnrealGameSyncCmd Mac-Arm64">
			<Error Message="Missing -HordeSettingsMac=... argument" If="'$(HordeSettingsMac)' == ''"/>
			<DeployTool Id="ugs-mac" Version="$(InformationalVersion)" Settings="$(HordeSettingsMac)" File="$(UgsCmdMacX64ZipFile)"/>
			<DeployTool Id="ugs-mac-arm64" Version="$(InformationalVersion)" Settings="$(HordeSettingsMac)" File="$(UgsCmdMacArm64ZipFile)"/>
		</Node>

	</Agent>

	<Aggregate Name="Publish Unstable With Zip" Requires="Publish Unstable;Publish Unstable Zip"/>
	<Aggregate Name="Publish Live With Zip" Requires="Publish Live;Publish Live Zip"/>

</BuildGraph>
