<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="Current" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <!-- Import Shared UnrealEngine props for all UnrealGameSync projects -->
  <Import Project="..\Shared\UnrealEngine.csproj.props" />

  <!-- Common UnrealGameSync project properties -->
  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
    <Nullable>enable</Nullable>
  </PropertyGroup>

</Project>
