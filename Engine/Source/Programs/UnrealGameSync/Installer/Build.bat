@echo off
setlocal

pushd %~dp0
set WIXDIR=..\..\..\ThirdParty\WiX\3.8
set SRCDIR=..\UnrealGameSync\bin\Release\net6.0-windows

%WIXDIR%\heat.exe dir "%SRCDIR%" -cg UGSLauncher_Project -dr INSTALLFOLDER -scom -sreg -srd -var var.BasePath -gg -sfrag -out obj/Release/UGSLauncher.wxs -nologo 
if errorlevel 1 goto :eof

%WIXDIR%\candle.exe -nologo -dBasePath="%SRCDIR%" -out obj\Release\ -dConfiguration=Release -dPlatform=x64 -arch x86 -ext ..\..\..\ThirdParty\WiX\3.8\WixUtilExtension.dll Product.wxs obj/Release/UGSLauncher.wxs
if errorlevel 1 goto :eof

set OUTPUT=%~dp0bin\Release\UnrealGameSync.msi
echo Linking %OUTPUT%...
%WIXDIR%\light.exe -nologo -out "bin/Release/UnrealGameSync.msi" -pdbout "bin/Release/UnrealGameSync.wixpdb" -cultures:null -ext ..\..\..\ThirdParty\WiX\3.8\WixUtilExtension.dll -sice:ICE69 obj\Release\Product.wixobj obj\Release\UGSLauncher.wixobj
if errorlevel 1 goto :eof
if not exist "%OUTPUT%" echo Unable to find MSI: %OUTPUT% & goto :eof
