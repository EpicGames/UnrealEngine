<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="4.0" DefaultTargets="Build" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <PropertyGroup>
    <Configuration Condition=" '$(Configuration)' == '' ">Debug</Configuration>
    <Platform Condition=" '$(Platform)' == '' ">x86</Platform>
    <ProductVersion>3.8</ProductVersion>
    <ProjectGuid>e079ddce-0c52-4e9a-aec5-ce425ad9874b</ProjectGuid>
    <SchemaVersion>2.0</SchemaVersion>
    <OutputName>UnrealGameSync</OutputName>
    <OutputType>Package</OutputType>
    <WixToolPath>$(ProjectDir)..\..\..\ThirdParty\WiX\3.8\</WixToolPath>
    <WixTargetsPath>$(WixToolPath)Wix.targets</WixTargetsPath>
    <WixTasksPath>$(WixToolPath)wixtasks.dll</WixTasksPath>
    <SuppressIces>ICE69</SuppressIces>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|x86' ">
    <OutputPath>bin\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Configuration)\</IntermediateOutputPath>
    <DefineConstants>Debug</DefineConstants>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|x86' ">
    <OutputPath>bin\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Configuration)\</IntermediateOutputPath>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|x64' ">
    <DefineConstants>Debug</DefineConstants>
    <OutputPath>bin\$(Platform)\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Platform)\$(Configuration)\</IntermediateOutputPath>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|x64' ">
    <OutputPath>bin\$(Platform)\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Platform)\$(Configuration)\</IntermediateOutputPath>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|x64' ">
    <DefineConstants>Debug</DefineConstants>
    <OutputPath>bin\$(Platform)\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Platform)\$(Configuration)\</IntermediateOutputPath>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|x64' ">
    <OutputPath>bin\$(Platform)\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Platform)\$(Configuration)\</IntermediateOutputPath>
  </PropertyGroup>
  <ItemGroup>
    <Compile Include="Product.wxs" />
    <Compile Include="UGSLauncher.wxs" />
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\InstallerCustomAction\InstallerCustomAction.csproj">
      <Name>InstallerCustomAction</Name>
      <Project>{0b0c6ca3-8c64-4e6b-801f-98e66efc0bf1}</Project>
      <Private>True</Private>
      <DoNotHarvest>True</DoNotHarvest>
      <RefProjectOutputGroups>Binaries;Content;Satellites</RefProjectOutputGroups>
      <RefTargetDir>INSTALLFOLDER</RefTargetDir>
    </ProjectReference>
    <ProjectReference Include="..\UnrealGameSync\UnrealGameSync.csproj">
      <Name>UnrealGameSyncLauncher</Name>
      <Project>{28679228-7901-4740-88ee-4dcb2d6a7535}</Project>
      <Private>True</Private>
      <DoNotHarvest>True</DoNotHarvest>
      <RefProjectOutputGroups>Binaries;Content;Satellites</RefProjectOutputGroups>
      <RefTargetDir>INSTALLFOLDER</RefTargetDir>
    </ProjectReference>
  </ItemGroup>
  <ItemGroup>
    <WixExtension Include="WixUtilExtension">
      <HintPath>$(WixExtDir)\WixUtilExtension.dll</HintPath>
      <Name>WixUtilExtension</Name>
    </WixExtension>
  </ItemGroup>
  <Import Project="$(WixTargetsPath)" />
  <PropertyGroup>
    <PostBuildEvent>copy "$(ProjectDir)\bin\$(Configuration)\UnrealGameSync.msi" "$(SolutionDir)\UnrealGameSync.msi"</PostBuildEvent>
  </PropertyGroup>
  <Target Name="BeforeBuild">
    <Exec Command="dotnet publish ..\UnrealGameSync\UnrealGameSync.csproj -c $(Configuration) -r win10-x64" />
    <PropertyGroup>
      <LinkerBaseInputPaths>..\UnrealGameSync\bin\$(Configuration)\netcoreapp3.1\win10-x64</LinkerBaseInputPaths>
      <DefineConstants>BasePath=..\UnrealGameSync\bin\$(Configuration)\netcoreapp3.1\win10-x64</DefineConstants>
    </PropertyGroup>
    <HeatDirectory OutputFile="UGSLauncher.wxs" DirectoryRefId="INSTALLFOLDER" ComponentGroupName="UGSLauncher_Project" SuppressCom="true" Directory="..\UnrealGameSync\bin\$(Configuration)\netcoreapp3.1\win10-x64" SuppressFragments="true" SuppressRegistry="true" SuppressRootDirectory="true" AutoGenerateGuids="false" GenerateGuidsNow="true" ToolPath="$(WixToolPath)" PreprocessorVariable="var.BasePath" />
  </Target>
  <!--
	To modify your build process, add your task inside one of the targets below and uncomment it.
	Other similar extension points exist, see Wix.targets.
	<Target Name="BeforeBuild">
	</Target>
	<Target Name="AfterBuild">
	</Target>
	-->
</Project>