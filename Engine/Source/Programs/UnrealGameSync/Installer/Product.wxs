<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi" xmlns:netfx="http://schemas.microsoft.com/wix/NetFxExtension" xmlns:util="http://schemas.microsoft.com/wix/UtilExtension">
	<!-- filFB05A6224821B9F6027728B48EBEE2FA is the component name of the UnrealGameSyncLauncher.exe file in the generated UGSLauncher.wxs-->
	<Product Id="6465D644-E775-44B5-B978-7B9EF8F0EF33" Name="UnrealGameSync" Language="1033" Version="!(bind.FileVersion.filFB05A6224821B9F6027728B48EBEE2FA)" Manufacturer="Epic Games, Inc." UpgradeCode="770e2608-c9b0-4a4e-9802-45f9ca696ec6">
		<Package InstallerVersion="200" Compressed="yes" InstallScope="perMachine" />

		<MajorUpgrade DowngradeErrorMessage="A newer version of UnrealGameSync is already installed." />
		<Media Id="1" Cabinet="Cab1.cab" EmbedCab="yes" />

		<!-- TODO: DETECT NET CORE -->

		<Icon Id="Installed.ico" SourceFile="Icon.ico"/>

		<Property Id="ARPPRODUCTICON" Value="Installed.ico" />

		<Feature Id="ProductFeature" Title="UnrealGameSync" Level="1">
			<ComponentRef Id="ShortcutComponent" />
			<ComponentRef Id="UnrealGameSyncLauncher" />
			<ComponentGroupRef Id="UGSLauncher_Project" />
		</Feature>

		<Directory Id="TARGETDIR" Name="SourceDir">
			<Directory Id="ProgramFilesFolder">
				<Directory Id="INSTALLFOLDER" Name="UnrealGameSync" />
			</Directory>
			<Directory Id="ProgramMenuFolder"/>
			<Directory Id="WindowsFolder" Name="WINDOWS"/>
		</Directory>

		<util:CloseApplication Id="CloseUnrealGameSync" Target="UnrealGameSync.exe" CloseMessage="yes" RebootPrompt="no"/>
		<util:CloseApplication Id="CloseUnrealGameSyncLauncher" Target="UnrealGameSyncLauncher.exe" CloseMessage="yes" RebootPrompt="no"/>

		<Property Id="WixShellExecTarget" Value="[#fil067A42F9208D0B6E086D4C5CFC1F1C9C]" />
		<CustomAction Id="LaunchApplication" BinaryKey="WixCA" DllEntry="WixShellExec" Impersonate="yes" />

		<InstallExecuteSequence>
			<Custom Action='LaunchApplication' After='InstallFinalize'>NOT Installed AND NOT REMOVE</Custom>
		</InstallExecuteSequence>

		<DirectoryRef Id="INSTALLFOLDER">
			<Component Id="UnrealGameSyncLauncher" Guid="828C2A14-2351-4957-B575-58E119E2DFA8">				
				<File Id="fil067A42F9208D0B6E086D4C5CFC1F1C9C" KeyPath="yes" Source="$(var.BasePath)\UnrealGameSync.exe" Name="UnrealGameSyncLauncher.exe"/>
			</Component>
		</DirectoryRef>

		<DirectoryRef Id="ProgramMenuFolder">
			<Component Id="ShortcutComponent" Guid="{BE12432C-5E4B-4E44-973E-84C680FCFFB7}">
				<Shortcut Id="ApplicationStartMenuShortcut" Name="UnrealGameSync" Description="Utility to help syncing and compiling UnrealEngine games" Target="[#fil067A42F9208D0B6E086D4C5CFC1F1C9C]" WorkingDirectory="INSTALLFOLDER"/>
				<RegistryValue Root="HKCU" Key="Software\Epic Games\UnrealGameSync" Name="installed" Type="integer" Value="1" KeyPath="yes"/>
			</Component>
		</DirectoryRef>
	</Product>
</Wix>