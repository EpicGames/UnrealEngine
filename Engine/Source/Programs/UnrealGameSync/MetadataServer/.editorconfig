# NOTE: Requires **VS2019 16.3** or later

# EditorConfig is awesome:http://EditorConfig.org

root=True
