// Copyright Epic Games, Inc. All Rights Reserved.

using System.Web.Http;

namespace MetadataServer
{
	public static class WebApiConfig
	{
		public static void Register(HttpConfiguration config)
		{
			// Web API routes
			config.MapHttpAttributeRoutes();

			config.Routes.MapHttpRoute(
				name: "IssueBuildsSubApi",
				routeTemplate: "api/issues/{IssueId}/builds",
				defaults: new { controller = "IssueBuildsSub" }
			);

			config.Routes.MapHttpRoute(
				name: "IssueBuildsApi",
				routeTemplate: "api/issuebuilds/{BuildId}",
				defaults: new { controller = "IssueBuilds" }
			);

			config.Routes.MapHttpRoute(
				name: "IssueDiagnosticsSubApi",
				routeTemplate: "api/issues/{IssueId}/diagnostics",
				defaults: new { controller = "IssueDiagnosticsSub" }
			);

			config.Routes.MapHttpRoute(
				name: "IssueWatchersApi",
				routeTemplate: "api/issues/{IssueId}/watchers",
				defaults: new { controller = "IssueWatchers" }
			);

			config.Routes.MapHttpRoute(
				name: "DefaultApi",
				routeTemplate: "api/{controller}/{id}",
				defaults: new { id = RouteParameter.Optional }
			);
		}
	}
}
