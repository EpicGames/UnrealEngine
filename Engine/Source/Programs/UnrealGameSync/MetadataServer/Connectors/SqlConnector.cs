// Copyright Epic Games, Inc. All Rights Reserved.

using Newtonsoft.Json;
using Newtonsoft.Json.Linq;
using System;
using System.Collections.Generic;
using System.Data.SqlClient;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using MetadataServer.Models;
using MySql.Data.MySqlClient;

namespace MetadataServer.Connectors
{
	public static class SqlConnector
	{
		private static string ConnectionString = System.Configuration.ConfigurationManager.ConnectionStrings["ConnectionString"].ConnectionString;
		private static bool StoreBuildMetadata = Convert.ToBoolean(System.Configuration.ConfigurationManager.AppSettings["StoreBuildMetadata"]);

		public static LatestData GetLastIds(string Project = null)
		{
			// Get ids going back 432 builds for the project being asked for
			// Do this by grouping by ChangeNumber to get unique entries, then take the 432nd id
			long LastEventId = 0;
			long LastCommentId = 0;
			long LastBuildId = 0;
			string ProjectLikeString = "%" + (Project == null ? String.Empty : GetProjectStream(Project)) + "%";
			using (MySqlConnection Connection = new MySqlConnection(ConnectionString))
			{
				Connection.Open();
				using (MySqlCommand Command = new MySqlCommand("WITH user_votes AS (SELECT UserVotes.Id, UserVotes.Changelist FROM ugs_db.UserVotes " +
																 "INNER JOIN ugs_db.Projects ON Projects.Id = UserVotes.ProjectId " +
																 "WHERE Projects.Name LIKE @param1 GROUP BY Changelist ORDER BY Changelist DESC LIMIT 100) " +
																 "SELECT * FROM user_votes ORDER BY user_votes.Changelist ASC LIMIT 1", Connection))
				{
					Command.Parameters.AddWithValue("@param1", ProjectLikeString);
					using (MySqlDataReader Reader = Command.ExecuteReader())
					{
						while (Reader.Read())
						{
							LastEventId = Reader.GetInt64(0);
							break;
						}
					}	
				}
				
				using (MySqlCommand Command = new MySqlCommand("WITH comments AS (SELECT Comments.Id, Comments.ChangeNumber FROM ugs_db.Comments " +
																 "INNER JOIN ugs_db.Projects ON Projects.Id = Comments.ProjectId " +
																 "WHERE Projects.Name LIKE @param1 GROUP BY ChangeNumber ORDER BY ChangeNumber DESC LIMIT 100) " +
																 "SELECT * FROM comments ORDER BY comments.ChangeNumber ASC LIMIT 1", Connection))
				{
					Command.Parameters.AddWithValue("@param1", ProjectLikeString);
					using (MySqlDataReader Reader = Command.ExecuteReader())
					{
						while (Reader.Read())
						{
							LastCommentId = Reader.GetInt32(0);
							break;
						}
					}
				}
				using (MySqlCommand Command = new MySqlCommand("WITH badges AS (SELECT Badges.Id, Badges.ChangeNumber FROM ugs_db.Badges " +
																 "INNER JOIN ugs_db.Projects ON Projects.Id = Badges.ProjectId " +
																 "WHERE Projects.Name LIKE @param1 GROUP BY ChangeNumber ORDER BY ChangeNumber DESC LIMIT 100) " + 
																 "SELECT * FROM badges ORDER BY badges.ChangeNumber ASC LIMIT 1", Connection))
				{
					//Command.Parameters.AddWithValue("@param1", ProjectId);
					Command.Parameters.AddWithValue("@param1", ProjectLikeString);
					using (MySqlDataReader Reader = Command.ExecuteReader())
					{
						while (Reader.Read())
						{
							LastBuildId = Math.Max(LastBuildId, Reader.GetInt32(0));
							break;
						}
					}
				}
			}
			return new LatestData { LastBuildId = LastBuildId, LastCommentId = LastCommentId, LastEventId = LastEventId };
		}
		public static List<EventData> GetUserVotes(string Project, long LastEventId)
		{
			List<EventData> ReturnedEvents = new List<EventData>();
			string ProjectLikeString = "%" + (Project == null ? String.Empty : GetProjectStream(Project)) + "%";
			using (MySqlConnection Connection = new MySqlConnection(ConnectionString))
			{
				Connection.Open();
				using (MySqlCommand Command = new MySqlCommand("SELECT UserVotes.Id, UserVotes.Changelist, UserVotes.UserName, UserVotes.Verdict, UserVotes.Project FROM ugs_db.UserVotes " +
															   "INNER JOIN ugs_db.Projects ON Projects.Id = UserVotes.ProjectId WHERE UserVotes.Id > @param1 AND Projects.Name LIKE @param2 ORDER BY UserVotes.Id", Connection))
				{
					Command.Parameters.AddWithValue("@param1", LastEventId);
					Command.Parameters.AddWithValue("@param2", ProjectLikeString);
					using (MySqlDataReader Reader = Command.ExecuteReader())
					{
						while (Reader.Read())
						{
							EventData Review = new EventData();
							Review.Id = Reader.GetInt64(0);
							Review.Change = Reader.GetInt32(1);
							Review.UserName = Reader.GetString(2);
							Review.Project = Reader.IsDBNull(4) ? null : Reader.GetString(4);
							if (Enum.TryParse(Reader.GetString(3), out Review.Type))
							{
								if (Review.Project == null || String.Compare(Review.Project, Project, true) == 0)
								{
									ReturnedEvents.Add(Review);
								}
							}
						}
					}
				}
			}
			return ReturnedEvents;
		}
		public static List<CommentData> GetComments(string Project, long LastCommentId)
		{
			List<CommentData> ReturnedComments = new List<CommentData>();
			string ProjectLikeString = "%" + (Project == null ? String.Empty : GetProjectStream(Project)) + "%";
			using (MySqlConnection Connection = new MySqlConnection(ConnectionString))
			{
				Connection.Open();
				using (MySqlCommand Command = new MySqlCommand("SELECT Comments.Id, Comments.ChangeNumber, Comments.UserName, Comments.Text, Comments.Project FROM ugs_db.Comments " +
															     "INNER JOIN ugs_db.Projects ON Projects.Id = Comments.ProjectId WHERE Comments.Id > @param1 AND Projects.Name LIKE @param2 ORDER BY Comments.Id", Connection))
				{
					Command.Parameters.AddWithValue("@param1", LastCommentId);
					Command.Parameters.AddWithValue("@param2", ProjectLikeString);
					using (MySqlDataReader Reader = Command.ExecuteReader())
					{
						while (Reader.Read())
						{
							CommentData Comment = new CommentData();
							Comment.Id = Reader.GetInt32(0);
							Comment.ChangeNumber = Reader.GetInt32(1);
							Comment.UserName = Reader.GetString(2);
							Comment.Text = Reader.GetString(3);
							Comment.Project = Reader.GetString(4);
							if (Comment.Project == null || String.Compare(Comment.Project, Project, true) == 0)
							{
								ReturnedComments.Add(Comment);
							}
						}
					}
				}
			}
			return ReturnedComments;
		}

		public static List<BuildData> GetBuilds(string Project, long LastBuildId)
		{
			List<BuildData> ReturnedBuilds = new List<BuildData>();
			string ProjectLikeString = "%" + (Project == null ? String.Empty : GetProjectStream(Project)) + "%";
			using (MySqlConnection Connection = new MySqlConnection(ConnectionString))
			{
				string QueryWithMetadata =
					"SELECT Badges.Id, Badges.ChangeNumber, Badges.BuildType, Badges.Result, Badges.Url, Projects.Name, Badges.ArchivePath, Badges.Metadata FROM ugs_db.Badges " +
					"INNER JOIN ugs_db.Projects ON Projects.Id = Badges.ProjectId WHERE Badges.Id > @param1 AND Projects.Name LIKE @param2 ORDER BY Badges.Id";
				string QueryWithoutMetadata =
					"SELECT Badges.Id, Badges.ChangeNumber, Badges.BuildType, Badges.Result, Badges.Url, Projects.Name, Badges.ArchivePath FROM ugs_db.Badges " +
					"INNER JOIN ugs_db.Projects ON Projects.Id = Badges.ProjectId WHERE Badges.Id > @param1 AND Projects.Name LIKE @param2 ORDER BY Badges.Id";
				Connection.Open();
				using (MySqlCommand Command = new MySqlCommand(StoreBuildMetadata ? QueryWithMetadata : QueryWithoutMetadata, Connection))
				{ 
					Command.Parameters.AddWithValue("@param1", LastBuildId);
					Command.Parameters.AddWithValue("@param2", ProjectLikeString);
					using (MySqlDataReader Reader = Command.ExecuteReader())
					{
						while (Reader.Read())
						{
							BuildData Build = new BuildData();
							Build.Id = Reader.GetInt32(0);
							Build.ChangeNumber = Reader.GetInt32(1);
							Build.BuildType = Reader.GetString(2).TrimEnd();
							if (Enum.TryParse(Reader.GetString(3).TrimEnd(), true, out Build.Result))
							{
								Build.Url = Reader.GetString(4);
								Build.Project = Reader.IsDBNull(5) ? null : Reader.GetString(5);
								if (Build.Project == null || String.Compare(Build.Project, Project, true) == 0 || MatchesWildcard(Build.Project, Project))
								{
									ReturnedBuilds.Add(Build);
								}
							}

							if (StoreBuildMetadata)
							{
								Build.Metadata = JsonConvert.DeserializeObject<BuildMetadata>(Reader.GetString(7));
							}

							LastBuildId = Math.Max(LastBuildId, Build.Id);
						}
					}
				}
			}
			return ReturnedBuilds;
		}

		public static List<TelemetryErrorData> GetErrorData(int Records)
		{
			List<TelemetryErrorData> ReturnedErrors = new List<TelemetryErrorData>();
			using (MySqlConnection Connection = new MySqlConnection(ConnectionString))
			{
				Connection.Open();
				using (MySqlCommand Command = new MySqlCommand("SELECT Id, Type, Text, UserName, Project, Timestamp, Version, IpAddress FROM ugs_db.Errors ORDER BY Id DESC LIMIT @param1", Connection))
				{
					Command.Parameters.AddWithValue("@param1", Records);
					using (MySqlDataReader Reader = Command.ExecuteReader())
					{
						while (Reader.Read())
						{
							TelemetryErrorData Error = new TelemetryErrorData();
							Error.Id = Reader.GetInt32(0);
							Enum.TryParse(Reader.GetString(1), true, out Error.Type);
							Error.Text = Reader.GetString(2);
							Error.UserName = Reader.GetString(3);
							Error.Project = Reader.IsDBNull(4) ? null : Reader.GetString(4);
							Error.Timestamp = Reader.GetDateTime(5);
							Error.Version = Reader.GetString(6);
							Error.IpAddress = Reader.GetString(7);
							ReturnedErrors.Add(Error);
						}
					}
				}
			}
			return ReturnedErrors;
		}

		private static long TryInsertAndGetProject(MySqlConnection Connection, string Project)
		{
			using (MySqlCommand Command = new MySqlCommand("INSERT IGNORE INTO ugs_db.Projects (Name) VALUES (@Project); SELECT Id FROM ugs_db.Projects WHERE Name = @Project", Connection))
			{
				Command.Parameters.AddWithValue("@Project", Project);
				object ProjectId = Command.ExecuteScalar();
				return Convert.ToInt64(ProjectId);
			}
		}
		public static void PostBuild(BuildData Build)
		{
			using (MySqlConnection Connection = new MySqlConnection(ConnectionString))
			{
				Connection.Open();
				long ProjectId = TryInsertAndGetProject(Connection, Build.Project);

				string InsertWithMetadata =
					"INSERT INTO ugs_db.Badges (ChangeNumber, BuildType, Result, URL, ArchivePath, ProjectId, Metadata) VALUES (@ChangeNumber, @BuildType, @Result, @URL, @ArchivePath, @ProjectId, @Metadata)";
				string InsertWithoutMetadata =
					"INSERT INTO ugs_db.Badges (ChangeNumber, BuildType, Result, URL, ArchivePath, ProjectId) VALUES (@ChangeNumber, @BuildType, @Result, @URL, @ArchivePath, @ProjectId)";
				using (MySqlCommand Command = new MySqlCommand(StoreBuildMetadata ? InsertWithMetadata : InsertWithoutMetadata, Connection))
				{
					Command.Parameters.AddWithValue("@ChangeNumber", Build.ChangeNumber);
					Command.Parameters.AddWithValue("@BuildType", Build.BuildType);
					Command.Parameters.AddWithValue("@Result", Build.Result);
					Command.Parameters.AddWithValue("@URL", Build.Url);
					Command.Parameters.AddWithValue("@ArchivePath", Build.ArchivePath);
					Command.Parameters.AddWithValue("@ProjectId", ProjectId);
					if (StoreBuildMetadata)
					{
						Command.Parameters.AddWithValue("@Metadata", JsonConvert.SerializeObject(Build.Metadata ?? new BuildMetadata()));
					}
					Command.ExecuteNonQuery();
				}
			}
		}

		public static void PostEvent(EventData Event)
		{
			using (MySqlConnection Connection = new MySqlConnection(ConnectionString))
			{
				Connection.Open();
				long ProjectId = TryInsertAndGetProject(Connection, Event.Project);
				using (MySqlCommand Command = new MySqlCommand("INSERT INTO ugs_db.UserVotes (Changelist, UserName, Verdict, Project, ProjectId) VALUES (@Changelist, @UserName, @Verdict, @Project, @ProjectId)", Connection))
				{
					Command.Parameters.AddWithValue("@Changelist", Event.Change);
					Command.Parameters.AddWithValue("@UserName", Event.UserName.ToString());
					Command.Parameters.AddWithValue("@Verdict", Event.Type.ToString());
					Command.Parameters.AddWithValue("@Project", Event.Project);
					Command.Parameters.AddWithValue("@ProjectId", ProjectId);
					Command.ExecuteNonQuery();
				}
			}
		}

		public static void PostComment(CommentData Comment)
		{
			using (MySqlConnection Connection = new MySqlConnection(ConnectionString))
			{
				Connection.Open();
				long ProjectId = TryInsertAndGetProject(Connection, Comment.Project);
				using (MySqlCommand Command = new MySqlCommand("INSERT INTO ugs_db.Comments (ChangeNumber, UserName, Text, Project, ProjectId) VALUES (@ChangeNumber, @UserName, @Text, @Project, @ProjectId)", Connection))
				{
					Command.Parameters.AddWithValue("@ChangeNumber", Comment.ChangeNumber);
					Command.Parameters.AddWithValue("@UserName", Comment.UserName);
					Command.Parameters.AddWithValue("@Text", Comment.Text);
					Command.Parameters.AddWithValue("@Project", Comment.Project);
					Command.Parameters.AddWithValue("@ProjectId", ProjectId);
					Command.ExecuteNonQuery();
				}
			}
		}

		public static void PostTelemetryData(TelemetryTimingData Data, string Version, string IpAddress)
		{
			using (MySqlConnection Connection = new MySqlConnection(ConnectionString))
			{
				Connection.Open();
				long ProjectId = TryInsertAndGetProject(Connection, Data.Project);
				using (MySqlCommand Command = new MySqlCommand("INSERT INTO ugs_db.Telemetry_v2 (Action, Result, UserName, Project, Timestamp, Duration, Version, IpAddress, ProjectId) VALUES (@Action, @Result, @UserName, @Project, @Timestamp, @Duration, @Version, @IpAddress, @ProjectId)", Connection))
				{
					Command.Parameters.AddWithValue("@Action", Data.Action);
					Command.Parameters.AddWithValue("@Result", Data.Result);
					Command.Parameters.AddWithValue("@UserName", Data.UserName);
					Command.Parameters.AddWithValue("@Project", Data.Project);
					Command.Parameters.AddWithValue("@Timestamp", Data.Timestamp);
					Command.Parameters.AddWithValue("@Duration", Data.Duration);
					Command.Parameters.AddWithValue("@Version", Version);
					Command.Parameters.AddWithValue("@IPAddress", IpAddress);
					Command.Parameters.AddWithValue("@ProjectId", ProjectId);
					Command.ExecuteNonQuery();
				}
			}
		}

		public static void PostErrorData(TelemetryErrorData Data, string Version, string IpAddress)
		{
			using (MySqlConnection Connection = new MySqlConnection(ConnectionString))
			{
				Connection.Open();
				long? ProjectId = null;
				if(Data.Project != null)
				{
					ProjectId = TryInsertAndGetProject(Connection, Data.Project);
				}
				using (MySqlCommand Command = new MySqlCommand("INSERT INTO ugs_db.Errors (Type, Text, UserName, Project, Timestamp, Version, IpAddress, ProjectId) VALUES (@Type, @Text, @UserName, @Project, @Timestamp, @Version, @IpAddress, @ProjectId)", Connection))
				{
					Command.Parameters.AddWithValue("@Type", Data.Type.ToString());
					Command.Parameters.AddWithValue("@Text", Data.Text);
					Command.Parameters.AddWithValue("@UserName", Data.UserName);
					if (Data.Project == null)
					{
						Command.Parameters.AddWithValue("@Project", DBNull.Value);
						Command.Parameters.AddWithValue("@ProjectId", DBNull.Value);
					}
					else
					{
						Command.Parameters.AddWithValue("@Project", Data.Project);
						Command.Parameters.AddWithValue("@ProjectId", ProjectId.Value);
					}
					Command.Parameters.AddWithValue("@Timestamp", Data.Timestamp);
					Command.Parameters.AddWithValue("@Version", Version);
					Command.Parameters.AddWithValue("@IPAddress", IpAddress);
					Command.ExecuteNonQuery();
				}
			}
		}
		private static string GetProjectStream(string Project)
		{
			// Get first two fragments of the p4 path.  If it doesn't work, just return back the project.
			Regex StreamPattern = new Regex("(\\/\\/[a-zA-Z0-9\\.\\-_]{1,}\\/[a-zA-Z0-9\\.\\-_]{1,})");
			Match StreamMatch = StreamPattern.Match(Project);
			if(StreamMatch.Success)
			{
				return StreamMatch.Groups[1].Value;
			}
			return Project;
		}
		private static bool MatchesWildcard(string Wildcard, string Project)
		{
			return Wildcard.EndsWith("...") && Project.StartsWith(Wildcard.Substring(0, Wildcard.Length - 4), StringComparison.InvariantCultureIgnoreCase);
		}

		private static string NormalizeUserName(string UserName)
		{
			return UserName.ToUpperInvariant();
		}

		public static long FindOrAddUserId(string Name)
		{
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();
				return FindOrAddUserId(Name, Connection);
			}
		}

		private static long FindOrAddUserId(string Name, MySqlConnection Connection)
		{
			if(Name.Length == 0)
			{
				return -1;
			}

			string NormalizedName = NormalizeUserName(Name);

			using (MySqlCommand Command = new MySqlCommand("SELECT Id FROM ugs_db.Users WHERE Name = @Name", Connection))
			{
				Command.Parameters.AddWithValue("@Name", NormalizedName);
				object UserId = Command.ExecuteScalar();
				if(UserId != null)
				{
					return Convert.ToInt64(UserId);
				}
			}

			using (MySqlCommand Command = new MySqlCommand("INSERT IGNORE INTO ugs_db.Users (Name) VALUES (@Name); SELECT Id FROM ugs_db.Users WHERE Name = @Name", Connection))
			{
				Command.Parameters.AddWithValue("@Name", NormalizedName);
				object UserId = Command.ExecuteScalar();
				return Convert.ToInt64(UserId);
			}
		}

		const int IssueSummaryMaxLength = 200;

		public static long AddIssue(IssueData Issue)
		{
			long IssueId;
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				using (MySqlCommand Command = new MySqlCommand("INSERT INTO ugs_db.Issues (Project, Summary, OwnerId, CreatedAt, FixChange) VALUES (@Project, @Summary, @OwnerId, UTC_TIMESTAMP(), 0)", Connection))
				{
					Command.Parameters.AddWithValue("@Project", Issue.Project);
					Command.Parameters.AddWithValue("@Summary", SanitizeText(Issue.Summary, IssueSummaryMaxLength));
					if (Issue.Owner != null)
					{
						Command.Parameters.AddWithValue("OwnerId", FindOrAddUserId(Issue.Owner, Connection));
					}
					else
					{
						Command.Parameters.AddWithValue("OwnerId", null);
					}
					Command.ExecuteNonQuery();

					IssueId = Command.LastInsertedId;
				}
			}
			return IssueId;
		}

		public static IssueData GetIssue(long IssueId)
		{
			List<IssueData> Issues = GetIssuesInternal(IssueId, null, true, -1);
			if(Issues.Count == 0)
			{
				return null;
			}
			else
			{
				return Issues[0];
			}
		}

		public static List<IssueData> GetIssues(bool IncludeResolved, int NumResults)
		{
			return GetIssuesInternal(-1, null, IncludeResolved, NumResults);
		}

		public static List<IssueData> GetIssues(string UserName)
		{
			return GetIssuesInternal(-1, UserName, false, -1);
		}

		private static List<IssueData> GetIssuesInternal(long IssueId, string UserName, bool IncludeResolved, int NumResults)
		{
			List<IssueData> Issues = new List<IssueData>();
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				long UserId = -1;
				if(UserName != null)
				{
					UserId = FindOrAddUserId(UserName);
				}

				StringBuilder CommandBuilder = new StringBuilder();
				CommandBuilder.Append("SELECT");
				CommandBuilder.Append(" Issues.Id, Issues.CreatedAt, UTC_TIMESTAMP(), Issues.Project, Issues.Summary, OwnerUsers.Name, NominatedByUsers.Name, Issues.AcknowledgedAt, Issues.FixChange, Issues.ResolvedAt");
				if(UserName != null)
				{
					CommandBuilder.Append(", IssueWatchers.UserId");
				}
				CommandBuilder.Append(" FROM ugs_db.Issues");
				CommandBuilder.Append(" LEFT JOIN ugs_db.Users AS OwnerUsers ON OwnerUsers.Id = Issues.OwnerId");
				CommandBuilder.Append(" LEFT JOIN ugs_db.Users AS NominatedByUsers ON NominatedByUsers.Id = Issues.NominatedById");
				if(UserName != null)
				{
					CommandBuilder.Append(" LEFT JOIN ugs_db.IssueWatchers ON IssueWatchers.IssueId = Issues.Id AND IssueWatchers.UserId = @UserId");
				}
				if(IssueId != -1)
				{
					CommandBuilder.Append(" WHERE Issues.Id = @IssueId");
				}
				else if(!IncludeResolved)
				{
					CommandBuilder.Append(" WHERE Issues.ResolvedAt IS NULL");
				}
				if(NumResults > 0)
				{
					CommandBuilder.AppendFormat(" ORDER BY Issues.Id DESC LIMIT {0}", NumResults);
				}

				using (MySqlCommand Command = new MySqlCommand(CommandBuilder.ToString(), Connection))
				{
					if(IssueId != -1)
					{
						Command.Parameters.AddWithValue("@IssueId", IssueId);
					}
					if(UserName != null)
					{
						Command.Parameters.AddWithValue("@UserId", UserId);
					}

					using(MySqlDataReader Reader = Command.ExecuteReader())
					{
						while(Reader.Read())
						{
							IssueData Issue = new IssueData();
							Issue.Id = Reader.GetInt64(0);
							Issue.CreatedAt = Reader.GetDateTime(1);
							Issue.RetrievedAt = Reader.GetDateTime(2);
							Issue.Project = Reader.GetString(3);
							Issue.Summary = Reader.GetString(4);
							Issue.Owner = Reader.IsDBNull(5)? null : Reader.GetString(5);
							Issue.NominatedBy = Reader.IsDBNull(6)? null : Reader.GetString(6);
							Issue.AcknowledgedAt = Reader.IsDBNull(7)? (DateTime?)null : Reader.GetDateTime(7);
							Issue.FixChange = Reader.GetInt32(8);
							Issue.ResolvedAt = Reader.IsDBNull(9)? (DateTime?)null : Reader.GetDateTime(9);
							if(UserName != null)
							{
								Issue.bNotify = !Reader.IsDBNull(10);
							}
							Issues.Add(Issue);
						}
					}
				}
			}
			return Issues;
		}

		public static void UpdateIssue(long IssueId, IssueUpdateData Issue)
		{
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				using (MySqlCommand Command = Connection.CreateCommand())
				{
					List<string> Columns = new List<string>();
					List<string> Values = new List<string>();
					if(Issue.Summary != null)
					{
						Columns.Add("Summary");
						Values.Add("@Summary");
						Command.Parameters.AddWithValue("@Summary", SanitizeText(Issue.Summary, IssueSummaryMaxLength));
					}
					if (Issue.Owner != null)
					{
						Columns.Add("OwnerId");
						Values.Add("@OwnerId");
						Command.Parameters.AddWithValue("OwnerId", FindOrAddUserId(Issue.Owner, Connection));
					}
					if(Issue.NominatedBy != null)
					{
						Columns.Add("NominatedById");
						Values.Add("@NominatedById");
						Command.Parameters.AddWithValue("NominatedById", FindOrAddUserId(Issue.NominatedBy, Connection));
					}
					if(Issue.Acknowledged.HasValue)
					{
						Columns.Add("AcknowledgedAt");
						Values.Add(Issue.Acknowledged.Value? "UTC_TIMESTAMP()" : "NULL");
					}
					if(Issue.FixChange.HasValue)
					{
						Columns.Add("FixChange");
						Values.Add("@FixChange");
						Command.Parameters.AddWithValue("FixChange", Issue.FixChange.Value);
					}
					if(Issue.Resolved.HasValue)
					{
						Columns.Add("ResolvedAt");
						Values.Add(Issue.Resolved.Value? "UTC_TIMESTAMP()" : "NULL");
					}

					StringBuilder CommandText = new StringBuilder("UPDATE ugs_db.Issues SET ");
					for(int idx = 0; idx < Columns.Count; idx++)
					{
						CommandText.Append(String.Format("{0}={1}", Columns[idx], Values[idx]));
						if(idx != Columns.Count - 1)
						{
							CommandText.Append(",");
						}
					}
					CommandText.Append(" WHERE Id = @IssueId");
					Command.CommandText = CommandText.ToString();
					Command.Parameters.AddWithValue("@IssueId", IssueId);
					Command.ExecuteNonQuery();
				}
			}
		}

		public static string SanitizeText(string Text, int Length)
		{
			if(Text.Length > Length)
			{
				int NewlineIdx = Text.LastIndexOf('\n', Length);
				if(NewlineIdx == -1)
				{
					Text = Text.Substring(0, Length - 3).TrimEnd() + "...";
				}
				else
				{
					Text = Text.Substring(0, NewlineIdx + 1) + "...";
				}
			}
			return Text;
		}

		public static void DeleteIssue(long IssueId)
		{
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				using (MySqlTransaction Transaction = Connection.BeginTransaction())
				{
					using (MySqlCommand Command = Connection.CreateCommand())
					{
						Command.Transaction = Transaction;

						Command.CommandText = "DELETE FROM ugs_db.IssueWatchers WHERE IssueId = @IssueId";
						Command.Parameters.AddWithValue("@IssueId", IssueId);
						Command.ExecuteNonQuery();

						Command.CommandText = "DELETE FROM ugs_db.IssueBuilds WHERE IssueId = @IssueId";
						Command.Parameters.AddWithValue("@IssueId", IssueId);
						Command.ExecuteNonQuery();

						Command.CommandText = "DELETE FROM ugs_db.Issues WHERE Id = @IssueId";
						Command.Parameters.AddWithValue("@IssueId", IssueId);
						Command.ExecuteNonQuery();

						Transaction.Commit();
					}
				}
			}
		}

		public static void AddDiagnostic(long IssueId, IssueDiagnosticData Diagnostic)
		{
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				using (MySqlCommand Command = new MySqlCommand("INSERT INTO ugs_db.IssueDiagnostics (IssueId, BuildId, Message, Url) VALUES (@IssueId, @BuildId, @Message, @Url)", Connection))
				{
					Command.Parameters.AddWithValue("@IssueId", IssueId);
					Command.Parameters.AddWithValue("@BuildId", Diagnostic.BuildId);
					Command.Parameters.AddWithValue("@Message", SanitizeText(Diagnostic.Message, 1000));
					Command.Parameters.AddWithValue("@Url", Diagnostic.Url);
					Command.ExecuteNonQuery();
				}
			}
		}

		public static List<IssueDiagnosticData> GetDiagnostics(long IssueId)
		{
			List<IssueDiagnosticData> Diagnostics = new List<IssueDiagnosticData>();
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				StringBuilder CommandBuilder = new StringBuilder();
				CommandBuilder.Append("SELECT BuildId, Message, Url FROM ugs_db.IssueDiagnostics");
				CommandBuilder.Append(" WHERE IssueDiagnostics.IssueId = @IssueId");

				using (MySqlCommand Command = new MySqlCommand(CommandBuilder.ToString(), Connection))
				{
					Command.Parameters.AddWithValue("@IssueId", IssueId);
					using (MySqlDataReader Reader = Command.ExecuteReader())
					{
						while (Reader.Read())
						{
							IssueDiagnosticData Diagnostic = new IssueDiagnosticData();
							Diagnostic.BuildId = Reader.IsDBNull(0)? (long?)null : (long?)Reader.GetInt64(0);
							Diagnostic.Message = Reader.GetString(1);
							Diagnostic.Url = Reader.GetString(2);
							Diagnostics.Add(Diagnostic);
						}
					}
				}
			}
			return Diagnostics;
		}

		public static void AddWatcher(long IssueId, string UserName)
		{
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				long UserId = FindOrAddUserId(UserName, Connection);

				using(MySqlCommand Command = new MySqlCommand("INSERT IGNORE INTO ugs_db.IssueWatchers (IssueId, UserId) VALUES (@IssueId, @UserId)", Connection))
				{
					Command.Parameters.AddWithValue("@IssueId", IssueId);
					Command.Parameters.AddWithValue("@UserId", UserId);
					Command.ExecuteNonQuery();
				}
			}
		}

		public static List<string> GetWatchers(long IssueId)
		{
			List<string> Watchers = new List<string>();
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				StringBuilder CommandBuilder = new StringBuilder();
				CommandBuilder.Append("SELECT Users.Name FROM ugs_db.IssueWatchers");
				CommandBuilder.Append(" LEFT JOIN ugs_db.Users ON IssueWatchers.UserId = Users.Id");
				CommandBuilder.Append(" WHERE IssueWatchers.IssueId = @IssueId");

				using(MySqlCommand Command = new MySqlCommand(CommandBuilder.ToString(), Connection))
				{
					Command.Parameters.AddWithValue("@IssueId", IssueId);
					using(MySqlDataReader Reader = Command.ExecuteReader())
					{
						while(Reader.Read())
						{
							Watchers.Add(Reader.GetString(0));
						}
					}
				}
			}
			return Watchers;
		}

		public static void RemoveWatcher(long IssueId, string UserName)
		{
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				long UserId = FindOrAddUserId(UserName, Connection);

				using(MySqlCommand Command = new MySqlCommand("DELETE FROM ugs_db.IssueWatchers WHERE IssueId = @IssueId AND UserId = @UserId", Connection))
				{
					Command.Parameters.AddWithValue("@IssueId", IssueId);
					Command.Parameters.AddWithValue("@UserId", UserId);
					Command.ExecuteNonQuery();
				}
			}
		}

		public static long AddBuild(long IssueId, IssueBuildData Build)
		{
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				using (MySqlCommand Command = new MySqlCommand("INSERT INTO ugs_db.IssueBuilds (IssueId, Stream, `Change`, JobName, JobUrl, JobStepName, JobStepUrl, ErrorUrl, Outcome) VALUES (@IssueId, @Stream, @Change, @JobName, @JobUrl, @JobStepName, @JobStepUrl, @ErrorUrl, @Outcome)", Connection))
				{
					Command.Parameters.AddWithValue("@IssueId", IssueId);
					Command.Parameters.AddWithValue("@Stream", Build.Stream);
					Command.Parameters.AddWithValue("@Change", Build.Change);
					Command.Parameters.AddWithValue("@JobName", Build.JobName);
					Command.Parameters.AddWithValue("@JobUrl", Build.JobUrl);
					Command.Parameters.AddWithValue("@JobStepName", Build.JobStepName);
					Command.Parameters.AddWithValue("@JobStepUrl", Build.JobStepUrl);
					Command.Parameters.AddWithValue("@ErrorUrl", Build.ErrorUrl);
					Command.Parameters.AddWithValue("@Outcome", Build.Outcome);
					Command.ExecuteNonQuery();

					return Command.LastInsertedId;
				}
			}
		}

		public static List<IssueBuildData> GetBuilds(long IssueId)
		{
			List<IssueBuildData> Builds = new List<IssueBuildData>();
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				using(MySqlCommand Command = new MySqlCommand("SELECT IssueBuilds.Id, IssueBuilds.Stream, IssueBuilds.Change, IssueBuilds.JobName, IssueBuilds.JobUrl, IssueBuilds.JobStepName, IssueBuilds.JobStepUrl, IssueBuilds.ErrorUrl, IssueBuilds.Outcome FROM ugs_db.IssueBuilds WHERE IssueBuilds.IssueId = @IssueId", Connection))
				{
					Command.Parameters.AddWithValue("@IssueId", IssueId);
					using(MySqlDataReader Reader = Command.ExecuteReader())
					{
						while(Reader.Read())
						{
							long Id = Reader.GetInt64(0);
							string Stream = Reader.GetString(1);
							int Change = Reader.GetInt32(2);
							string JobName = Reader.GetString(3);
							string JobUrl = Reader.GetString(4);
							string JobStepName = Reader.GetString(5);
							string JobStepUrl = Reader.GetString(6);
							string ErrorUrl = Reader.IsDBNull(7)? null : Reader.GetString(7);
							int Outcome = Reader.GetInt32(8);
							Builds.Add(new IssueBuildData { Id = Id, Stream = Stream, Change = Change, JobName = JobName, JobUrl = JobUrl, JobStepName = JobStepName, JobStepUrl = JobStepUrl, ErrorUrl = ErrorUrl, Outcome = Outcome });
						}
					}
				}
			}
			return Builds;
		}

		public static IssueBuildData GetBuild(long BuildId)
		{
			IssueBuildData Build = null;
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				using(MySqlCommand Command = new MySqlCommand("SELECT IssueBuilds.Id, IssueBuilds.Stream, IssueBuilds.Change, IssueBuilds.JobName, IssueBuilds.JobUrl, IssueBuilds.JobStepName, IssueBuilds.JobStepUrl, IssueBuilds.ErrorUrl, IssueBuilds.Outcome FROM ugs_db.IssueBuilds WHERE IssueBuilds.Id = @BuildId", Connection))
				{
					Command.Parameters.AddWithValue("@BuildId", BuildId);
					using(MySqlDataReader Reader = Command.ExecuteReader())
					{
						while(Reader.Read())
						{
							long Id = Reader.GetInt64(0);
							string Stream = Reader.GetString(1);
							int Change = Reader.GetInt32(2);
							string JobName = Reader.GetString(3);
							string JobUrl = Reader.GetString(4);
							string JobStepName = Reader.GetString(5);
							string JobStepUrl = Reader.GetString(6);
							string ErrorUrl = Reader.GetString(7);
							int Outcome = Reader.GetInt32(8);

							Build = new IssueBuildData { Id = Id, Stream = Stream, Change = Change, JobName = JobName, JobUrl = JobUrl, JobStepName = JobStepName, JobStepUrl = JobStepUrl, ErrorUrl = ErrorUrl, Outcome = Outcome };
						}
					}
				}
			}
			return Build;
		}

		public static void UpdateBuild(long BuildId, int Outcome)
		{
			using (MySqlConnection Connection = new MySqlConnection(SqlConnector.ConnectionString))
			{
				Connection.Open();

				using (MySqlCommand Command = new MySqlCommand("UPDATE ugs_db.IssueBuilds SET (Outcome) = (@Outcome) WHERE Id = @BuildId", Connection))
				{
					Command.Parameters.AddWithValue("@BuildId", BuildId);
					Command.Parameters.AddWithValue("@Outcome", Outcome);
					Command.ExecuteNonQuery();
				}
			}
		}
	}
}