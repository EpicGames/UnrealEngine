// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Web.Http;
using MetadataServer.Connectors;
using MetadataServer.Models;

namespace MetadataServer.Controllers
{
    public class BuildController : ApiController
    {
		public List<BuildData> Get(string Project, long LastBuildId)
		{
			return SqlConnector.GetBuilds(Project, LastBuildId);
		}
		public void Post([FromBody]BuildData Build)
		{
			SqlConnector.PostBuild(Build);
		}
	}
}
