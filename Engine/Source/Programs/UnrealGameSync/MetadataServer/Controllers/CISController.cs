// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Web.Http;
using MetadataServer.Connectors;
using MetadataServer.Models;

namespace MetadataServer.Controllers
{
	// Deprecated, use Latest or Build controllers instead
    public class CISController : ApiController
    {
		public long[] Get(string Project = null)
		{
			LatestData LatestIDs = SqlConnector.GetLastIds(Project);
			return new long[] { LatestIDs.LastEventId, LatestIDs.LastCommentId, LatestIDs.LastBuildId };
		}
		public List<BuildData> Get(string Project, long LastBuildId)
		{
			return SqlConnector.GetBuilds(Project, LastBuildId);
		}
		public void Post([FromBody]BuildData Build)
		{
			SqlConnector.PostBuild(Build);
		}
	}
}
