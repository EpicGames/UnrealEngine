// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Web.Http;
using MetadataServer.Connectors;
using MetadataServer.Models;

namespace MetadataServer.Controllers
{
	public class CommentController : ApiController
	{
		public List<CommentData> Get(string Project, long LastCommentId)
		{
			return SqlConnector.GetComments(Project, LastCommentId);
		}
		public void Post([FromBody] CommentData Comment)
		{
			SqlConnector.PostComment(Comment);
		}
	}
}