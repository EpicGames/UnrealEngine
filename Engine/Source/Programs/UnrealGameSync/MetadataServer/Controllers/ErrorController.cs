// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Web.Http;
using MetadataServer.Connectors;
using MetadataServer.Models;

namespace MetadataServer.Controllers
{
    public class ErrorController : ApiController
    {
		public List<TelemetryErrorData> Get(int Records = 10)
		{
			return SqlConnector.GetErrorData(Records);
		}
		public void Post([FromBody] TelemetryErrorData Data, string Version, string IpAddress)
		{
			SqlConnector.PostErrorData(Data, Version, IpAddress);
		}
    }
}
