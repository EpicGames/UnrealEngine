// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Web.Http;
using MetadataServer.Models;
using MetadataServer.Connectors;

namespace MetadataServer.Controllers
{
	public class EventController : ApiController
	{
		public List<EventData> Get(string Project, long LastEventId)
		{
			return SqlConnector.GetUserVotes(Project, LastEventId);
		}

		public void Post([FromBody] EventData Event)
		{
			SqlConnector.PostEvent(Event);
		}
	}
}