// Copyright Epic Games, Inc. All Rights Reserved.

using Newtonsoft.Json.Linq;
using System;
using System.Collections.Generic;
using System.Dynamic;
using System.Linq;
using System.Net.Http;
using System.Web.Http;
using System.Web.Http.Results;
using MetadataServer.Connectors;
using MetadataServer.Models;

namespace MetadataServer.Controllers
{
	public class IssuesController : ApiController
	{
		[HttpGet]
		public List<IssueData> Get(bool IncludeResolved = false, int MaxResults = -1)
		{
			return SqlConnector.GetIssues(IncludeResolved, MaxResults);
		}

		[HttpGet]
		public List<IssueData> Get(string User)
		{
			return SqlConnector.GetIssues(User);
		}

		[HttpGet]
		public IssueData Get(long id)
		{
			return SqlConnector.GetIssue(id);
		}

		[HttpPut]
		public void Put(long id, IssueUpdateData Issue)
		{
			SqlConnector.UpdateIssue(id, Issue);
		}

		[HttpPost]
		public object Post(IssueData Issue)
		{
			long IssueId = SqlConnector.AddIssue(Issue);
			return new { Id = IssueId };
		}

		[HttpDelete]
		public void Delete(long id)
		{
			SqlConnector.DeleteIssue(id);
		}
	}

	public class IssueBuildsSubController : ApiController
	{
		[HttpGet]
		public List<IssueBuildData> Get(long IssueId)
		{
			return SqlConnector.GetBuilds(IssueId);
		}

		[HttpPost]
		public object Post(long IssueId, [FromBody] IssueBuildData Data)
		{
			long BuildId = SqlConnector.AddBuild(IssueId, Data);
			return new { Id = BuildId };
		}
	}

	public class IssueDiagnosticsSubController : ApiController
	{
		[HttpGet]
		public List<IssueDiagnosticData> Get(long IssueId)
		{
			return SqlConnector.GetDiagnostics(IssueId);
		}

		[HttpPost]
		public void Post(long IssueId, [FromBody] IssueDiagnosticData Data)
		{
			SqlConnector.AddDiagnostic(IssueId, Data);
		}
	}

	public class IssueBuildsController : ApiController
	{
		[HttpGet]
		public IssueBuildData Get(long BuildId)
		{
			return SqlConnector.GetBuild(BuildId);
		}

		[HttpPut]
		public void Put(long BuildId, [FromBody] IssueBuildUpdateData Data)
		{
			SqlConnector.UpdateBuild(BuildId, Data.Outcome);
		}
	}

	public class IssueWatchersController : ApiController
	{
		[HttpGet]
		public List<string> Get(long IssueId)
		{
			return SqlConnector.GetWatchers(IssueId);
		}

		[HttpPost]
		public void Post(long IssueId, [FromBody] IssueWatcherData Data)
		{
			SqlConnector.AddWatcher(IssueId, Data.UserName);
		}

		[HttpDelete]
		public void Delete(long IssueId, [FromBody] IssueWatcherData Data)
		{
			SqlConnector.RemoveWatcher(IssueId, Data.UserName);
		}
	}
}
