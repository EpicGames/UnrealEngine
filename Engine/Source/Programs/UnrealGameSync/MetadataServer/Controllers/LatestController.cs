// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Web.Http;
using MetadataServer.Connectors;
using MetadataServer.Models;

namespace MetadataServer.Controllers
{
    public class LatestController : ApiController
    {
		public LatestData Get(string Project = null)
		{
			return SqlConnector.GetLastIds(Project);
		}
	}
}
