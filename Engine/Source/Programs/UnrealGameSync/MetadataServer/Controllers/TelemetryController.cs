// Copyright Epic Games, Inc. All Rights Reserved.

using System.Web.Http;
using MetadataServer.Connectors;
using MetadataServer.Models;

namespace MetadataServer.Controllers
{
    public class TelemetryController : ApiController
    {
		public void Post([FromBody] TelemetryTimingData Data, string Version, string IpAddress)
		{
			SqlConnector.PostTelemetryData(Data, Version, IpAddress);
		}
    }
}
