// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Web.Http;
using MetadataServer.Connectors;
using MetadataServer.Models;

namespace MetadataServer.Controllers
{
	public class UserController : ApiController
	{
		public object Get(string Name)
		{
			long UserId = SqlConnector.FindOrAddUserId(Name);
			return new { Id = UserId };
		}
	}
}
