<%@ Application Codebehind="Global.asax.cs" Inherits="MetadataServer.WebApiApplication" Language="C#" %>
