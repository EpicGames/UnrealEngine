// Copyright Epic Games, Inc. All Rights Reserved.
using MySql.Data.MySqlClient;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Web;
using System.Web.Http;

namespace MetadataServer
{
	public class WebApiApplication : System.Web.HttpApplication
	{
        protected void Application_Start()
        {
            try
            {
                string ConnectionString = System.Configuration.ConfigurationManager.ConnectionStrings["ConnectionString"].ConnectionString;
				using (MySqlConnection Connection = new MySqlConnection(ConnectionString))
				{
					Connection.Open();
					using (MySqlCommand Command = new MySqlCommand(Properties.Resources.Setup, Connection))
					{
						Command.ExecuteNonQuery();
					}
				}
			}
            catch (Exception)
            {
                HttpRuntime.UnloadAppDomain();
                return;
            }
            GlobalConfiguration.Configure(WebApiConfig.Register);
        }
	}
    
}
