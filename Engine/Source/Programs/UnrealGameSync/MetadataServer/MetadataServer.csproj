<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="14.0" DefaultTargets="Build" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <Import Project="$(MSBuildExtensionsPath)\$(MSBuildToolsVersion)\Microsoft.Common.props" Condition="Exists('$(MSBuildExtensionsPath)\$(MSBuildToolsVersion)\Microsoft.Common.props')" />
  <PropertyGroup>
    <Configuration Condition=" '$(Configuration)' == '' ">Debug</Configuration>
    <Platform Condition=" '$(Platform)' == '' ">AnyCPU</Platform>
    <ProductVersion>
    </ProductVersion>
    <SchemaVersion>2.0</SchemaVersion>
    <ProjectGuid>{C8D13E98-C8CD-42FB-9C79-F3EB11B0F3E8}</ProjectGuid>
    <ProjectTypeGuids>{349c5851-65df-11da-9384-00065b846f21};{fae04ec0-301f-11d3-bf4b-00c04f79efbc}</ProjectTypeGuids>
    <OutputType>Library</OutputType>
    <AppDesignerFolder>Properties</AppDesignerFolder>
    <RootNamespace>MetadataServer</RootNamespace>
    <AssemblyName>MetadataServer</AssemblyName>
    <TargetFrameworkVersion>v4.6.2</TargetFrameworkVersion>
    <MvcBuildViews>false</MvcBuildViews>
    <UseIISExpress>true</UseIISExpress>
    <IISExpressSSLPort />
    <IISExpressAnonymousAuthentication />
    <IISExpressWindowsAuthentication />
    <IISExpressUseClassicPipelineMode />
    <UseGlobalApplicationHostFile />
    <NuGetPackageImportStamp>
    </NuGetPackageImportStamp>
    <TargetFrameworkProfile />
    <Use64BitIISExpress />
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|AnyCPU' ">
    <TransformWebConfigEnabled>False</TransformWebConfigEnabled>
    <DebugSymbols>true</DebugSymbols>
    <DebugType>full</DebugType>
    <Optimize>false</Optimize>
    <OutputPath>bin\</OutputPath>
    <DefineConstants>DEBUG;TRACE</DefineConstants>
    <ErrorReport>prompt</ErrorReport>
    <WarningLevel>4</WarningLevel>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|AnyCPU' ">
    <TransformWebConfigEnabled>False</TransformWebConfigEnabled>
    <DebugType>pdbonly</DebugType>
    <Optimize>true</Optimize>
    <OutputPath>bin\</OutputPath>
    <DefineConstants>TRACE</DefineConstants>
    <ErrorReport>prompt</ErrorReport>
    <WarningLevel>4</WarningLevel>
  </PropertyGroup>
  <ItemGroup>
    <Reference Include="BouncyCastle.Crypto, Version=1.8.3.0, Culture=neutral, PublicKeyToken=0e99375e54769942">
      <HintPath>..\packages\BouncyCastle.1.8.3.1\lib\BouncyCastle.Crypto.dll</HintPath>
    </Reference>
    <Reference Include="Google.Protobuf, Version=3.6.1.0, Culture=neutral, PublicKeyToken=a7d26565bac4d604, processorArchitecture=MSIL">
      <HintPath>..\packages\Google.Protobuf.3.6.1\lib\net45\Google.Protobuf.dll</HintPath>
    </Reference>
    <Reference Include="Microsoft.CSharp" />
    <Reference Include="MySql.Data, Version=8.0.16.0, Culture=neutral, PublicKeyToken=c5687fc88969c44d, processorArchitecture=MSIL">
      <HintPath>..\packages\MySql.Data.8.0.16\lib\net452\MySql.Data.dll</HintPath>
    </Reference>
    <Reference Include="Newtonsoft.Json, Version=12.0.0.0, Culture=neutral, PublicKeyToken=30ad4fe6b2a6aeed, processorArchitecture=MSIL">
      <HintPath>..\packages\Newtonsoft.Json.12.0.2\lib\net45\Newtonsoft.Json.dll</HintPath>
    </Reference>
    <Reference Include="System" />
    <Reference Include="System.Data" />
    <Reference Include="System.Drawing" />
    <Reference Include="System.Net.Http.Formatting, Version=5.2.7.0, Culture=neutral, PublicKeyToken=31bf3856ad364e35, processorArchitecture=MSIL">
      <HintPath>..\packages\Microsoft.AspNet.WebApi.Client.5.2.7\lib\net45\System.Net.Http.Formatting.dll</HintPath>
    </Reference>
    <Reference Include="System.Web.DynamicData" />
    <Reference Include="System.Web.Entity" />
    <Reference Include="System.Web.ApplicationServices" />
    <Reference Include="System.ComponentModel.DataAnnotations" />
    <Reference Include="System.Web" />
    <Reference Include="System.Web.Abstractions" />
    <Reference Include="System.Web.Http, Version=5.2.7.0, Culture=neutral, PublicKeyToken=31bf3856ad364e35, processorArchitecture=MSIL">
      <HintPath>..\packages\Microsoft.AspNet.WebApi.Core.5.2.7\lib\net45\System.Web.Http.dll</HintPath>
    </Reference>
    <Reference Include="System.Web.Http.WebHost, Version=5.2.7.0, Culture=neutral, PublicKeyToken=31bf3856ad364e35, processorArchitecture=MSIL">
      <HintPath>..\packages\Microsoft.AspNet.WebApi.WebHost.5.2.7\lib\net45\System.Web.Http.WebHost.dll</HintPath>
    </Reference>
    <Reference Include="System.Web.Routing" />
    <Reference Include="System.Xml" />
    <Reference Include="System.Configuration" />
    <Reference Include="System.Runtime.Serialization" />
    <Reference Include="System.Net.Http">
    </Reference>
    <Reference Include="System.Net.Http.WebRequest">
    </Reference>
  </ItemGroup>
  <ItemGroup>
    <Folder Include="App_Data\" />
  </ItemGroup>
  <ItemGroup>
    <Compile Include="App_Start\WebApiConfig.cs" />
    <Compile Include="Controllers\IssuesController.cs" />
    <Compile Include="Controllers\UserController.cs" />
    <Compile Include="Models\IssueData.cs" />
    <Compile Include="Properties\Resources.Designer.cs">
      <AutoGen>True</AutoGen>
      <DesignTime>True</DesignTime>
      <DependentUpon>Resources.resx</DependentUpon>
    </Compile>
    <Content Include="Setup.sql" />
    <Compile Include="Controllers\BuildController.cs" />
    <Compile Include="Controllers\CISController.cs" />
    <Compile Include="Controllers\CommentController.cs" />
    <Compile Include="Controllers\ErrorController.cs" />
    <Compile Include="Controllers\EventController.cs" />
    <Compile Include="Controllers\LatestController.cs" />
    <Compile Include="Controllers\TelemetryController.cs" />
    <Compile Include="Global.asax.cs">
      <DependentUpon>Global.asax</DependentUpon>
    </Compile>
    <Compile Include="Models\BuildData.cs" />
    <Compile Include="Models\CommentData.cs" />
    <Compile Include="Models\EventData.cs" />
    <Compile Include="Models\EventSummary.cs" />
    <Compile Include="Models\LatestData.cs" />
    <Compile Include="Models\TelemetryErrorData.cs" />
    <Compile Include="Models\TelemetryTimingData.cs" />
    <Compile Include="Properties\AssemblyInfo.cs" />
    <Compile Include="Connectors\SqlConnector.cs" />
  </ItemGroup>
  <ItemGroup>
    <Content Include="Global.asax" />
    <Content Include="Web.config">
      <SubType>Designer</SubType>
    </Content>
    <None Include="Web.template.config">
      <SubType>Designer</SubType>
    </None>
    <None Include="Web.Debug.config">
      <DependentUpon>Web.template.config</DependentUpon>
    </None>
    <None Include="Web.Release.config">
      <DependentUpon>Web.template.config</DependentUpon>
    </None>
  </ItemGroup>
  <ItemGroup>
    <Content Include="NotForLicensees\Web.config">
      <SubType>Designer</SubType>
    </Content>
  </ItemGroup>
  <ItemGroup>
    <Content Include="packages.config" />
  </ItemGroup>
  <ItemGroup>
    <EmbeddedResource Include="Properties\Resources.resx">
      <Generator>ResXFileCodeGenerator</Generator>
      <LastGenOutput>Resources.Designer.cs</LastGenOutput>
      <SubType>Designer</SubType>
    </EmbeddedResource>
  </ItemGroup>
  <PropertyGroup>
    <VisualStudioVersion Condition="'$(VisualStudioVersion)' == ''">10.0</VisualStudioVersion>
    <VSToolsPath Condition="'$(VSToolsPath)' == ''">$(MSBuildExtensionsPath32)\Microsoft\VisualStudio\v$(VisualStudioVersion)</VSToolsPath>
  </PropertyGroup>
  <Import Project="$(MSBuildBinPath)\Microsoft.CSharp.targets" />
  <Import Project="$(VSToolsPath)\WebApplications\Microsoft.WebApplication.targets" Condition="'$(VSToolsPath)' != ''" />
  <Import Project="$(MSBuildExtensionsPath32)\Microsoft\VisualStudio\v10.0\WebApplications\Microsoft.WebApplication.targets" Condition="false" />
  <Target Name="MvcBuildViews" AfterTargets="AfterBuild" Condition="'$(MvcBuildViews)'=='true'">
    <AspNetCompiler VirtualPath="temp" PhysicalPath="$(WebProjectOutputDir)" />
  </Target>
  <ProjectExtensions>
    <VisualStudio>
      <FlavorProperties GUID="{349c5851-65df-11da-9384-00065b846f21}">
        <WebProjectProperties>
          <UseIIS>True</UseIIS>
          <AutoAssignPort>True</AutoAssignPort>
          <DevelopmentServerPort>62486</DevelopmentServerPort>
          <DevelopmentServerVPath>/</DevelopmentServerVPath>
          <IISUrl>http://localhost:62487/</IISUrl>
          <NTLMAuthentication>False</NTLMAuthentication>
          <UseCustomServer>False</UseCustomServer>
          <CustomServerUrl>
          </CustomServerUrl>
          <SaveServerSettingsInUserFile>False</SaveServerSettingsInUserFile>
        </WebProjectProperties>
      </FlavorProperties>
    </VisualStudio>
  </ProjectExtensions>
  <!-- To modify your build process, add your task inside one of the targets below and uncomment it.
       Other similar extension points exist, see Microsoft.Common.targets.
  <Target Name="BeforeBuild">
  </Target> -->
  <Target Name="BeforeBuild">
    <Message Importance="High" Text="Creating web.config from web.template.config" />
    <TransformXml Source="Web.template.config" Transform="Web.$(Configuration).config" Destination="Web.config" />
    <Message Importance="High" Text="Found NotForLicensees\Web.config, applying transform" Condition="Exists('NotForLicensees\Web.config')" />
    <TransformXml Source="Web.config" Transform="NotForLicensees\Web.config" Destination="Web.config" Condition="Exists('NotForLicensees\Web.config')" />
  </Target>
</Project>