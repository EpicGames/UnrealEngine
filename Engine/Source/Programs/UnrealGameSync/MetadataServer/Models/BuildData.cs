// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Web;

namespace MetadataServer.Models
{
	public class Link
	{
		public string Text { get; set; } = String.Empty;
		public string Url { get; set; } = String.Empty;
	}

   	public class BuildMetadata
	{
		public List<Link> Links { get; set; } = new List<Link>();
	}

	public class BuildData
	{
		public enum BuildDataResult
		{
			Starting,
			Failure,
			Warning,
			Success,
			Skipped,
		}

		public long Id;
		public int ChangeNumber;
		public string BuildType;
		public BuildDataResult Result;
		public string Url;
		public string Project;
		public string ArchivePath;
		public BuildMetadata Metadata;

		public bool IsSuccess
		{
			get { return Result == BuildDataResult.Success || Result == BuildDataResult.Warning; }
		}

		public bool IsFailure
		{
			get { return Result == BuildDataResult.Failure; }
		}
	}
}