// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Web;

namespace MetadataServer.Models
{
	public class CommentData
	{
		public long Id;
		public int ChangeNumber;
		public string UserName;
		public string Text;
		public string Project;
	}
}