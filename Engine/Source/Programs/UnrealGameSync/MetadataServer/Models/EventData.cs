// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Web;

namespace MetadataServer.Models
{
	public class EventData
	{
		public enum EventType
		{
			Syncing,

			// Reviews
			Compiles,
			DoesNotCompile,
			Good,
			Bad,
			Unknown,

			// Starred builds
			Starred,
			Unstarred,

			// Investigating events
			Investigating,
			Resolved,
		}
		
		public long Id;
		public int Change;
		public string UserName;
		public EventType Type;
		public string Project;
	}
}