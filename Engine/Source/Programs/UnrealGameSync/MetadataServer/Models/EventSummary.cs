// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Web;

namespace MetadataServer.Models
{
	public class EventSummary
	{
		public enum ReviewVerdict
		{
			Unknown,
			Good,
			Bad,
			Mixed,
		}

		public int ChangeNumber;
		public ReviewVerdict Verdict;
		public List<EventData> SyncEvents = new List<EventData>();
		public List<EventData> Reviews = new List<EventData>();
		public List<string> CurrentUsers = new List<string>();
		public EventData LastStarReview;
		public List<BuildData> Builds = new List<BuildData>();
		public List<CommentData> Comments = new List<CommentData>();
	}
}