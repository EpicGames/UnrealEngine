// Copyright Epic Games, Inc. All Rights Reserved.

using Newtonsoft.Json.Linq;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Web;

namespace MetadataServer.Models
{
	public class IssueWatcherData
	{
		public string UserName;
	}

	public class IssueBuildData
	{
		public long Id;
		public string Stream;
		public int Change;
		public string JobName;
		public string JobUrl;
		public string JobStepName;
		public string JobStepUrl;
		public string ErrorUrl;
		public int Outcome;
	}

	public class IssueBuildUpdateData
	{
		public int Outcome;
	}

	public class IssueDiagnosticData
	{
		public long? BuildId;
		public string Message;
		public string Url;
	}

	public class IssueData
	{
		public long Id;
		public DateTime CreatedAt;
		public DateTime RetrievedAt;
		public string Project;
		public string Summary;
		public string Owner;
		public string NominatedBy;
		public DateTime? AcknowledgedAt;
		public int FixChange;
		public DateTime? ResolvedAt;
		public bool bNotify;
	}

	public class IssueUpdateData
	{
		public string Summary;
		public string Owner;
		public string NominatedBy;
		public bool? Acknowledged;
		public int? FixChange;
		public bool? Resolved;
	}
}
