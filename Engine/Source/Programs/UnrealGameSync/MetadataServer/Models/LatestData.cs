// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Web;

namespace MetadataServer.Models
{
	public class LatestData
	{
		public long LastEventId { get; set; }
		public long LastCommentId { get; set; }
		public long LastBuildId { get; set; }
	}
}