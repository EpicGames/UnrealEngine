// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Web;

namespace MetadataServer.Models
{
	public class TelemetryErrorData
	{
		public enum TelemetryErrorType
		{
			Crash,
		}
		public int Id;
		public TelemetryErrorType Type;
		public string Text;
		public string UserName;
		public string Project;
		public DateTime Timestamp;
		public string Version;
		public string IpAddress;
	}
}