// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Web;

namespace MetadataServer.Models
{
	public class TelemetryTimingData
	{
		public string Action;
		public string Result;
		public string UserName;
		public string Project;
		public DateTime Timestamp;
		public float Duration;
	}
}