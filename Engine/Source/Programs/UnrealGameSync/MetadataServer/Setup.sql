CREATE DATABASE IF NOT EXISTS ugs_db;

CREATE TABLE IF NOT EXISTS ugs_db.Projects (
	Id INT NOT NULL AUTO_INCREMENT,
	`Name` NVARCHAR(128) NOT NULL UNIQUE,
    PRIMARY KEY ( Id )
);

CREATE TABLE IF NOT EXISTS ugs_db.CIS (
    Id INT NOT NULL AUTO_INCREMENT,
    ChangeNumber INT NOT NULL,
    BuildType NCHAR(32) NOT NULL,
    Result NCHAR(10) NOT NULL,
    Url VARCHAR(512) NOT NULL,
    Project VARCHAR(512) NULL,
    ProjectId INTEGER NULL,
	ArchivePath VARCHAR(512) NULL,
    PRIMARY KEY ( Id ),
    FOREIGN KEY(ProjectId) REFERENCES Projects(Id)
);

CREATE TABLE IF NOT EXISTS ugs_db.Comments (
	Id INT NOT NULL AUTO_INCREMENT,
    ChangeNumber INT NOT NULL,
    UserName VARCHAR(128) NOT NULL,
    Text VARCHAR(1024) NOT NULL,
    Project VARCHAR(128) NOT NULL,
	ProjectId INTEGER NULL,
    PRIMARY KEY ( Id ),
    FOREIGN KEY(ProjectId) REFERENCES Projects(Id)
);

CREATE TABLE IF NOT EXISTS ugs_db.Errors (
	Id INT NOT NULL AUTO_INCREMENT,
    Type VARCHAR(50) NOT NULL,
    Text VARCHAR(1024) NOT NULL,
    UserName NVARCHAR(128) NOT NULL,
    Project VARCHAR(128) NOT NULL,
    ProjectId INTEGER NULL,
    Timestamp DATETIME NOT NULL,
    Version VARCHAR(64) NOT NULL,
    IpAddress VARCHAR(64) NOT NULL,
    PRIMARY KEY ( Id ),
    FOREIGN KEY(ProjectId) REFERENCES Projects(Id)
);

CREATE TABLE IF NOT EXISTS ugs_db.Telemetry_v2 (
	Id INT NOT NULL AUTO_INCREMENT,
    Action VARCHAR(128) NOT NULL,
    Result VARCHAR(128) NOT NULL,
    UserName VARCHAR(128) NOT NULL,
	Project VARCHAR(128) NOT NULL,
    ProjectId INTEGER NULL,
    Timestamp DATETIME NOT NULL,
    Duration REAL NOT NULL,
    Version VARCHAR(64) NOT NULL,
    IpAddress VARCHAR(64) NOT NULL,
    PRIMARY KEY ( Id ),
    FOREIGN KEY(ProjectId) REFERENCES Projects(Id)
);

CREATE TABLE IF NOT EXISTS ugs_db.UserVotes (
	Id INT NOT NULL AUTO_INCREMENT,
    Changelist INT NOT NULL,
    UserName NVARCHAR(128) NOT NULL,
    Verdict NCHAR(32) NOT NULL,
    Project NVARCHAR(256) NULL,
    ProjectId INTEGER NULL,
    PRIMARY KEY ( Id ),
    FOREIGN KEY(ProjectId) REFERENCES Projects(Id)
);

CREATE TABLE IF NOT EXISTS ugs_db.Issues (
	Id INT NOT NULL AUTO_INCREMENT,
    CreatedAt DATETIME NOT NULL,
    Project NVARCHAR(64) NOT NULL,
    Summary NVARCHAR(256) NOT NULL,
    OwnerId INT NULL,
    NominatedById INT NULL,
    AcknowledgedAt DATETIME NULL,
    FixChange INT NULL,
    ResolvedAt DATETIME NULL,
    PRIMARY KEY ( Id )
);

CREATE TABLE IF NOT EXISTS ugs_db.IssueBuilds (
	Id INT NOT NULL AUTO_INCREMENT,
    IssueId INT NOT NULL,
    Stream NVARCHAR(128) NOT NULL,
    `Change` INT NOT NULL,
    JobName NVARCHAR(1024) NOT NULL,
    JobUrl NVARCHAR(1024) NOT NULL,
    JobStepName NVARCHAR(1024) NULL,
    JobStepUrl NVARCHAR(1024) NULL,
    ErrorUrl NVARCHAR(1024) NULL,
    Outcome INT NOT NULL,
    PRIMARY KEY ( Id ),
	FOREIGN KEY(IssueId) REFERENCES Issues(Id)
);

CREATE TABLE IF NOT EXISTS ugs_db.IssueDiagnostics (
	Id INT NOT NULL AUTO_INCREMENT,
	IssueId INT NOT NULL,
	BuildId INT NULL,
	Message NVARCHAR (1024) NOT NULL,
	Url NVARCHAR (1024) NULL,
    PRIMARY KEY ( Id ),
	FOREIGN KEY(IssueId) REFERENCES Issues(Id),
	FOREIGN KEY(BuildId) REFERENCES IssueBuilds(Id)
);

CREATE TABLE IF NOT EXISTS ugs_db.Users (
	Id INT NOT NULL AUTO_INCREMENT,
	`Name` NVARCHAR(128) NOT NULL UNIQUE,
    PRIMARY KEY ( Id )
);

CREATE TABLE IF NOT EXISTS ugs_db.IssueWatchers (
	IssueId INT NOT NULL,
	UserId INT NOT NULL,
    PRIMARY KEY(IssueId, UserId),
	FOREIGN KEY(IssueId) REFERENCES Issues(Id),
	FOREIGN KEY(UserId) REFERENCES Users(Id)
);

CREATE TABLE IF NOT EXISTS ugs_db.Badges (
	Id INT NOT NULL AUTO_INCREMENT,
	ChangeNumber INT NOT NULL,
	BuildType NVARCHAR(32) NOT NULL,
	Result NVARCHAR(10) NOT NULL,
	Url NVARCHAR(512) NOT NULL,
	ProjectId INT NOT NULL,
	ArchivePath NVARCHAR(512) NULL,
	PRIMARY KEY ( Id ),
	FOREIGN KEY(ProjectId) REFERENCES Projects(Id)
);