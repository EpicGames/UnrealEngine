ALTER TABLE ugs_db.Badges ADD COLUMN Metadata JSON NOT NULL DEFAULT ( JSON_OBJECT('Links', JSON_ARRAY()) );
