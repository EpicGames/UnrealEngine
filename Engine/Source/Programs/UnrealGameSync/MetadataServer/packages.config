<?xml version="1.0" encoding="utf-8"?>
<packages>
  <package id="BouncyCastle" version="1.8.3.1" targetFramework="net452" />
  <package id="Google.Protobuf" version="3.6.1" targetFramework="net452" />
  <package id="Microsoft.AspNet.WebApi" version="5.2.7" targetFramework="net45" />
  <package id="Microsoft.AspNet.WebApi.Client" version="5.2.7" targetFramework="net45" />
  <package id="Microsoft.AspNet.WebApi.Core" version="5.2.7" targetFramework="net45" />
  <package id="Microsoft.AspNet.WebApi.WebHost" version="5.2.7" targetFramework="net45" />
  <package id="MySql.Data" version="8.0.16" targetFramework="net452" />
  <package id="Newtonsoft.Json" version="12.0.2" targetFramework="net45" />
</packages>