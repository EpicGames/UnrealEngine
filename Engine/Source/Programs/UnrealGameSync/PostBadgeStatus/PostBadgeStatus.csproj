<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <NoWarn>$(NoWarn);SYSLIB0014</NoWarn>
  </PropertyGroup>
</Project>
