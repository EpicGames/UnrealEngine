// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Net;
using System.Text;
using System.Text.Json;
using System.Threading;

namespace WriteBadgeStatus
{
	class Program
	{
		static int Main(string[] Args)
		{
			// Parse all the parameters
			List<string> Arguments = new List<string>(Args);
			string? Name = ParseParam(Arguments, "Name");
			string? Change = ParseParam(Arguments, "Change");
			string? Project = ParseParam(Arguments, "Project");
			string? RestUrl = ParseParam(Arguments, "RestUrl");
			string? Status = ParseParam(Arguments, "Status");
			string? Url = ParseParam(Arguments, "Url");
			string? Metadata = ParseParam(Arguments, "Metadata");

			// Check we've got all the arguments we need (and no more)
			if (Arguments.Count > 0 || Name == null || Change == null || Project == null || RestUrl == null || Status == null || Url == null)
			{
				Console.WriteLine("Syntax:");
				Console.WriteLine("  PostBadgeStatus -Name=<Name> -Change=<CL> -Project=<DepotPath> -RestUrl=<Url> -Status=<Status> -Url=<Url> -Metadata=<json metadata>");
				Console.WriteLine("  Metadata is optional, but if provided, it has to contain array of links; format: {'Links': [{'Text': 'link text', 'Url': 'http://your.url'}]}");
				return 1;
			}

			BuildData Build = new BuildData
			{
				BuildType = Name,
				Url = Url,
				Project = Project,
				ArchivePath = ""
			};
			if (!Int32.TryParse(Change, out Build.ChangeNumber))
			{
				Console.WriteLine("Change must be an integer!");
				return 1;
			}
			if (!Enum.TryParse<BuildData.BuildDataResult>(Status, true, out Build.Result))
			{
				Console.WriteLine("Change must be Starting, Failure, Warning, Success, or Skipped!");
				return 1;
			}
			if (!String.IsNullOrWhiteSpace(Metadata))
			{
				Build.Metadata = JsonSerializer.Deserialize<JsonElement>(Metadata);
			}
			int NumRetries = 0;
			while (true)
			{
				try
				{
					return SendRequest(RestUrl, "Build", "POST", JsonSerializer.Serialize(Build));
				}
				catch (Exception ex)
				{
					if (++NumRetries <= 3)
					{
						Console.WriteLine(String.Format("An exception was thrown attempting to send the request: {0}, retrying...", ex.Message));
						// Wait 5 seconds and retry;
						Thread.Sleep(5000);
					}
					else
					{
						Console.WriteLine("Failed to POST due to multiple failures. Aborting.");
						return 1;
					}
				}
			}
		}

		static string? ParseParam(List<string> Arguments, string ParamName)
		{
			string ParamPrefix = String.Format("-{0}=", ParamName);
			for (int Idx = 0; Idx < Arguments.Count; Idx++)
			{
				if (Arguments[Idx].StartsWith(ParamPrefix, StringComparison.InvariantCultureIgnoreCase))
				{
					string ParamValue = Arguments[Idx].Substring(ParamPrefix.Length);
					Arguments.RemoveAt(Idx);
					return ParamValue;
				}
			}
			return null;
		}

		class BuildData
		{
			public enum BuildDataResult
			{
				Starting,
				Failure,
				Warning,
				Success,
				Skipped,
			}

			public int ChangeNumber;
			public required string BuildType;
			public BuildDataResult Result;
			public required string Url;
			public required string Project;
			public required string ArchivePath;
			public JsonElement? Metadata;

			public bool IsSuccess => Result == BuildDataResult.Success || Result == BuildDataResult.Warning;

			public bool IsFailure => Result == BuildDataResult.Failure;
		}

		static int SendRequest(string URI, string Resource, string Method, string? RequestBody = null, params string[] QueryParams)
		{
			// set up the query string
			StringBuilder TargetURI = new StringBuilder(String.Format("{0}/api/{1}", URI, Resource));
			if (QueryParams.Length != 0)
			{
				TargetURI.Append('?');
				for (int Idx = 0; Idx < QueryParams.Length; Idx++)
				{
					TargetURI.Append(QueryParams[Idx]);
					if (Idx != QueryParams.Length - 1)
					{
						TargetURI.Append('&');
					}
				}
			}
			HttpWebRequest Request = (HttpWebRequest)WebRequest.Create(new Uri(TargetURI.ToString()));
			Request.ContentType = "application/json";
			Request.Method = Method;

			// Add json to request body
			if (!String.IsNullOrEmpty(RequestBody))
			{
				if (Method == "POST")
				{
					byte[] bytes = Encoding.UTF8.GetBytes(RequestBody);
					using (Stream RequestStream = Request.GetRequestStream())
					{
						RequestStream.Write(bytes, 0, bytes.Length);
					}
				}
			}
			try
			{
				using (HttpWebResponse Response = (HttpWebResponse)Request.GetResponse())
				{
					string? ResponseContent = null;
					using (StreamReader ResponseReader = new System.IO.StreamReader(Response.GetResponseStream(), Encoding.UTF8))
					{
						ResponseContent = ResponseReader.ReadToEnd();
						Console.WriteLine(ResponseContent);
						return ((int)Response.StatusCode >= 200 && (int)Response.StatusCode <= 299) ? 0 : 1;
					}
				}
			}
			catch (WebException ex)
			{
				if (ex.Response != null)
				{
					throw new Exception(String.Format("Request returned status: {0}, message: {1}", ((HttpWebResponse)ex.Response).StatusCode, ex.Message));
				}
				else
				{
					throw new Exception(String.Format("Request returned message: {0}", ex.InnerException?.Message ?? ex.Message));
				}
			}
			catch (Exception ex)
			{
				throw new Exception(String.Format("Couldn't complete the request, error: {0}", ex.Message));
			}
		}
	}
}
