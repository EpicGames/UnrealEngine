// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;

namespace UnrealGameSync
{
	class ActivationListener : IDisposable
	{
		Thread? _backgroundThread;
		readonly EventWaitHandle _activateEventHandle;
		readonly EventWaitHandle _quitEventHandle;

		public event Action? OnActivate;

		public ActivationListener(EventWaitHandle inActivateEventHandle)
		{
			_activateEventHandle = inActivateEventHandle;
			_quitEventHandle = new EventWaitHandle(false, EventResetMode.ManualReset);
		}

		public void Start()
		{
			if (_backgroundThread == null)
			{
				_backgroundThread = new Thread(x => ThreadProc());
				_backgroundThread.IsBackground = true;
				_backgroundThread.Start();
			}
		}

		public void Stop()
		{
			if (_backgroundThread != null)
			{
				_quitEventHandle.Set();

				_backgroundThread.Join();
				_backgroundThread = null;
			}
		}

		public void Dispose()
		{
			Stop();

			_quitEventHandle.Dispose();
			_activateEventHandle.Dispose();
		}

		void ThreadProc()
		{
			for (; ; )
			{
				int index = EventWaitHandle.WaitAny(new WaitHandle[] { _activateEventHandle, _quitEventHandle }, Timeout.Infinite);
				if (index == 0)
				{
					OnActivate?.Invoke();
				}
				else
				{
					break;
				}
			}
		}
	}
}
