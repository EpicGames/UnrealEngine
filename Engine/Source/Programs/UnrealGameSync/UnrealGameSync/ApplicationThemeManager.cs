// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Drawing;
using System.Linq;
using System.Runtime.InteropServices;
using System.Windows.Forms;
using Microsoft.Win32;
using UnrealGameSync.Controls;

namespace UnrealGameSync
{
	public sealed class InvertibleImage : IDisposable
	{
		readonly bool _disposeImage;
		Image? _standardImage;
		Image? _invertedImage;

		public InvertibleImage(Image image, bool disposeImage)
		{
			_disposeImage = disposeImage;
			_standardImage = image;

			InitInvertedImage();
		}

		public Image? ResolveImage()
		{
			bool isInverted = ProgramApplicationContext.GetTheme().ShouldInvertColorBrightness;
			return isInverted ? _invertedImage : _standardImage;
		}
		
		public void Dispose()
		{
			if (_disposeImage && _standardImage != null)
			{
				_standardImage.Dispose();
			}
			_standardImage = null;

			_invertedImage?.Dispose();
			_invertedImage = null;
		}
		
		private void InitInvertedImage()
		{
			if (_standardImage != null)
			{
				Bitmap invertedLogoBitmap = new Bitmap(_standardImage);

				for (int x = 0; x < invertedLogoBitmap.Width; ++x)
				{
					for (int y = 0; y < invertedLogoBitmap.Height; ++y)
					{
						invertedLogoBitmap.SetPixel(x, y, ApplicationTheme.InvertColorBrightness(invertedLogoBitmap.GetPixel(x, y)));
					}
				}

				_invertedImage = invertedLogoBitmap;
			}
		}
	}
	
	public class InvertibleColor
	{
		public Color StandardColor { get; init; }
		public Color InvertedColor { get; init; }

		public InvertibleColor(Color color)
		{
			StandardColor = color;
			InvertedColor = ApplicationTheme.InvertColorBrightness(color);
		}

		public Color ResolveColor()
		{
			bool isInverted = ProgramApplicationContext.GetTheme().ShouldInvertColorBrightness;
			return isInverted ? InvertedColor : StandardColor;
		}
	}
	
	interface ICustomThemeHandler
	{
		public void ApplyTheme(ApplicationThemeManager themeManager);
	}

	public sealed class ApplicationThemeManager : IDisposable
	{
#pragma warning disable CA1822 // Make property static
		public ApplicationTheme Theme => ProgramApplicationContext.GetTheme();
#pragma warning restore CA1822
		
		readonly InvertibleImage _dropListIcon = new InvertibleImage(global::UnrealGameSync.Properties.Resources.DropList, false);

		public ApplicationThemeManager()
		{
			//UpdateDefaultTheme();
		}
		
		public void Dispose()
		{
			_dropListIcon.Dispose();
		}
		
		[DllImport("dwmapi.dll")]
		private static extern int DwmSetWindowAttribute(IntPtr hwnd, int attr, ref int attrValue, int attrSize);

		[DllImport("uxtheme.dll", CharSet = CharSet.Unicode)]
		private static extern int SetWindowTheme(IntPtr hWnd, string? pszSubAppName, string? pszSubIdList);
		
		[System.Diagnostics.CodeAnalysis.SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Not currently used")]
		private static void UpdateDefaultTheme()
		{
			// Set the default theme based on the user's system settings
			int? lightThemeValue = (int?)Registry.GetValue("HKEY_CURRENT_USER\\SOFTWARE\\Microsoft\\Windows\\CurrentVersion\\Themes\\Personalize", "AppsUseLightTheme", -1);
			bool isDarkDefault = (lightThemeValue ?? -1) == 0;

			ApplicationTheme.DefaultThemeId = isDarkDefault ? ApplicationTheme.DarkThemeId : ApplicationTheme.LightThemeId;
		}
		
		private void ApplyDarkMode(Control control)
		{
			if (!control.IsHandleCreated)
			{
				return;
			}
			
			if (Theme.UseDarkMode)
			{
				string darkTheme = "darkmode_explorer";

				switch (control)
				{
					// special case for the custom list since this makes the group header legible (but the scrollbar is white)
					// where without this, "darkmode_explorer" will make the group header a dark blue (but have a dark scrollbox)
					case CustomListViewControl:
						darkTheme = "darkmode_itemsview";
						break;
					// Properly themes the full dropdown
					case ComboBox:
						darkTheme = "darkmode_cfd";
						break;
				}
				
				SetWindowTheme(control.Handle, darkTheme, null);
			}
			else
			{
				SetWindowTheme(control.Handle, null, null);
			}

			if (Environment.OSVersion.Version.Major >= 10 && Environment.OSVersion.Version.Build >= 17763)
			{
				const int DWMWA_USE_IMMERSIVE_DARK_MODE_BEFORE_20H1 = 19;
				const int DWMWA_USE_IMMERSIVE_DARK_MODE = 20;

				int attribute = (Environment.OSVersion.Version.Build >= 18985) ? DWMWA_USE_IMMERSIVE_DARK_MODE : DWMWA_USE_IMMERSIVE_DARK_MODE_BEFORE_20H1;
				int useImmersiveDarkMode = Theme.UseDarkMode ? 1 : 0;

				DwmSetWindowAttribute(control.Handle, attribute, ref useImmersiveDarkMode, sizeof(int));
			}
		}

		public void ApplyThemeRecursively()
		{
			foreach (Form form in Application.OpenForms)
			{
				ApplyThemeRecursively(form);
			}
		}

		private void ControlHandleCreated(object? sender, EventArgs args)
		{
			if (sender is Control control)
			{
				ApplyThemeRecursively(control);
			}
		}

		private void ControlEnabledChanged(object? sender, EventArgs args)
		{
			if (sender is Control control)
			{
				ApplyThemeRecursively(control);
			}
		}

		private void ControlAdded(object? sender, ControlEventArgs args)
		{
			if (args.Control != null)
			{
				ApplyThemeRecursively(args.Control);
			}
		}

		private void ControlHandleDestroyed(object? sender, EventArgs args)
		{
			if (sender is Control control)
			{
				UnitialiseControl(control);
			}
		}

		private void ToolStripDropDownClosed(object? sender, EventArgs args)
		{
			if (sender is ToolStripDropDown menu)
			{
				UninitialiseToolStripDropDown(menu);
			}
		}

		private void UninitialiseToolStripDropDown(ToolStripDropDown dropDown)
		{
			dropDown.Closed -= ToolStripDropDownClosed;
			UnitialiseControl(dropDown);

			foreach (ToolStripDropDownItem item in dropDown.Items.OfType<ToolStripDropDownItem>())
			{
				UninitialiseToolStripDropDown(item.DropDown);
			}
		}

		private void UnitialiseControl(Control control)
		{
			if (_registeredControls.Remove(control))
			{
				control.HandleCreated -= ControlHandleCreated;
				control.EnabledChanged -= ControlEnabledChanged;
				control.ControlAdded -= ControlAdded;
				control.HandleDestroyed -= ControlHandleDestroyed;

				foreach (Control childControl in control.Controls)
				{
					UnitialiseControl(childControl);
				}
			}
		}

		private readonly HashSet<Control> _registeredControls = new HashSet<Control>();

		public void ApplyThemeRecursively(Control control)
		{
			if ( _registeredControls.Contains(control) == false )
			{
				_registeredControls.Add(control);

				control.HandleCreated += ControlHandleCreated;
				control.ControlAdded += ControlAdded;
				control.HandleDestroyed += ControlHandleDestroyed;
				control.EnabledChanged += ControlEnabledChanged;

				if (control is ToolStripDropDown menu)
				{
					menu.Closed += ToolStripDropDownClosed;
				}
			}

			if (control is ICustomThemeHandler themeHandler)
			{
				themeHandler.ApplyTheme(this);
			}
			else
			{
				DefaultApplyThemeToControl(control);
			}

			ApplyThemeRecursively(control.Controls);
		}

		public void ApplyThemeRecursively(Control.ControlCollection controls)
		{
			foreach (Control control in controls)
			{
				ApplyThemeRecursively(control);
			}
		}

		public void ApplyThemeRecursively(ToolStripItemCollection items)
		{
			foreach (ToolStripItem item in items)
			{
				ApplyThemeRecursively(item);
			}
		}

		public void ApplyThemeRecursively(ToolStripItem item)
		{
			item.BackColor = Theme.BackgroundColor;
			item.ForeColor = Theme.TextColor;

			if (item is ToolStripDropDownItem dropDownItem)
			{
				ApplyThemeRecursively(dropDownItem.DropDown);
			}
		}

		public void ApplyThemeRecursively(ToolTip item)
		{
			item.BackColor = Theme.BackgroundColor;
			item.ForeColor = Theme.TextColor;
		}

		public void DefaultApplyThemeToControl(Control control)
		{
			control.BackColor = Theme.BackgroundColor;
			control.ForeColor = Theme.ForegroundColor;
			ApplyDarkMode(control);

			switch (control)
			{
				case UpDownBase upDownControl:
					upDownControl.BorderStyle = Theme.UseFlatControls ? BorderStyle.None : BorderStyle.Fixed3D;
					break;
				case LinkLabel linkLabel:
					control.ForeColor = control.Enabled ? Theme.TextColor : Theme.DimmedText;
					linkLabel.LinkColor = Theme.LinkColor;
					linkLabel.ActiveLinkColor = Theme.LinkColorActive;
					break;
				case Label:
				case GroupBox:
					control.ForeColor = control.Enabled ? Theme.TextColor : Theme.DimmedText;
					break;
				case Button button:
					control.BackColor = Theme.InputBackgroundColor;
					control.ForeColor = control.Enabled ? Theme.TextColor : Theme.DimmedText;

					button.FlatStyle = Theme.UseFlatControls ? FlatStyle.Flat : FlatStyle.Standard;
					button.FlatAppearance.BorderColor = Theme.InputBorderColor;

					if ("dropdown".Equals(button.Tag))
					{
						button.Image = _dropListIcon.ResolveImage();
					}
					break;
				case TextBox textBox:
					control.BackColor = Theme.InputBackgroundColor;
					control.ForeColor = control.Enabled ? Theme.TextColor : Theme.DimmedText;
					textBox.BorderStyle = Theme.UseFlatControls ? BorderStyle.FixedSingle : BorderStyle.Fixed3D;
					break;
				case ComboBox comboBox:
					control.BackColor = Theme.InputBackgroundColor;
					control.ForeColor = control.Enabled ? Theme.TextColor : Theme.DimmedText;

					if (control.Focused == false && comboBox.DropDownStyle != ComboBoxStyle.DropDownList)
					{
						comboBox.SelectionLength = 0;
					}

					break;
				case ReadOnlyCheckbox readOnlyCheckbox:
					readOnlyCheckbox.ForeColor = readOnlyCheckbox.ReadOnly ? Theme.DimmedText : Theme.TextColor;
					break;
				case CheckBox checkBox:
					control.ForeColor = control.Enabled ? Theme.TextColor : Theme.DimmedText;
					control.ForeColor = checkBox.AutoCheck ? Theme.TextColor : Theme.DimmedText;
					break;
				case ListBox:
					control.BackColor = Theme.InputBackgroundColor;
					control.ForeColor = control.Enabled ? Theme.TextColor : Theme.DimmedText;
					break;
				case ToolStrip toolStrip:
					control.ForeColor = control.Enabled ? Theme.TextColor : Theme.DimmedText;
					if (toolStrip.Renderer is not ThemedToolStripRenderer)
					{
						toolStrip.Renderer = new ThemedToolStripRenderer(this);
					}
					ApplyThemeRecursively(toolStrip.Items);
					break;
				case ListView:
					control.BackColor = Theme.InputBackgroundColor;
					control.ForeColor = control.Enabled ? Theme.TextColor : Theme.DimmedText;
					break;
				case Panel panel:
					panel.BorderStyle = BorderStyle.None;
					break;
				case TreeView:
					control.ForeColor = control.Enabled ? Theme.TextColor : Theme.DimmedText;
					break;
				case DataGridView dataGridView:
					dataGridView.BackgroundColor = Theme.InputBackgroundColor;
					dataGridView.DefaultCellStyle.BackColor = Theme.InputBackgroundColor;
					dataGridView.ForeColor = control.Enabled ? Theme.TextColor : Theme.DimmedText;
					break;
			}
		}

		private class ThemedToolStripRenderer(ApplicationThemeManager themeManager) : ToolStripProfessionalRenderer(new ThemedColorTable(themeManager))
		{
			protected override void OnRenderImageMargin(ToolStripRenderEventArgs e)
			{
				Color imageMargeBrushColor = themeManager.Theme.BackgroundColor;
				using (Brush imageMarginBrush = new SolidBrush(imageMargeBrushColor))
				{
					e.Graphics.FillRectangle(imageMarginBrush, e.AffectedBounds);
				}
			}

			protected override void OnRenderArrow(ToolStripArrowRenderEventArgs e)
			{
				if (e.Item != null)
				{
					e.ArrowColor = e.Item.Enabled ? themeManager.Theme.TextColor : themeManager.Theme.DimmedText;
				}
				
				base.OnRenderArrow(e);
			}
			
			private class ThemedColorTable(ApplicationThemeManager themeManager) : ProfessionalColorTable
			{
				public override Color MenuItemSelectedGradientBegin => themeManager.Theme.MenuHighlightBGColor;
				public override Color MenuItemSelectedGradientEnd => themeManager.Theme.MenuHighlightBGColor;
				public override Color MenuItemBorder => themeManager.Theme.MenuHighlightBorderColor;
				public override Color CheckBackground => themeManager.Theme.MenuHighlightBGColor;
				public override Color CheckPressedBackground => themeManager.Theme.MenuHighlightBGColor;
				public override Color CheckSelectedBackground => themeManager.Theme.MenuHighlightBGColor;
				public override Color ButtonSelectedBorder => themeManager.Theme.MenuHighlightBorderColor;
			}
		}
	}
}
