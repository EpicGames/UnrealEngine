// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;
using UnrealGameSync.Forms;

namespace UnrealGameSync
{
	class ArtifactDownload
	{
		class CopyProgressAdapter : IProgress<IExtractStats>
		{
			readonly IProgress<string> _inner;

			public CopyProgressAdapter(IProgress<string> inner)
			{
				_inner = inner;
			}

			public void Report(IExtractStats value)
			{
				_inner.Report($"Copied {value.NumFiles} files ({value.ExtractSize / (1024.0 * 1024.0):n1}mb, {value.ExtractRate / (1024.0 * 1024.0):n1}mb/s)...");
			}
		}

		static async Task DoSyncAsync(Uri baseUri, RefName refName, DirectoryReference outputDir, IProgress<string> progress, CancellationToken cancellationToken)
		{
			DirectoryReference stateFolder = DirectoryReference.Combine(outputDir, ".ugs");
			DirectoryReference.CreateDirectory(stateFolder);

			using (ILoggerProvider loggerProvider = Logging.CreateLoggerProvider(FileReference.Combine(stateFolder, "sync.log")))
			{
				Uri serverUrl = new Uri(baseUri.GetLeftPart(UriPartial.Authority));

				ServiceCollection services = new ServiceCollection();
				services.AddLogging(builder =>
				{
					builder.AddProvider(loggerProvider);
					builder.SetMinimumLevel(LogLevel.Trace);
				});
				services.AddHorde(options =>
				{
					options.ServerUrl = serverUrl;
					options.AllowAuthPrompt = true;
				});

				await using ServiceProvider serviceProvider = services.BuildServiceProvider();

				IHordeClient hordeClient = serviceProvider.GetRequiredService<IHordeClient>();
				IStorageNamespace storageNamespace = hordeClient.GetStorageNamespace(baseUri.AbsolutePath);

				progress.Report("Connecting to server...");

				IBlobRef<DirectoryNode>? blobRef = await storageNamespace.ReadRefAsync<DirectoryNode>(refName, cancellationToken: cancellationToken);

				progress.Report("Starting...");
				ILogger logger = serviceProvider.GetRequiredService<ILogger<ArtifactDownload>>();

				ExtractOptions options = new ExtractOptions();
				options.Progress = new CopyProgressAdapter(progress);
				options.ProgressUpdateFrequency = TimeSpan.FromSeconds(0.2);

				await blobRef.ExtractAsync(outputDir.ToDirectoryInfo(), options, serviceProvider.GetRequiredService<ILogger<ArtifactDownload>>(), cancellationToken);

				logger.LogInformation("Download complete");
			}
		}

		static async Task DoSyncZenAsync(Uri baseUri, DirectoryReference outputDir, IProgress<string> progress, CancellationToken cancellationToken)
		{
			DirectoryReference stateFolder = DirectoryReference.Combine(outputDir, ".ugs");
			DirectoryReference.CreateDirectory(stateFolder);

			using (ILoggerProvider loggerProvider = Logging.CreateLoggerProvider(FileReference.Combine(stateFolder, "sync.log")))
			{
				ServiceCollection services = new ServiceCollection();
				services.AddLogging(builder =>
				{
					builder.AddProvider(loggerProvider);
					builder.SetMinimumLevel(LogLevel.Trace);
				});
				services.AddCloudStorage();

				await using ServiceProvider serviceProvider = services.BuildServiceProvider();

				ICloudStorage cloudStorage = serviceProvider.GetRequiredService<ICloudStorage>();
				ILogger logger = serviceProvider.GetRequiredService<ILogger<ArtifactDownload>>();
				progress.Report("Connecting to server...");
				bool failure = await cloudStorage.DownloadBuildFromUriAsync(baseUri, outputDir, progress, cancellationToken);

				if (!failure)
				{
					logger.LogInformation("Download complete");
				}
				else
				{
					logger.LogError("Failed to delete from {Url} to {OutputDirectory}", baseUri, outputDir);
				}
			}
		}

		public static void RegisterFileAssociations(string application)
		{
			string progId = "UnrealGameSync.Artifact";

			using (RegistryKey baseKey = Registry.CurrentUser.CreateSubKey("SOFTWARE\\Classes"))
			{
				using (RegistryKey extensionKey = baseKey.CreateSubKey(".uartifact"))
				{
					extensionKey.SetValue("", progId);
				}

				using (RegistryKey progKey = baseKey.CreateSubKey(progId))
				{
					progKey.SetValue("", "Horde artifact descriptor");

					using (RegistryKey defaultIconKey = progKey.CreateSubKey("DefaultIcon"))
					{
						defaultIconKey.SetValue("", $"\"{application}\",0");
					}

					using (RegistryKey shellKey = progKey.CreateSubKey("Shell"))
					{
						using (RegistryKey openKey = shellKey.CreateSubKey("open"))
						{
							using RegistryKey commandKey = openKey.CreateSubKey("command");
							commandKey.SetValue("", $"\"{application}\" -artifact=\"%1\"");
						}
					}
				}
			}
		}

		public static bool ProcessCommandLine(string[] args)
		{
			const string ArtifactArg = "-artifact=";

			string? artifactFile = args.FirstOrDefault(x => x.StartsWith(ArtifactArg, StringComparison.OrdinalIgnoreCase));
			if (artifactFile == null)
			{
				return false;
			}

			try
			{
				Download(artifactFile.Substring(ArtifactArg.Length));
			}
			catch (Exception ex)
			{
				MessageBox.Show(ex.ToString());
			}

			return true;
		}

		static void Download(string artifactFile)
		{
			byte[] data = File.ReadAllBytes(artifactFile);

			ArtifactDescriptor descriptor = ArtifactDescriptor.Deserialize(data);

			string defaultOutputDir = Directory.GetCurrentDirectory();

			DirectoryReference? outputDir;
			if (!DownloadSettingsWindow.Show(descriptor.BaseUrl.ToString(), defaultOutputDir, out outputDir))
			{
				return;
			}

			try
			{
				DownloadProgressWindow.Execute((p, ctx) =>
				{
					if (descriptor.Backend.Type == ArtifactBackend.BackendTypeEnum.Horde)
					{
						return DoSyncAsync(descriptor.BaseUrl, descriptor.RefName, outputDir, p, ctx);
					}
					else if (descriptor.Backend.Type == ArtifactBackend.BackendTypeEnum.Zen)
					{
						return DoSyncZenAsync(descriptor.BaseUrl, outputDir, p, ctx);
					}
					else
					{
						throw new NotImplementedException($"Unknown backend type {descriptor.Backend.Type}");
					}
				}, CancellationToken.None);
			}
			catch (OperationCanceledException)
			{
				return;
			}
			catch (Exception ex)
			{
				MessageBox.Show($"An error occurred while downloading the specified item.\n\n{ex}");
				return;
			}

			ProcessStartInfo startInfo = new ProcessStartInfo();
			startInfo.FileName = "explorer.exe";
			startInfo.ArgumentList.Add(outputDir.ToString());
			startInfo.UseShellExecute = true;
			using Process? _ = Process.Start(startInfo);
		}
	}
}
