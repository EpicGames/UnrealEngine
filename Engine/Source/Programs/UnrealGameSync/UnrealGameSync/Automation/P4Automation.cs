// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	static class P4Automation
	{
		public static IPerforceSettings GetConnectionSettings()
		{
			// Read the settings
			LauncherSettings launcherSettings = new LauncherSettings();
			launcherSettings.Read();

			return Utility.OverridePerforceSettings(PerforceSettings.Default, launcherSettings.PerforceServerAndPort, launcherSettings.PerforceUserName);
		}

		public static Task<string> PrintToTempFile(IPerforceConnection? connection, string depotPath, ILogger logger)
		{
			return PrintToTempFileAsync(connection, depotPath, logger, CancellationToken.None);
		}

		public static async Task<string> PrintToTempFileAsync(IPerforceConnection? connection, string depotPath, ILogger logger, CancellationToken cancellationToken)
		{
			bool createNewConnection = (connection == null);
			try
			{
				if (connection == null)
				{
					IPerforceSettings settings = GetConnectionSettings();
					connection = await PerforceConnection.CreateAsync(logger);
				}

				string depotFileName = Path.GetFileName(depotPath);

				// Reorder CL and extension
				int index = depotFileName.IndexOf('@', StringComparison.Ordinal);
				if (index == -1)
				{
					depotFileName += "@Latest";
					index = depotFileName.IndexOf('@', StringComparison.Ordinal);
				}

				string cl = depotFileName.Substring(index + 1);
				string fileName = depotFileName.Substring(0, index);
				string tempFileName = String.Format("{0}@{1}{2}", Path.GetFileNameWithoutExtension(fileName), cl, Path.GetExtension(fileName));

				tempFileName = Path.Combine(Path.GetTempPath(), tempFileName);
				await connection.PrintAsync(tempFileName, depotFileName, cancellationToken);

				return tempFileName;
			}
			finally
			{
				// If we created a new connection, tear it down now.
				if (createNewConnection)
				{
					connection?.Dispose();
				}
			}
		}
	}
}