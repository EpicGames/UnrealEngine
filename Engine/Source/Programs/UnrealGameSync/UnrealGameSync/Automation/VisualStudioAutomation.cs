// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Runtime.InteropServices.ComTypes;
using System.Windows.Forms;
using EnvDTE;
using Microsoft.VisualStudio.Setup.Configuration;

namespace UnrealGameSync
{

	static class VisualStudioAutomation
	{
		public static bool OpenFile(string fileName, out string? errorMessage, int line = -1)
		{
			errorMessage = null;

			// first try to open via DTE
			DTE? dte = VisualStudioAccessor.GetDte();
			if (dte != null)
			{
				dte.ItemOperations.OpenFile(fileName);
				dte.MainWindow.Activate();

				if (line != -1)
				{
					(dte.ActiveDocument.Selection as TextSelection)?.GotoLine(line);
				}

				return true;
			}

			// unable to get DTE connection, so launch nesw VS instance
			string arguments = String.Format("\"{0}\"", fileName);
			if (line != -1)
			{
				arguments += String.Format(" /command \"edit.goto {0}\"", line);
			}

			// Launch new visual studio instance
			if (!VisualStudioAccessor.LaunchVisualStudio(arguments, out errorMessage))
			{
				return false;
			}

			return true;
		}
	}

	/// <summary>
	/// Visual Studio automation accessor
	/// </summary>
	static class VisualStudioAccessor
	{
		public static bool LaunchVisualStudio(string arguments, out string errorMessage)
		{
			errorMessage = "";
			VisualStudioInstallation? install = VisualStudioInstallations.GetPreferredInstallation();

			if (install == null || install.DevEnvPath == null)
			{
				errorMessage = String.Format("Unable to get Visual Studio installation");
				return false;
			}
			try
			{
				using System.Diagnostics.Process vsProcess = new System.Diagnostics.Process { StartInfo = new ProcessStartInfo(install.DevEnvPath, arguments) };
				vsProcess.Start();
				vsProcess.WaitForInputIdle();
			}
			catch (Exception ex)
			{
				errorMessage = ex.Message;
				return false;
			}

			return true;

		}

		[STAThread]
		public static DTE? GetDte()
		{
			IRunningObjectTable table;
			if (Succeeded(GetRunningObjectTable(0, out table)) && table != null)
			{
				IEnumMoniker monikersTable;
				table.EnumRunning(out monikersTable);

				if (monikersTable == null)
				{
					return null;
				}

				monikersTable.Reset();

				// Look for all visual studio instances in the ROT
				IMoniker[] monikers = new IMoniker[1];
				while (monikersTable.Next(1, monikers, IntPtr.Zero) == 0)
				{
					IBindCtx bindContext;
					string outDisplayName;
					IMoniker currentMoniker = monikers[0];

					if (!Succeeded(CreateBindCtx(0, out bindContext)))
					{
						continue;
					}

					try
					{
						currentMoniker.GetDisplayName(bindContext, null, out outDisplayName);
						if (String.IsNullOrEmpty(outDisplayName) || !IsVisualStudioDteMoniker(outDisplayName))
						{
							continue;
						}
					}
					catch (UnauthorizedAccessException)
					{
						// Some ROT objects require elevated permissions
						continue;
					}

					object comObject;
					if (!Succeeded(table.GetObject(currentMoniker, out comObject)))
					{
						continue;
					}

					return comObject as DTE;
				}
			}

			return null;

		}

		static bool IsVisualStudioDteMoniker(string inName)
		{
			IReadOnlyList<VisualStudioInstallation> installs = VisualStudioInstallations.Installs;

			for (int idx = 0; idx < installs.Count; idx++)
			{
				string? moniker = installs[idx].RotMoniker;
				if (moniker != null && inName.StartsWith(moniker, StringComparison.Ordinal))
				{
					return true;
				}
			}

			return false;
		}

		static bool Succeeded(int result)
		{
			return result >= 0;
		}

		[DllImport("ole32.dll")]
		public static extern int CreateBindCtx(int reserved, out IBindCtx bindCtx);

		[DllImport("ole32.dll")]
		public static extern int GetRunningObjectTable(int reserved, out IRunningObjectTable rot);
	}

	class VisualStudioInstallation
	{
		/// <summary>
		/// Base directory for the installation
		/// </summary>
		public string? BaseDir { get; set; }

		/// <summary>
		/// Path of the devenv executable
		/// </summary>
		public string? DevEnvPath { get; set; }

		/// <summary>
		/// Visual Studio major version number
		/// </summary>
		public int MajorVersion { get; set; }

		/// <summary>
		/// Running Object Table moniker for this installation
		/// </summary>
		public string? RotMoniker { get; set; }
	}

	/// <summary>
	///  
	/// </summary>
	static class VisualStudioInstallations
	{

		public static VisualStudioInstallation? GetPreferredInstallation(int majorVersion = 0)
		{

			if (s_cachedInstalls.Count == 0)
			{
				return null;
			}

			if (majorVersion == 0)
			{
				return s_cachedInstalls.First();
			}

			VisualStudioInstallation? installation = s_cachedInstalls.FirstOrDefault(install => { return install.MajorVersion == majorVersion; });

			return installation;

		}

		public static IReadOnlyList<VisualStudioInstallation> Installs => s_cachedInstalls;

		public static void Refresh()
		{
			GetVisualStudioInstallations();
		}

		static List<VisualStudioInstallation> GetVisualStudioInstallations()
		{
			s_cachedInstalls.Clear();

			try
			{
				SetupConfiguration setup = new SetupConfiguration();
				IEnumSetupInstances enumerator = setup.EnumAllInstances();

				ISetupInstance[] instances = new ISetupInstance[1];
				for (; ; )
				{
					int numFetched;
					enumerator.Next(1, instances, out numFetched);

					if (numFetched == 0)
					{
						break;
					}

					ISetupInstance2 instance = (ISetupInstance2)instances[0];
					if ((instance.GetState() & InstanceState.Local) == InstanceState.Local)
					{
						string versionString = instance.GetInstallationVersion();
						string[] components = versionString.Split('.');

						if (components.Length == 0)
						{
							continue;
						}

						int majorVersion;
						string installationPath = instance.GetInstallationPath();
						string devEnvPath = Path.Combine(installationPath, "Common7\\IDE\\devenv.exe");

						if (!Int32.TryParse(components[0], out majorVersion) || (majorVersion != 15 && majorVersion != 16))
						{
							continue;
						}

						if (!File.Exists(devEnvPath))
						{
							continue;
						}

						VisualStudioInstallation installation = new VisualStudioInstallation() { BaseDir = installationPath, DevEnvPath = devEnvPath, MajorVersion = majorVersion, RotMoniker = String.Format("!VisualStudio.DTE.{0}.0", majorVersion) };

						s_cachedInstalls.Add(installation);
					}
				}
			}
			catch (Exception ex)
			{
				MessageBox.Show(String.Format("Exception while finding Visual Studio installations {0}", ex.Message));
			}

			// prefer newer versions
			s_cachedInstalls.Sort((a, b) => { return -a.MajorVersion.CompareTo(b.MajorVersion); });

			return s_cachedInstalls;
		}

		static VisualStudioInstallations()
		{
			GetVisualStudioInstallations();
		}

		static readonly List<VisualStudioInstallation> s_cachedInstalls = new List<VisualStudioInstallation>();
	}
}