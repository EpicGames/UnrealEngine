// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.IO.Pipes;
using System.Net;
using System.Net.Sockets;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;

namespace UnrealGameSync
{
	enum AutomationRequestType
	{
		SyncProject = 0,
		FindProject = 1,
		OpenProject = 2,
		ExecCommand = 3,
		OpenIssue = 4,
		RunCustomTool = 5,
	}

	class AutomationRequestInput
	{
		public AutomationRequestType Type;
		public byte[] Data;

		public AutomationRequestInput(AutomationRequestType type, byte[] data)
		{
			Type = type;
			Data = data;
		}

		public static AutomationRequestInput Read(Stream inputStream)
		{
			using BinaryReader reader = new BinaryReader(inputStream);

			int type = reader.ReadInt32();
			int inputSize = reader.ReadInt32();
			byte[] input = reader.ReadBytes(inputSize);

			return new AutomationRequestInput((AutomationRequestType)type, input);
		}

		public void Write(Stream outputStream)
		{
			using BinaryWriter writer = new BinaryWriter(outputStream, Encoding.UTF8, true);

			writer.Write((int)Type);
			writer.Write(Data.Length);
			writer.Write(Data);
		}
	}

	enum AutomationRequestResult
	{
		Ok,
		Invalid,
		Busy,
		Canceled,
		Error,
		NotFound
	}

	class AutomationRequestOutput
	{
		public AutomationRequestResult Result;
		public byte[] Data;

		public AutomationRequestOutput(AutomationRequestResult result)
		{
			Result = result;
			Data = Array.Empty<byte>();
		}

		public AutomationRequestOutput(AutomationRequestResult result, byte[] data)
		{
			Result = result;
			Data = data;
		}

		public static AutomationRequestOutput Read(Stream inputStream)
		{
			using (BinaryReader reader = new BinaryReader(inputStream))
			{
				AutomationRequestResult result = (AutomationRequestResult)reader.ReadInt32();
				int dataSize = reader.ReadInt32();
				byte[] data = reader.ReadBytes(dataSize);
				return new AutomationRequestOutput(result, data);
			}
		}

		public void Write(Stream outputStream)
		{
			using (BinaryWriter writer = new BinaryWriter(outputStream))
			{
				writer.Write((int)Result);
				writer.Write(Data.Length);
				writer.Write(Data);
			}
		}
	}

	class AutomationRequest : IDisposable
	{
		public AutomationRequestInput Input;
		public AutomationRequestOutput? Output;
		public ManualResetEventSlim Complete;

		public AutomationRequest(AutomationRequestInput input)
		{
			Input = input;
			Complete = new ManualResetEventSlim(false);
		}

		public void SetOutput(AutomationRequestOutput output)
		{
			Output = output;
			Complete?.Set();
		}

		public void Dispose()
		{
			Complete.Dispose();
		}
	}

	class AutomationServer : IAsyncDisposable, IDisposable
	{
		static readonly UnicodeEncoding _streamEncoding = new UnicodeEncoding();

		readonly CancellationTokenSource _cancellationSource = new CancellationTokenSource();

		const string IpcChannel = @"\.\pipe\UGSChannel";
		readonly Task? _ipcTask;

		public const int DefaultPortNumber = 30422;
		readonly Task? _tcpTask;

		readonly Action<AutomationRequest> _postRequest;

		readonly ILogger _logger;

		public AutomationServer(Action<AutomationRequest> postRequest, string? uri, ILogger<AutomationServer> logger)
		{
			_postRequest = postRequest;
			_logger = logger;

			try
			{
				// IPC named pipe
				_ipcTask = Task.Run(() => RunIpcAsync(uri, _cancellationSource.Token), _cancellationSource.Token);

				// TCP listener setup
				int portNumber = GetPortNumber();
				if (portNumber > 0)
				{
					try
					{
						_tcpTask = Task.Run(() => RunTcpAsync(portNumber, _cancellationSource.Token), _cancellationSource.Token);
					}
					catch (Exception ex)
					{
						logger.LogError(ex, "Unable to start automation server tcp listener");
					}
				}
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Unable to start automation server");
			}
		}

		public static void SetPortNumber(int portNumber)
		{
			if (portNumber <= 0)
			{
				Utility.DeleteRegistryKey(Registry.CurrentUser, "Software\\Epic Games\\UnrealGameSync", "AutomationPort");
			}
			else
			{
				Registry.SetValue("HKEY_CURRENT_USER\\Software\\Epic Games\\UnrealGameSync", "AutomationPort", portNumber);
			}
		}

		public static int GetPortNumber()
		{
			object? portValue = Registry.GetValue("HKEY_CURRENT_USER\\Software\\Epic Games\\UnrealGameSync", "AutomationPort", null);
			if (portValue != null && portValue is int portValueInt)
			{
				return portValueInt;
			}
			else
			{
				return -1;
			}
		}

		async Task RunIpcAsync(string? commandLineUri, CancellationToken cancellationToken)
		{
			// Handle main process command line URI request
			if (!String.IsNullOrEmpty(commandLineUri))
			{
				HandleUri(commandLineUri);
			}

			using NamedPipeServerStream ipcStream = new NamedPipeServerStream(IpcChannel, PipeDirection.In, 1, PipeTransmissionMode.Message, PipeOptions.Asynchronous);
			while (!cancellationToken.IsCancellationRequested)
			{
				try
				{
					await ipcStream.WaitForConnectionAsync(cancellationToken).ConfigureAwait(false);
					try
					{
						_logger.LogInformation("Accepted Uri connection");

						// Read URI
						string uri = ReadString(ipcStream);

						_logger.LogInformation("Received Uri: {Uri}", uri);

						HandleUri(uri);
					}
					finally
					{
						ipcStream.Disconnect();
					}
				}
				catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
				{
					break;
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Error during automation connection");
				}
			}
		}

		async Task RunTcpAsync(int portNumber, CancellationToken cancellationToken)
		{
			using TcpListener listener = new TcpListener(IPAddress.Loopback, portNumber);
			using (IDisposable disposable = cancellationToken.Register(() => listener.Stop()))
			{
				listener.Start();
				while (!cancellationToken.IsCancellationRequested)
				{
					try
					{
						TcpClient client = await listener.AcceptTcpClientAsync(cancellationToken).ConfigureAwait(false);
						try
						{
							_logger.LogInformation("Accepted connection from {Remote}", client.Client.RemoteEndPoint);

							NetworkStream stream = client.GetStream();

							AutomationRequestInput input = AutomationRequestInput.Read(stream);
							_logger.LogInformation("Received input: {Type} (+{NumBytes} bytes)", input.Type, input.Data.Length);

							AutomationRequestOutput output;
							using (AutomationRequest request = new AutomationRequest(input))
							{
								_postRequest(request);
								request.Complete.Wait(cancellationToken);
								output = request.Output!;
							}

							output.Write(stream);
							_logger.LogInformation("Sent output: {Result} (+{NumBytes} bytes)", output.Result, output.Data.Length);
						}
						catch (Exception ex)
						{
							_logger.LogError(ex, "Exception during automation");
						}
						finally
						{
							_logger.LogInformation("Closed connection.");
						}
					}
					catch when (cancellationToken.IsCancellationRequested)
					{
						break;
					}
					catch (Exception ex)
					{
						_logger.LogError(ex, "Exception during automation operation");
					}
				}
			}
		}

		void HandleUri(string uri)
		{
			try
			{
				UriResult result = UriHandler.HandleUri(uri);
				if (!result.Success)
				{
					if (!String.IsNullOrEmpty(result.Error))
					{
						MessageBox.Show(String.Format("Error handling uri: {0}", result.Error));
					}

					return;
				}

				if (result.Request != null)
				{
					_postRequest(result.Request);
					result.Request.Complete.Wait();
					result.Request.Dispose();
				}
			}
			catch { }
		}

		/// <summary>
		/// Sends UGS scope URI from secondary process to main for handling
		/// </summary>		
		public static void SendUri(string uri)
		{
			using (NamedPipeClientStream clientStream = new NamedPipeClientStream(".", IpcChannel, PipeDirection.Out, PipeOptions.None))
			{
				try
				{
					clientStream.Connect(5000);
					WriteString(clientStream, uri);
				}
				catch (Exception)
				{

				}
			}
		}

		static string ReadString(Stream stream)
		{
			int len = stream.ReadByte() * 256;
			len += stream.ReadByte();
			byte[] inBuffer = new byte[len];
			stream.ReadExactly(inBuffer, 0, len);

			return _streamEncoding.GetString(inBuffer);
		}

		static void WriteString(Stream stream, string output)
		{
			byte[] outBuffer = _streamEncoding.GetBytes(output);

			int len = outBuffer.Length;

			if (len > UInt16.MaxValue)
			{
				len = UInt16.MaxValue;
			}

			stream.WriteByte((byte)(len / 256));
			stream.WriteByte((byte)(len & 255));
			stream.Write(outBuffer, 0, len);
			stream.Flush();
		}

		public void Dispose()
		{
			DisposeAsync().AsTask().Wait();
		}

		public async ValueTask DisposeAsync()
		{
			await _cancellationSource.CancelAsync().ConfigureAwait(false);

			if (_ipcTask != null)
			{
				try
				{
					await _ipcTask.ConfigureAwait(false);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Error awaiting IPC background task");
				}
			}

			if (_tcpTask != null)
			{
				try
				{
					await _tcpTask.ConfigureAwait(false);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Error awaiting TCP background task");
				}
			}

			_cancellationSource.Dispose();
		}
	}
}
