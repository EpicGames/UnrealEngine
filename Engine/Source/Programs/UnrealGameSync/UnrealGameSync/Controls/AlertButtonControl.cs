// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Drawing;
using System.Drawing.Drawing2D;
using System.Windows.Forms;

namespace UnrealGameSync
{
	partial class AlertButtonControl : Button
	{
		public enum AlertButtonTheme
		{
			Normal,
			Green,
			Red,
			Strong,
			Custom
		}

		[TypeConverter(typeof(ExpandableObjectConverter))]
		public struct AlertButtonColors
		{
			public Color ForeColor
			{
				get; set;
			}

			public Color BorderColor
			{
				get; set;
			}

			public Color BackgroundColor1
			{
				get; set;
			}

			public Color BackgroundColor2
			{
				get; set;
			}

			public Color BackgroundColorHover1
			{
				get; set;
			}

			public Color BackgroundColorHover2
			{
				get; set;
			}

			public Color BackgroundColorDown1
			{
				get; set;
			}

			public Color BackgroundColorDown2
			{
				get; set;
			}
		}

		AlertButtonTheme _themeValue;
		AlertButtonColors _colors;
		AlertButtonColors _customColorsValue;
		bool _mouseOver;
		bool _mouseDown;

		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public AlertButtonTheme Theme
		{
			get => _themeValue;
			set
			{
				_themeValue = value;
				UpdateThemeColors();
			}
		}

		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public AlertButtonColors CustomColors
		{
			get => _customColorsValue;
			set
			{
				_customColorsValue = value;
				UpdateThemeColors();
			}
		}

		public AlertButtonControl()
		{
			Theme = AlertButtonTheme.Normal;
			UpdateThemeColors();
		}

		private void UpdateThemeColors()
		{
			switch (Theme)
			{
				case AlertButtonTheme.Normal:
					_colors.ForeColor = Color.FromArgb(64, 86, 106);
					_colors.BorderColor = Color.FromArgb(230, 232, 235);
					_colors.BackgroundColor1 = Color.FromArgb(255, 255, 255);
					_colors.BackgroundColor2 = Color.FromArgb(244, 245, 247);
					_colors.BackgroundColorHover1 = Color.FromArgb(244, 245, 247);
					_colors.BackgroundColorHover2 = Color.FromArgb(244, 245, 247);
					_colors.BackgroundColorDown1 = Color.FromArgb(234, 235, 237);
					_colors.BackgroundColorDown2 = Color.FromArgb(234, 235, 237);
					break;
				case AlertButtonTheme.Green:
					_colors.ForeColor = Color.FromArgb(255, 255, 255);
					_colors.BorderColor = Color.FromArgb(143, 199, 156);
					_colors.BackgroundColor1 = Color.FromArgb(116, 192, 134);
					_colors.BackgroundColor2 = Color.FromArgb(99, 175, 117);
					_colors.BackgroundColorHover1 = Color.FromArgb(99, 175, 117);
					_colors.BackgroundColorHover2 = Color.FromArgb(99, 175, 117);
					_colors.BackgroundColorDown1 = Color.FromArgb(90, 165, 107);
					_colors.BackgroundColorDown2 = Color.FromArgb(90, 165, 107);
					break;
				case AlertButtonTheme.Red:
					_colors.ForeColor = Color.FromArgb(255, 255, 255);
					_colors.BorderColor = Color.FromArgb(230, 232, 235);
					_colors.BackgroundColor1 = Color.FromArgb(222, 108, 86);
					_colors.BackgroundColor2 = Color.FromArgb(214, 69, 64);
					_colors.BackgroundColorHover1 = Color.FromArgb(214, 69, 64);
					_colors.BackgroundColorHover2 = Color.FromArgb(214, 69, 64);
					_colors.BackgroundColorDown1 = Color.FromArgb(204, 59, 54);
					_colors.BackgroundColorDown2 = Color.FromArgb(204, 59, 54);
					break;
				case AlertButtonTheme.Strong:
					_colors.ForeColor = Color.FromArgb(255, 255, 255);
					_colors.BorderColor = Color.FromArgb(230, 232, 235);
					_colors.BackgroundColor1 = Color.FromArgb(200, 74, 49);
					_colors.BackgroundColor2 = Color.FromArgb(200, 74, 49);
					_colors.BackgroundColorHover1 = Color.FromArgb(222, 108, 86);
					_colors.BackgroundColorHover2 = Color.FromArgb(222, 108, 86);
					_colors.BackgroundColorDown1 = Color.FromArgb(204, 59, 54);
					_colors.BackgroundColorDown2 = Color.FromArgb(204, 59, 54);
					break;
				case AlertButtonTheme.Custom:
					_colors = _customColorsValue;
					break;
			}

			base.ForeColor = _colors.ForeColor;
			Invalidate();
		}

		protected override void OnMouseDown(MouseEventArgs mevent)
		{
			base.OnMouseDown(mevent);

			_mouseDown = true;
			Invalidate();
		}

		protected override void OnMouseUp(MouseEventArgs mevent)
		{
			base.OnMouseUp(mevent);

			_mouseDown = false;
			Invalidate();
		}

		protected override void OnMouseEnter(EventArgs e)
		{
			base.OnMouseHover(e);

			_mouseOver = true;
			Invalidate();
		}

		protected override void OnMouseLeave(EventArgs e)
		{
			base.OnMouseLeave(e);

			_mouseOver = false;
			Invalidate();
		}

		protected override void OnPaint(PaintEventArgs e)
		{
			e.Graphics.FillRectangle(SystemBrushes.Window, 0, 0, Width, Height);
			using (GraphicsPath path = new GraphicsPath())
			{
				const int diameter = 4;

				path.StartFigure();
				path.AddArc(Width - 1 - diameter, Height - 1 - diameter, diameter, diameter, 0, 90);
				path.AddArc(0, Height - 1 - diameter, diameter, diameter, 90, 90);
				path.AddArc(0, 0, diameter, diameter, 180, 90);
				path.AddArc(Width - 1 - diameter, 0, diameter, diameter, 270, 90);
				path.CloseFigure();

				Color backgroundColorMin = (_mouseDown && _mouseOver) ? _colors.BackgroundColorDown1 : _mouseOver ? _colors.BackgroundColorHover1 : _colors.BackgroundColor1;
				Color backgroundColorMax = (_mouseDown && _mouseOver) ? _colors.BackgroundColorDown2 : _mouseOver ? _colors.BackgroundColorHover2 : _colors.BackgroundColor2;

				using (LinearGradientBrush brush = new LinearGradientBrush(new Point(0, 0), new Point(0, Height), backgroundColorMin, backgroundColorMax))
				{
					e.Graphics.FillPath(brush, path);
				}
				using (Pen solidPen = new Pen(_colors.BorderColor))
				{
					e.Graphics.DrawPath(solidPen, path);
				}
			}

			TextRenderer.DrawText(e.Graphics, Text, Font, new Rectangle(0, 0, Width, Height), ForeColor, TextFormatFlags.HorizontalCenter | TextFormatFlags.VerticalCenter | TextFormatFlags.SingleLine);
		}
	}
}
