// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Drawing;
using System.Drawing.Drawing2D;
using System.Windows.Forms;

namespace UnrealGameSync
{
	partial class AlertImageControl : UserControl
	{
		Image? _imageValue;

		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public Image? Image
		{
			get => _imageValue;
			set
			{
				_imageValue = value;
				Invalidate();
			}
		}

		public AlertImageControl()
		{
		}

		protected override void OnPaint(PaintEventArgs e)
		{
			e.Graphics.FillRectangle(SystemBrushes.Window, 0, 0, Width, Height);

			if (Image != null)
			{
				float scale = Math.Min((float)Width / Image.Width, (float)Height / Image.Height);

				int imageW = (int)(Image.Width * scale);
				int imageH = (int)(Image.Height * scale);

				e.Graphics.InterpolationMode = InterpolationMode.HighQualityBicubic;
				e.Graphics.SmoothingMode = SmoothingMode.HighQuality;
				e.Graphics.PixelOffsetMode = PixelOffsetMode.HighQuality;
				e.Graphics.DrawImage(Image, (Width - imageW) / 2, (Height - imageH) / 2, imageW, imageH);
			}
		}
	}
}
