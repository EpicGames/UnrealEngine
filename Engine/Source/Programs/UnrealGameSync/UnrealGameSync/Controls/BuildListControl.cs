// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Diagnostics.CodeAnalysis;
using System.Drawing;
using System.Runtime.InteropServices;
using System.Windows.Forms;

namespace UnrealGameSync
{
	class BuildListControl : CustomListViewControl
	{
		[StructLayout(LayoutKind.Sequential)]
		public class Scrollinfo
		{
			public int cbSize;
			public int fMask;
			public int nMin;
			public int nMax;
			public int nPage;
			public int nPos;
			public int nTrackPos;
		}

		const int WmVscroll = 0x115;
		const int WmMousewheel = 0x020A;

		//const int SbHorz = 0;
		const int SbVert = 1;

		const int SifRange = 0x0001;
		const int SifPage = 0x0002;
		const int SifPos = 0x0004;
		//const int SifDisablenoscroll = 0x0008;
		const int SifTrackpos = 0x0010;
		const int SifAll = (SifRange | SifPage | SifPos | SifTrackpos);

		const int LvmGettopindex = 0x1000 + 39;
		const int LvmGetcountperpage = 0x1000 + 40;

		[DllImport("user32.dll")]
		private static extern int GetScrollInfo(IntPtr hwnd, int fnBar, Scrollinfo lpsi);

		[DllImport("user32.dll")]
		private static extern int SetScrollInfo(IntPtr hwnd, int fnBar, Scrollinfo lpsi, int redraw);

		[DllImport("user32.dll", CharSet = CharSet.Auto)]
		private static extern IntPtr SendMessage(IntPtr hWnd, uint msg, IntPtr wParam, IntPtr lParam);

		public delegate void OnScrollDelegate();

		public event OnScrollDelegate? OnScroll;

		public BuildListControl()
		{
			Font = SystemFonts.IconTitleFont!;
		}

		public int GetFirstVisibleIndex()
		{
			if (!IsHandleCreated)
			{
				return 0;
			}
			return SendMessage(Handle, LvmGettopindex, IntPtr.Zero, IntPtr.Zero).ToInt32();
		}

		public int GetVisibleItemsPerPage()
		{
			if (!IsHandleCreated)
			{
				return Height / Font!.Height;
			}
			return SendMessage(Handle, LvmGetcountperpage, IntPtr.Zero, IntPtr.Zero).ToInt32();
		}

		public bool GetScrollPosition(out int scrollY)
		{
			if (!IsHandleCreated)
			{
				scrollY = 0;
				return false;
			}

			Scrollinfo scrollInfo = new Scrollinfo();
			scrollInfo.cbSize = Marshal.SizeOf(scrollInfo);
			scrollInfo.fMask = SifAll;
			if (GetScrollInfo(Handle, SbVert, scrollInfo) == 0)
			{
				scrollY = 0;
				return false;
			}
			else
			{
				scrollY = scrollInfo.nPos;
				return true;
			}
		}

		public void SetScrollPosition(int scrollY)
		{
			if (IsHandleCreated)
			{
				Scrollinfo scrollInfo = new Scrollinfo();
				scrollInfo.cbSize = Marshal.SizeOf(scrollInfo);
				scrollInfo.nPos = scrollY;
				scrollInfo.fMask = SifPos;
				SetScrollInfo(Handle, SbVert, scrollInfo, 0);
			}
		}

		public bool IsScrolledToLastPage()
		{
			if (IsHandleCreated)
			{
				Scrollinfo scrollInfo = new Scrollinfo();
				scrollInfo.cbSize = Marshal.SizeOf(scrollInfo);
				scrollInfo.fMask = SifAll;
				if (GetScrollInfo(Handle, SbVert, scrollInfo) != 0 && scrollInfo.nPos >= scrollInfo.nMax - scrollInfo.nPage)
				{
					return true;
				}
			}
			return false;
		}

		protected override void WndProc(ref Message message)
		{
			base.WndProc(ref message);

			switch (message.Msg)
			{
				case WmVscroll:
				case WmMousewheel:
					if (OnScroll != null)
					{
						OnScroll();
					}
					break;
			}
		}

		[Browsable(false)]
		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		[AllowNull]
		public override Font Font
		{
			get => base.Font;
			set => base.Font = value;
		}

		protected override void Dispose(bool disposing)
		{
			Font?.Dispose();
			Font = null;

			base.Dispose(disposing);
		}
	}
}
