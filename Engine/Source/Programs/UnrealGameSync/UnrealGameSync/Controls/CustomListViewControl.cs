// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Drawing;
using System.Windows.Forms;
using System.Windows.Forms.VisualStyles;

namespace UnrealGameSync
{
	abstract class CustomListViewWidget
	{
		public ListViewItem Item
		{
			get;
			private set;
		}

		Rectangle? _previousBounds;

		public Cursor Cursor
		{
			get;
			set;
		}

		public CustomListViewWidget(ListViewItem item)
		{
			Item = item;
			Cursor = Cursors.Arrow;
		}

		public void Invalidate()
		{
			_previousBounds = null;
		}

		public virtual bool RequiresLayout()
		{
			return !_previousBounds.HasValue;
		}

		public void ConditionalLayout(Control owner, Rectangle bounds)
		{
			if (RequiresLayout() || _previousBounds == null || _previousBounds.Value != bounds)
			{
				using (Graphics graphics = owner.CreateGraphics())
				{
					Layout(graphics, bounds);
				}
				_previousBounds = bounds;
			}
		}

		public virtual void OnMouseMove(Point location)
		{
		}

		public virtual void OnMouseLeave()
		{
		}

		public virtual void OnMouseDown(Point location)
		{
		}

		public virtual void OnMouseUp(Point location)
		{
		}

		public abstract void Layout(Graphics graphics, Rectangle bounds);
		public abstract void Render(Graphics graphics);
	}

	class StatusLineListViewWidget : CustomListViewWidget
	{
		public StatusElementResources Resources { get; }
		public StatusLine Line { get; } = new StatusLine();

		public HorizontalAlignment HorizontalAlignment
		{
			get; set;
		}

		Rectangle _bounds;
		StatusElement? _mouseDownElement;
		StatusElement? _mouseOverElement;

		public StatusLineListViewWidget(ListViewItem item, StatusElementResources resources)
			: base(item)
		{
			Resources = resources;
			HorizontalAlignment = HorizontalAlignment.Center;
		}

		public override bool RequiresLayout()
		{
			return base.RequiresLayout() || Line.RequiresLayout();
		}

		public override void OnMouseDown(Point location)
		{
			StatusElement? element;
			if (Line.HitTest(location, out element))
			{
				_mouseDownElement = element;
				_mouseDownElement.MouseDown = true;

				Invalidate();
			}
		}

		public override void OnMouseUp(Point location)
		{
			if (_mouseDownElement != null)
			{
				StatusElement clickElement = _mouseDownElement;

				_mouseDownElement.MouseDown = false;
				_mouseDownElement = null;

				if (clickElement.Bounds.Contains(location))
				{
					clickElement.OnClick(location);
				}

				Invalidate();
			}
		}

		public override void OnMouseMove(Point location)
		{
			StatusElement? newMouseOverElement;
			Line.HitTest(location, out newMouseOverElement);

			if (_mouseOverElement != newMouseOverElement)
			{
				if (_mouseOverElement != null)
				{
					Cursor = Cursors.Arrow;
					_mouseOverElement.MouseOver = false;
				}

				_mouseOverElement = newMouseOverElement;

				if (_mouseOverElement != null)
				{
					Cursor = _mouseOverElement.Cursor;
					_mouseOverElement.MouseOver = true;
				}

				Invalidate();
			}
		}

		public override void OnMouseLeave()
		{
			if (_mouseOverElement != null)
			{
				_mouseOverElement.MouseOver = false;
				_mouseOverElement = null;

				Invalidate();
			}
		}

		public override void Layout(Graphics graphics, Rectangle bounds)
		{
			_bounds = bounds;

			int offsetY = bounds.Y + bounds.Height / 2;
			Line.Layout(graphics, new Point(bounds.X, offsetY), Resources);

			if (HorizontalAlignment == HorizontalAlignment.Center)
			{
				int offsetX = bounds.X + (bounds.Width - Line.Bounds.Width) / 2;
				Line.Layout(graphics, new Point(offsetX, offsetY), Resources);
			}
			else if (HorizontalAlignment == HorizontalAlignment.Right)
			{
				int offsetX = bounds.Right - Line.Bounds.Width;
				Line.Layout(graphics, new Point(offsetX, offsetY), Resources);
			}
		}

		public override void Render(Graphics graphics)
		{
			graphics.IntersectClip(_bounds);

			Line.Draw(graphics, Resources);
		}
	}

	partial class CustomListViewControl : ListView, ICustomThemeHandler
	{
		readonly VisualStyleRenderer? _selectedItemRenderer;
		readonly VisualStyleRenderer? _trackedItemRenderer;

		public int HoverItem = -1;

		CustomListViewWidget? _mouseOverWidget;
		CustomListViewWidget? _mouseDownWidget;

		Color _textColor = SystemColors.WindowText;

		public CustomListViewControl()
		{
			if (Application.RenderWithVisualStyles)
			{
				_selectedItemRenderer = new VisualStyleRenderer("Explorer::ListView", 1, 3);
				_trackedItemRenderer = new VisualStyleRenderer("Explorer::ListView", 1, 2);
			}
		}

		protected void ConditionalLayoutWidget(CustomListViewWidget widget)
		{
			if (widget.Item.Tag == widget)
			{
				widget.ConditionalLayout(this, widget.Item.Bounds);
			}
			else
			{
				for (int idx = 0; idx < widget.Item.SubItems.Count; idx++)
				{
					ListViewItem.ListViewSubItem subItem = widget.Item.SubItems[idx];
					if (subItem.Tag == widget)
					{
						Rectangle bounds = subItem.Bounds;
						for (int endIdx = idx + 1; endIdx < widget.Item.SubItems.Count && widget.Item.SubItems[endIdx].Tag == widget; endIdx++)
						{
							Rectangle endBounds = widget.Item.SubItems[endIdx].Bounds;
							bounds = new Rectangle(bounds.X, bounds.Y, endBounds.Right - bounds.X, endBounds.Bottom - bounds.Y);
						}
						widget.ConditionalLayout(this, bounds);
						break;
					}
				}
			}
		}

		protected void ConditionalRedrawWidget(CustomListViewWidget widget)
		{
			if (widget.Item.Index != -1 && widget.RequiresLayout())
			{
				ConditionalLayoutWidget(widget);
				RedrawItems(widget.Item.Index, widget.Item.Index, true);
			}
		}

		protected CustomListViewWidget? FindWidget(Point location)
		{
			return FindWidget(HitTest(location));
		}

		protected static CustomListViewWidget? FindWidget(ListViewHitTestInfo hitTest)
		{
			if (hitTest.Item != null)
			{
				CustomListViewWidget? widget = hitTest.Item.Tag as CustomListViewWidget;
				if (widget != null)
				{
					return widget;
				}
			}
			if (hitTest.SubItem != null)
			{
				CustomListViewWidget? widget = hitTest.SubItem.Tag as CustomListViewWidget;
				if (widget != null)
				{
					return widget;
				}
			}
			return null;
		}

		protected override void OnMouseLeave(EventArgs e)
		{
			base.OnMouseLeave(e);

			if (_mouseOverWidget != null)
			{
				_mouseOverWidget.OnMouseLeave();
				ConditionalRedrawWidget(_mouseOverWidget);
				_mouseOverWidget = null;
				Invalidate();
			}

			if (HoverItem != -1)
			{
				HoverItem = -1;
				Invalidate();
			}
		}

		protected override void OnMouseDown(MouseEventArgs e)
		{
			base.OnMouseDown(e);

			if (_mouseDownWidget != null)
			{
				_mouseDownWidget.OnMouseUp(e.Location);
				ConditionalRedrawWidget(_mouseDownWidget);
			}

			if ((e.Button & MouseButtons.Left) != 0)
			{
				_mouseDownWidget = FindWidget(e.Location);
			}
			else
			{
				_mouseDownWidget = null;
			}

			if (_mouseDownWidget != null)
			{
				_mouseDownWidget.OnMouseDown(e.Location);
				ConditionalRedrawWidget(_mouseDownWidget);
			}
		}

		protected override void OnMouseUp(MouseEventArgs e)
		{
			base.OnMouseUp(e);

			if (_mouseDownWidget != null)
			{
				CustomListViewWidget widget = _mouseDownWidget;
				_mouseDownWidget = null;

				widget.OnMouseUp(e.Location);
				ConditionalRedrawWidget(widget);
			}
		}

		protected override void OnMouseMove(MouseEventArgs e)
		{
			ListViewHitTestInfo hitTest = HitTest(e.Location);

			CustomListViewWidget? newMouseOverWidget = FindWidget(hitTest);
			if (_mouseOverWidget != null && _mouseOverWidget != newMouseOverWidget)
			{
				Cursor = Cursors.Arrow;
				_mouseOverWidget.OnMouseLeave();
				ConditionalRedrawWidget(_mouseOverWidget);
			}
			_mouseOverWidget = newMouseOverWidget;
			if (_mouseOverWidget != null)
			{
				Cursor = _mouseOverWidget.Cursor;
				_mouseOverWidget.OnMouseMove(e.Location);
				ConditionalRedrawWidget(_mouseOverWidget);
			}

			int prevHoverItem = HoverItem;
			HoverItem = (hitTest.Item == null) ? -1 : hitTest.Item.Index;
			if (HoverItem != prevHoverItem)
			{
				if (HoverItem != -1)
				{
					RedrawItems(HoverItem, HoverItem, true);
				}
				if (prevHoverItem != -1 && prevHoverItem < Items.Count)
				{
					RedrawItems(prevHoverItem, prevHoverItem, true);
				}
			}

			base.OnMouseMove(e);
		}
		public void DrawText(Graphics graphics, Rectangle bounds, HorizontalAlignment textAlign, Color textColor, string text)
		{
			TextFormatFlags flags = TextFormatFlags.EndEllipsis | TextFormatFlags.SingleLine | TextFormatFlags.VerticalCenter | TextFormatFlags.NoPrefix;
			if (textAlign == HorizontalAlignment.Left)
			{
				flags |= TextFormatFlags.Left;
			}
			else if (textAlign == HorizontalAlignment.Center)
			{
				flags |= TextFormatFlags.HorizontalCenter;
			}
			else
			{
				flags |= TextFormatFlags.Right;
			}
			TextRenderer.DrawText(graphics, text, Font, bounds, textColor, flags);
		}

		public void DrawIcon(Graphics graphics, Rectangle bounds, Rectangle icon)
		{
			_ = this;

			float dpiScaleX = graphics.DpiX / 96.0f;
			float dpiScaleY = graphics.DpiY / 96.0f;

			float iconX = bounds.Left + (bounds.Width - 16 * dpiScaleX) / 2;
			float iconY = bounds.Top + (bounds.Height - 16 * dpiScaleY) / 2;

			graphics.DrawImage(Properties.Resources.Icons, iconX, iconY, icon, GraphicsUnit.Pixel);
		}

		public void DrawNormalSubItem(DrawListViewSubItemEventArgs e)
		{
			if (e.SubItem != null)
			{
				DrawText(e.Graphics, e.SubItem.Bounds, Columns[e.ColumnIndex].TextAlign, _textColor, e.SubItem.Text);
			}
		}

		public void DrawCustomSubItem(Graphics graphics, ListViewItem.ListViewSubItem subItem)
		{
			CustomListViewWidget? widget = subItem.Tag as CustomListViewWidget;
			if (widget != null)
			{
				foreach (ListViewItem.ListViewSubItem? otherSubItem in widget.Item.SubItems)
				{
					if (otherSubItem != null && otherSubItem.Tag == widget)
					{
						if (otherSubItem == subItem)
						{
							ConditionalLayoutWidget(widget);
							widget.Render(graphics);
						}
						break;
					}
				}
			}
		}

		public void DrawBackground(Graphics graphics, ListViewItem item)
		{
			if (item.Selected)
			{
				DrawSelectedBackground(graphics, item.Bounds);
			}
			else if (item.Index == HoverItem)
			{
				DrawTrackedBackground(graphics, item.Bounds);
			}
			else
			{
				DrawDefaultBackground(graphics, item.Bounds);
			}
		}

		public void DrawDefaultBackground(Graphics graphics, Rectangle bounds)
		{
			bounds.Inflate(0, 1);
			_ = this;
			using SolidBrush inputBGBrush = new SolidBrush(BackColor);
			graphics.FillRectangle(inputBGBrush, bounds);
		}

		public void DrawSelectedBackground(Graphics graphics, Rectangle bounds)
		{
			bounds.Inflate(0, 1);
			if (_selectedItemRenderer != null)
			{
				_selectedItemRenderer.DrawBackground(graphics, bounds);
			}
			else
			{
				using SolidBrush inputBGBrush = new SolidBrush(BackColor);
				graphics.FillRectangle(inputBGBrush, bounds);
			}
		}

		public void DrawTrackedBackground(Graphics graphics, Rectangle bounds)
		{
			bounds.Inflate(0, 1);
			if (_trackedItemRenderer != null)
			{
				_trackedItemRenderer.DrawBackground(graphics, bounds);
			}
			else
			{
				using SolidBrush inputBGBrush = new SolidBrush(BackColor);
				graphics.FillRectangle(inputBGBrush, bounds);
			}
		}

		protected override void OnDrawColumnHeader(DrawListViewColumnHeaderEventArgs e)
		{
			ApplicationTheme theme = ProgramApplicationContext.GetTheme();
			e.DrawDefault = false;

			using (SolidBrush backBrush = new SolidBrush(theme.InputBackgroundColor))
			{
				e.Graphics.FillRectangle(backBrush, e.Bounds);
			}

			using (Pen backgroundPen = new Pen(theme.InputBackgroundColor))
			using (Pen linePen = new Pen(theme.BackgroundColor))
			{
				Rectangle r = e.Bounds;
				r.Width -= 1;
				r.Height -= 1;

				e.Graphics.DrawRectangle(backgroundPen, r);

				r.Width -= 1;
				r.Height -= 1;

				e.Graphics.DrawLine(linePen, r.X, r.Y, r.Right, r.Y);
				e.Graphics.DrawLine(linePen, r.X, r.Y, r.X, r.Bottom);

				e.Graphics.DrawLine(backgroundPen, r.X + 1, r.Bottom, r.Right, r.Bottom);
				e.Graphics.DrawLine(backgroundPen, r.Right, r.Y + 1, r.Right, r.Bottom);

				Rectangle newBounds = Rectangle.Inflate(e.Bounds, -TextRenderer.MeasureText(" ", e.Font).Width, 0);
				TextRenderer.DrawText(e.Graphics, e.Header?.Text, e.Font, newBounds, _textColor, TextFormatFlags.Left | TextFormatFlags.WordEllipsis | TextFormatFlags.VerticalCenter);
			}
		}

		public void ApplyTheme(ApplicationThemeManager themeManager)
		{
			themeManager.DefaultApplyThemeToControl(this);

			_textColor = themeManager.Theme.TextColor;

			BackColor = themeManager.Theme.InputBackgroundColor;
			ForeColor = themeManager.Theme.ForegroundColor;
		}
	}
}
