// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Drawing;

namespace UnrealGameSync
{
	class ErrorPanel : StatusPanel, IMainWindowTabPanel
	{
		public ErrorPanel(UserSelectedProjectSettings inSelectedProject)
		{
			SelectedProject = inSelectedProject;
			SetProjectLogo(Properties.Resources.DefaultErrorLogo, false);
		}

		public void Activate()
		{
			BringToFront();
		}

		public void Deactivate()
		{
		}

		public bool IsBusy()
		{
			return false;
		}

		public bool CanClose()
		{
			return true;
		}

		public bool CanSyncNow()
		{
			return false;
		}

		public void SyncLatestChange()
		{
		}

		public bool CanLaunchEditor()
		{
			return false;
		}

		public void LaunchEditor()
		{
		}

		public void UpdateSettings()
		{
		}

		public Color? TintColor => null;

		public Tuple<TaskbarState, float> DesiredTaskbarState => Tuple.Create(TaskbarState.Normal, 0.0f);

		public UserSelectedProjectSettings SelectedProject
		{
			get;
			private set;
		}
	}
}
