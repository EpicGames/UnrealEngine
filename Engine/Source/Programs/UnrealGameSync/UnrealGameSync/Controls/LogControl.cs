// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Data;
using System.Drawing;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.RegularExpressions;
using System.Windows.Forms;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	public class LogControl : UserControl, ICustomThemeHandler
	{
		[Flags]
		public enum ScrollInfoMask : int
		{
			SifRange = 0x1,
			SifPage = 0x2,
			SifPos = 0x4,
			SifDisablenoscroll = 0x8,
			SifTrackpos = 0x10,
			SifAll = SifRange | SifPage | SifPos | SifTrackpos,
		}

		[StructLayout(LayoutKind.Sequential)]
		class Scrollinfo
		{
			public int cbSize;
			public ScrollInfoMask fMask;
			public int nMin;
			public int nMax;
			public int nPage;
			public int nPos;
			public int nTrackPos;
		}

		enum ScrollBarType : int
		{
			SbHorz = 0,
			SbVert = 1,
			SbControl = 2,
			SbBoth = 3,
		}

		enum ScrollBarArrows : uint
		{
			EsbEnableBoth = 0,
		}

		[DllImport("user32.dll")]
		static extern bool EnableScrollBar(IntPtr hWnd, ScrollBarType wSBflags, ScrollBarArrows wArrows);

		[DllImport("user32.dll")]
		static extern int SetScrollInfo(IntPtr hwnd, ScrollBarType fnBar, Scrollinfo lpsi, bool fRedraw);

		[DllImport("user32.dll")]
		static extern int GetScrollInfo(IntPtr hwnd, ScrollBarType fnBar, Scrollinfo lpsi);

		[DllImport("user32.dll", CharSet = CharSet.Auto)]
		static extern IntPtr SendMessage(IntPtr hWnd, uint msg, IntPtr wParam, IntPtr lParam);

		struct TextLocation
		{
			public readonly int LineIdx;
			public readonly int ColumnIdx;

			public TextLocation(int inLineIdx, int inColumnIdx)
			{
				LineIdx = inLineIdx;
				ColumnIdx = inColumnIdx;
			}

			public override string ToString()
			{
				return String.Format("Ln {0}, Col {1}", LineIdx, ColumnIdx);
			}
		}

		class TextSelection
		{
			public TextLocation Start;
			public TextLocation End;

			public TextSelection(TextLocation inStart, TextLocation inEnd)
			{
				Start = inStart;
				End = inEnd;
			}

			public bool IsEmpty()
			{
				return Start.LineIdx == End.LineIdx && Start.ColumnIdx == End.ColumnIdx;
			}

			public override string ToString()
			{
				return String.Format("{0} - {1}", Start, End);
			}
		}

		readonly List<string> _lines = new List<string>();
		int _maxLineLength;

		int _scrollLine;
		int _scrollLinesPerPage;
		bool _trackingScroll;

		int _scrollColumn;
		int _scrollColumnsPerPage;

		bool _isSelecting;
		TextSelection? _selection;

		Size _fontSize;
		Color _textColor = Color.Black;
		Color _warningColor = Color.FromArgb(128, 128, 0);
		Color _errorColor = Color.FromArgb(189, 54, 47);

		Timer? _selectionScrollTimer;
		int _autoScrollRate = 0;

		Timer? _updateTimer;
		ConcurrentQueue<string> _queuedLines = new ConcurrentQueue<string>();
		FileStream? _logFileStream;

		public LogControl()
		{
			BackColor = Color.White;
			ForeColor = Color.FromArgb(255, 32, 32, 32);

			DoubleBuffered = true;

			ContextMenuStrip = new ContextMenuStrip();
			ContextMenuStrip.Items.Add("Copy", null, new EventHandler(ContextMenu_CopySelection));
			ContextMenuStrip.Items.Add("-");
			ContextMenuStrip.Items.Add("Select All", null, new EventHandler(ContextMenu_SelectAll));
			ContextMenuStrip.Items.Add("-");
			ContextMenuStrip.Items.Add("Clear", null, new EventHandler(ContextMenu_Clear));

			SetStyle(ControlStyles.Selectable, true);
		}

		protected override void OnCreateControl()
		{
			base.OnCreateControl();

			EnableScrollBar(Handle, ScrollBarType.SbBoth, ScrollBarArrows.EsbEnableBoth);

			Cursor = Cursors.IBeam;

			_updateTimer = new Timer();
			_updateTimer.Interval = 200;
			_updateTimer.Tick += (a, b) => Tick();
			_updateTimer.Enabled = true;

			_selectionScrollTimer = new Timer();
			_selectionScrollTimer.Interval = 200;
			_selectionScrollTimer.Tick += new EventHandler(SelectionScrollTimer_TimerElapsed);

			Clear();

			UpdateFontMetrics();
			UpdateScrollBarPageSize();
		}

		protected override void Dispose(bool disposing)
		{
			base.Dispose(disposing);

			if (_updateTimer != null)
			{
				_updateTimer.Dispose();
				_updateTimer = null;
			}

			if (_selectionScrollTimer != null)
			{
				_selectionScrollTimer.Dispose();
				_selectionScrollTimer = null;
			}

			CloseFile();
		}

		public bool OpenFile(FileReference newLogFileName)
		{
			CloseFile();
			Clear();
			try
			{
				_logFileStream = FileReference.Open(newLogFileName, FileMode.OpenOrCreate, FileAccess.ReadWrite, FileShare.Read);
			}
			catch (Exception)
			{
				return false;
			}

			_logFileStream.Seek(0, SeekOrigin.Begin);

			int batchSize = 1024;
			using StreamReader reader = new StreamReader(_logFileStream, leaveOpen: true);

			List<string> lines = new List<string>(batchSize);
			while (!reader.EndOfStream)
			{
				lines.Add(reader.ReadLine()!.TrimEnd('\r', '\n'));

				if (lines.Count >= batchSize)
				{
					AddLinesInternal(lines.Select(x => x + "\n").ToList());
					lines.Clear();
				}
			}

			if (lines.Any())
			{
				AddLinesInternal(lines.Select(x => x + "\n").ToList());
			}

			Invalidate();
			return true;
		}

		public void CloseFile()
		{
			if (_logFileStream != null)
			{
				_logFileStream.Dispose();
				_logFileStream = null;
			}
		}

		public void Clear()
		{
			_lines.Clear();
			_maxLineLength = 0;

			_scrollLine = 0;
			_scrollColumn = 0;
			_isSelecting = false;
			_selection = null;

			if (IsHandleCreated)
			{
				Scrollinfo scrollInfo = new Scrollinfo();
				scrollInfo.cbSize = Marshal.SizeOf(scrollInfo);
				scrollInfo.fMask = ScrollInfoMask.SifRange | ScrollInfoMask.SifPage | ScrollInfoMask.SifPos;
				SetScrollInfo(Handle, ScrollBarType.SbHorz, scrollInfo, true);
				SetScrollInfo(Handle, ScrollBarType.SbVert, scrollInfo, true);
			}

			_queuedLines = new ConcurrentQueue<string>();
			if (_logFileStream != null)
			{
				try
				{
					_logFileStream.SetLength(0);
				}
				catch { }
			}
			Invalidate();
		}

		public void ScrollToEnd()
		{
			ScrollWindow(_lines.Count);
			Invalidate();
		}

		public void AppendLine(string line)
		{
			_queuedLines.Enqueue(line);
		}

		private void Tick()
		{
			if (!Focused || !Capture)
			{
				List<string> newLines = new List<string>();
				for (; ; )
				{
					string? nextLine;
					if (!_queuedLines.TryDequeue(out nextLine))
					{
						break;
					}
					newLines.Add(nextLine.TrimEnd('\r', '\n') + "\n");
				}
				if (newLines.Count > 0)
				{
					StringBuilder textToAppendBuilder = new StringBuilder();
					foreach (string newLine in newLines)
					{
						textToAppendBuilder.AppendLine(newLine.TrimEnd('\n'));
					}

					string textToAppend = textToAppendBuilder.ToString();
					if (_logFileStream != null)
					{
						byte[] data = Encoding.UTF8.GetBytes(textToAppend);
						try
						{
							_logFileStream.Write(data, 0, data.Length);
							_logFileStream.Flush();
						}
						catch (Exception ex)
						{
							newLines.Add(String.Format("Failed to write to log file ({0}): {1}\n", _logFileStream.Name, ex.ToString()));
						}
					}

					AddLinesInternal(newLines);
				}
			}
		}

		private void AddLinesInternal(List<string> newLines)
		{
			_lines.AddRange(newLines);

			if (IsHandleCreated)
			{
				// Figure out if we're tracking the last line
				bool isTrackingLastLine = IsTrackingLastLine();

				int newMaxLineLength = Math.Max(_maxLineLength, newLines.Max(x => x.Length));
				if (newMaxLineLength > _maxLineLength)
				{
					_maxLineLength = newMaxLineLength;

					Scrollinfo horizontalScroll = new Scrollinfo();
					horizontalScroll.cbSize = Marshal.SizeOf(horizontalScroll);
					horizontalScroll.fMask = ScrollInfoMask.SifPage | ScrollInfoMask.SifRange;
					horizontalScroll.nMin = 0;
					horizontalScroll.nMax = newMaxLineLength;
					horizontalScroll.nPage = _scrollColumnsPerPage;
					SetScrollInfo(Handle, ScrollBarType.SbHorz, horizontalScroll, true);
				}

				Scrollinfo verticalScroll = new Scrollinfo();
				verticalScroll.cbSize = Marshal.SizeOf(verticalScroll);
				verticalScroll.fMask = ScrollInfoMask.SifPos | ScrollInfoMask.SifRange | ScrollInfoMask.SifPage | ScrollInfoMask.SifTrackpos;
				GetScrollInfo(Handle, ScrollBarType.SbVert, verticalScroll);

				if (_trackingScroll)
				{
					UpdateVerticalScrollPosition(verticalScroll.nTrackPos, ref verticalScroll);
				}
				else
				{
					verticalScroll.fMask = ScrollInfoMask.SifRange | ScrollInfoMask.SifPage;
					verticalScroll.nMin = 0;
					if (isTrackingLastLine)
					{
						verticalScroll.fMask |= ScrollInfoMask.SifPos;
						verticalScroll.nPos = Math.Max(_lines.Count - 1 - _scrollLinesPerPage + 1, 0);
						_scrollLine = Math.Max(_lines.Count - 1 - _scrollLinesPerPage + 1, 0);
					}
					verticalScroll.nMax = Math.Max(_lines.Count - 1, 1);
					verticalScroll.nPage = _scrollLinesPerPage;
					SetScrollInfo(Handle, ScrollBarType.SbVert, verticalScroll, true);
				}
			}

			Invalidate();
		}

		private bool IsTrackingLastLine()
		{
			Scrollinfo verticalScroll = new Scrollinfo();
			verticalScroll.cbSize = Marshal.SizeOf(verticalScroll);
			verticalScroll.fMask = ScrollInfoMask.SifPos | ScrollInfoMask.SifRange | ScrollInfoMask.SifPage | ScrollInfoMask.SifTrackpos;
			GetScrollInfo(Handle, ScrollBarType.SbVert, verticalScroll);

			return (verticalScroll.nPos >= verticalScroll.nMax - _scrollLinesPerPage + 1);
		}

		private void CopySelection()
		{
			if (_lines.Count > 0)
			{
				StringBuilder selectedText = new StringBuilder();
				if (_selection == null || _selection.IsEmpty())
				{
					foreach (string line in _lines)
					{
						selectedText.Append(line);
					}
				}
				else
				{
					for (int lineIdx = Math.Min(_selection.Start.LineIdx, _selection.End.LineIdx); lineIdx <= Math.Max(_selection.Start.LineIdx, _selection.End.LineIdx); lineIdx++)
					{
						int minIdx, maxIdx;
						ClipSelectionToLine(lineIdx, out minIdx, out maxIdx);
						selectedText.Append(_lines[lineIdx], minIdx, maxIdx - minIdx);
					}
				}
				Clipboard.SetText(Regex.Replace(selectedText.ToString(), "(?<!\r)\n", "\r\n"));
			}
		}

		private void ContextMenu_CopySelection(object? sender, EventArgs e)
		{
			CopySelection();
		}

		private void ContextMenu_SelectAll(object? sender, EventArgs e)
		{
			SelectAll();
		}

		private void ContextMenu_Clear(object? sender, EventArgs e)
		{
			Clear();
		}

		public void SelectAll()
		{
			if (_lines.Count > 0)
			{
				_selection = new TextSelection(new TextLocation(0, 0), new TextLocation(_lines.Count - 1, _lines[_lines.Count - 1].Length));
				Invalidate();
			}
		}

		TextLocation PointToTextLocation(Point clientPoint)
		{
			if (_lines.Count == 0)
			{
				return new TextLocation(0, 0);
			}
			else
			{
				int unclippedLineIdx = (clientPoint.Y / _fontSize.Height) + _scrollLine;
				if (unclippedLineIdx < 0)
				{
					return new TextLocation(0, 0);
				}
				else if (unclippedLineIdx >= _lines.Count)
				{
					return new TextLocation(_lines.Count - 1, _lines[_lines.Count - 1].Length);
				}
				else
				{
					return new TextLocation(unclippedLineIdx, Math.Max(0, Math.Min((clientPoint.X / _fontSize.Width) + _scrollColumn, _lines[unclippedLineIdx].Length)));
				}
			}
		}

		protected override void OnResize(EventArgs e)
		{
			base.OnResize(e);

			UpdateScrollBarPageSize();

			int newScrollLine = _scrollLine;
			newScrollLine = Math.Min(newScrollLine, _lines.Count - _scrollLinesPerPage);
			newScrollLine = Math.Max(newScrollLine, 0);

			if (_scrollLine != newScrollLine)
			{
				_scrollLine = newScrollLine;
				Invalidate();
			}
		}

		void UpdateVerticalScrollBarPageSize()
		{
			// Calculate the number of lines per page
			_scrollLinesPerPage = ClientRectangle.Height / _fontSize.Height;

			// Update the vertical scroll bar
			Scrollinfo verticalScroll = new Scrollinfo();
			verticalScroll.cbSize = Marshal.SizeOf(verticalScroll);
			verticalScroll.fMask = ScrollInfoMask.SifPage;
			verticalScroll.nPage = _scrollLinesPerPage;
			SetScrollInfo(Handle, ScrollBarType.SbVert, verticalScroll, true);
		}

		void UpdateScrollBarPageSize()
		{
			// Update the vertical scroll bar, so we know what fits
			UpdateVerticalScrollBarPageSize();

			// This may have modified the client area. Now calculate the number of columns per page.
			_scrollColumnsPerPage = ClientRectangle.Width / _fontSize.Width;

			// And update the horizontal scroll bar
			Scrollinfo horizontalScroll = new Scrollinfo();
			horizontalScroll.cbSize = Marshal.SizeOf(horizontalScroll);
			horizontalScroll.fMask = ScrollInfoMask.SifPage;
			horizontalScroll.nPage = _scrollColumnsPerPage;
			SetScrollInfo(Handle, ScrollBarType.SbHorz, horizontalScroll, true);

			// Now that we know whether we have a horizontal scroll bar or not, calculate the vertical scroll again
			UpdateVerticalScrollBarPageSize();
		}

		protected override void OnFontChanged(EventArgs e)
		{
			base.OnFontChanged(e);

			UpdateFontMetrics();
		}

		private void UpdateFontMetrics()
		{
			using (Graphics graphics = CreateGraphics())
			{
				_fontSize = TextRenderer.MeasureText(graphics, "A", Font, new Size(Int32.MaxValue, Int32.MaxValue), TextFormatFlags.NoPadding);
			}
		}

		protected override void OnMouseDown(MouseEventArgs e)
		{
			base.OnMouseDown(e);

			if (e.Button == MouseButtons.Left)
			{
				_isSelecting = true;

				TextLocation location = PointToTextLocation(e.Location);
				_selection = new TextSelection(location, location);

				_selectionScrollTimer!.Start();
				_autoScrollRate = 0;

				Capture = true;
				Invalidate();
			}
		}

		protected override void OnMouseCaptureChanged(EventArgs e)
		{
			base.OnMouseCaptureChanged(e);

			EndSelection();
		}

		protected override void OnMouseMove(MouseEventArgs e)
		{
			base.OnMouseMove(e);

			if (_isSelecting && _selection != null)
			{
				TextLocation newSelectionEnd = PointToTextLocation(e.Location);
				if (newSelectionEnd.LineIdx != _selection.End.LineIdx || newSelectionEnd.ColumnIdx != _selection.End.ColumnIdx)
				{
					_selection.End = newSelectionEnd;
					Invalidate();
				}

				_autoScrollRate = ((e.Y < 0) ? e.Y : Math.Max(e.Y - ClientSize.Height, 0)) / _fontSize.Height;
			}
		}

		protected override void OnMouseUp(MouseEventArgs e)
		{
			base.OnMouseUp(e);

			EndSelection();
			Capture = false;

			if (e.Button == MouseButtons.Right)
			{
				ContextMenuStrip!.Show(this, e.Location);
				ProgramApplicationContext.GetThemeManager()?.ApplyThemeRecursively(ContextMenuStrip);
			}
		}

		protected override void OnMouseWheel(MouseEventArgs e)
		{
			base.OnMouseWheel(e);

			ScrollWindow(-e.Delta / 60);

			Invalidate();
		}

		protected override bool IsInputKey(Keys key)
		{
			switch (key & ~Keys.Modifiers)
			{
				case Keys.Up:
				case Keys.Down:
					return true;
				default:
					return base.IsInputKey(key);
			}
		}

		protected override void OnKeyDown(KeyEventArgs e)
		{
			switch (e.KeyCode & ~Keys.Modifiers)
			{
				case Keys.Up:
					ScrollWindow(-1);
					Invalidate();
					break;
				case Keys.Down:
					ScrollWindow(+1);
					Invalidate();
					break;
				case Keys.PageUp:
					ScrollWindow(-_scrollLinesPerPage);
					Invalidate();
					break;
				case Keys.PageDown:
					ScrollWindow(+_scrollLinesPerPage);
					Invalidate();
					break;
				case Keys.Home:
					ScrollWindow(-_lines.Count);
					Invalidate();
					break;
				case Keys.End:
					ScrollWindow(+_lines.Count);
					Invalidate();
					break;
				case Keys.A:
					if (e.Control)
					{
						SelectAll();
					}
					break;
				case Keys.C:
					if (e.Control)
					{
						CopySelection();
					}
					break;
			}
		}

		void EndSelection()
		{
			if (_isSelecting)
			{
				if (_selection != null && _selection.IsEmpty())
				{
					_selection = null;
				}
				_selectionScrollTimer!.Stop();
				_isSelecting = false;
			}
		}

		protected override void OnScroll(ScrollEventArgs se)
		{
			base.OnScroll(se);

			if (se.ScrollOrientation == ScrollOrientation.HorizontalScroll)
			{
				// Get the current scroll position
				Scrollinfo scrollInfo = new Scrollinfo();
				scrollInfo.cbSize = Marshal.SizeOf(scrollInfo);
				scrollInfo.fMask = ScrollInfoMask.SifAll;
				GetScrollInfo(Handle, ScrollBarType.SbHorz, scrollInfo);

				// Get the new scroll position
				int targetScrollPos = scrollInfo.nPos;
				switch (se.Type)
				{
					case ScrollEventType.SmallDecrement:
						targetScrollPos = scrollInfo.nPos - 1;
						break;
					case ScrollEventType.SmallIncrement:
						targetScrollPos = scrollInfo.nPos + 1;
						break;
					case ScrollEventType.LargeDecrement:
						targetScrollPos = scrollInfo.nPos - scrollInfo.nPage;
						break;
					case ScrollEventType.LargeIncrement:
						targetScrollPos = scrollInfo.nPos + scrollInfo.nPage;
						break;
					case ScrollEventType.ThumbPosition:
						targetScrollPos = scrollInfo.nTrackPos;
						break;
					case ScrollEventType.ThumbTrack:
						targetScrollPos = scrollInfo.nTrackPos;
						break;
				}
				_scrollColumn = Math.Max(0, Math.Min(targetScrollPos, scrollInfo.nMax - scrollInfo.nPage + 1));

				// Update the scroll bar if we're not tracking
				if (se.Type != ScrollEventType.ThumbTrack)
				{
					scrollInfo.fMask = ScrollInfoMask.SifPos;
					scrollInfo.nPos = _scrollColumn;
					SetScrollInfo(Handle, ScrollBarType.SbHorz, scrollInfo, true);
				}

				Invalidate();
			}
			else if (se.ScrollOrientation == ScrollOrientation.VerticalScroll)
			{
				// Get the current scroll position
				Scrollinfo scrollInfo = new Scrollinfo();
				scrollInfo.cbSize = Marshal.SizeOf(scrollInfo);
				scrollInfo.fMask = ScrollInfoMask.SifAll;
				GetScrollInfo(Handle, ScrollBarType.SbVert, scrollInfo);

				// Get the new scroll position
				int targetScrollPos = scrollInfo.nPos;
				switch (se.Type)
				{
					case ScrollEventType.SmallDecrement:
						targetScrollPos = scrollInfo.nPos - 1;
						break;
					case ScrollEventType.SmallIncrement:
						targetScrollPos = scrollInfo.nPos + 1;
						break;
					case ScrollEventType.LargeDecrement:
						targetScrollPos = scrollInfo.nPos - _scrollLinesPerPage;
						break;
					case ScrollEventType.LargeIncrement:
						targetScrollPos = scrollInfo.nPos + _scrollLinesPerPage;
						break;
					case ScrollEventType.ThumbPosition:
						targetScrollPos = scrollInfo.nTrackPos;
						break;
					case ScrollEventType.ThumbTrack:
						targetScrollPos = scrollInfo.nTrackPos;
						break;
				}

				// Try to move to the new scroll position
				UpdateVerticalScrollPosition(targetScrollPos, ref scrollInfo);

				// Update the new range as well
				if (se.Type == ScrollEventType.ThumbTrack)
				{
					_trackingScroll = true;
				}
				else
				{
					// If we've just finished tracking, allow updating the range
					scrollInfo.nPos = _scrollLine;
					if (_trackingScroll)
					{
						scrollInfo.fMask |= ScrollInfoMask.SifRange;
						scrollInfo.nMax = Math.Max(_lines.Count, 1) - 1;
						_trackingScroll = false;
					}
					SetScrollInfo(Handle, ScrollBarType.SbVert, scrollInfo, true);
				}

				Invalidate();
			}
		}

		void UpdateVerticalScrollPosition(int targetScrollPos, ref Scrollinfo scrollInfo)
		{
			// Scale it up to the number of lines. We don't adjust the scroll max while tracking so we can easily scroll to the end without more stuff being added.
			float ratio = Math.Max(Math.Min((float)targetScrollPos / (float)(scrollInfo.nMax - _scrollLinesPerPage + 1), 1.0f), 0.0f);

			// Calculate the new scroll line, rounding to the nearest
			_scrollLine = (int)((((Math.Max(_lines.Count, 1) - 1) - _scrollLinesPerPage + 1) * ratio) + 0.5f);
		}

		protected override void OnPaint(PaintEventArgs e)
		{
			base.OnPaint(e);

			int textX = -_scrollColumn * _fontSize.Width;
			for (int idx = _scrollLine; idx < _scrollLine + _scrollLinesPerPage + 1 && idx < _lines.Count; idx++)
			{
				int selectMinIdx;
				int selectMaxIdx;
				ClipSelectionToLine(idx, out selectMinIdx, out selectMaxIdx);

				Color textColor;
				if (Regex.IsMatch(_lines[idx], "(?<!\\w)error[: ]", RegexOptions.IgnoreCase))
				{
					textColor = _errorColor;
				}
				else if (Regex.IsMatch(_lines[idx], "(?<!\\w)warning[: ]", RegexOptions.IgnoreCase))
				{
					textColor = _warningColor;
				}
				else
				{
					textColor = _textColor;
				}

				int textY = (idx - _scrollLine) * _fontSize.Height;
				if (selectMinIdx > 0)
				{
					TextRenderer.DrawText(e.Graphics, _lines[idx].Substring(0, selectMinIdx), Font, new Point(textX, textY), textColor, TextFormatFlags.NoPadding);
				}
				if (selectMinIdx < selectMaxIdx)
				{
					e.Graphics.FillRectangle(SystemBrushes.Highlight, textX + (selectMinIdx * _fontSize.Width), textY, (selectMaxIdx - selectMinIdx) * _fontSize.Width, _fontSize.Height);
					TextRenderer.DrawText(e.Graphics, _lines[idx].Substring(selectMinIdx, selectMaxIdx - selectMinIdx), Font, new Point(textX + (selectMinIdx * _fontSize.Width), textY), SystemColors.HighlightText, TextFormatFlags.NoPadding);
				}
				if (selectMaxIdx < _lines[idx].Length)
				{
					TextRenderer.DrawText(e.Graphics, _lines[idx].Substring(selectMaxIdx), Font, new Point(textX + (selectMaxIdx * _fontSize.Width), textY), textColor, TextFormatFlags.NoPadding);
				}
			}
		}

		void ScrollWindow(int scrollDelta)
		{
			Scrollinfo verticalScroll = new Scrollinfo();
			verticalScroll.cbSize = Marshal.SizeOf(verticalScroll);
			verticalScroll.fMask = ScrollInfoMask.SifAll;
			GetScrollInfo(Handle, ScrollBarType.SbVert, verticalScroll);

			verticalScroll.nPos = Math.Min(Math.Max(verticalScroll.nPos + scrollDelta, 0), verticalScroll.nMax - verticalScroll.nPage + 1);
			verticalScroll.fMask = ScrollInfoMask.SifPos;
			SetScrollInfo(Handle, ScrollBarType.SbVert, verticalScroll, true);

			_scrollLine = verticalScroll.nPos;
		}

		private void SelectionScrollTimer_TimerElapsed(object? sender, EventArgs args)
		{
			if (_autoScrollRate != 0 && _selection != null)
			{
				ScrollWindow(_autoScrollRate);

				Scrollinfo verticalScroll = new Scrollinfo();
				verticalScroll.cbSize = Marshal.SizeOf(verticalScroll);
				verticalScroll.fMask = ScrollInfoMask.SifAll;
				GetScrollInfo(Handle, ScrollBarType.SbVert, verticalScroll);

				if (_autoScrollRate < 0)
				{
					if (_selection.End.LineIdx > verticalScroll.nPos)
					{
						_selection.End = new TextLocation(verticalScroll.nPos, 0);
					}
				}
				else
				{
					if (_selection.End.LineIdx < verticalScroll.nPos + verticalScroll.nPage)
					{
						int lineIdx = Math.Min(verticalScroll.nPos + verticalScroll.nPage, _lines.Count - 1);

						if (lineIdx >= 0)
						{
							_selection.End = new TextLocation(lineIdx, _lines[lineIdx].Length);
						}
					}
				}

				Invalidate();
			}
		}

		protected void ClipSelectionToLine(int lineIdx, out int selectMinIdx, out int selectMaxIdx)
		{
			if (_selection == null)
			{
				selectMinIdx = 0;
				selectMaxIdx = 0;
			}
			else if (_selection.Start.LineIdx < _selection.End.LineIdx)
			{
				if (lineIdx < _selection.Start.LineIdx)
				{
					selectMinIdx = _lines[lineIdx].Length;
					selectMaxIdx = _lines[lineIdx].Length;
				}
				else if (lineIdx == _selection.Start.LineIdx)
				{
					selectMinIdx = _selection.Start.ColumnIdx;
					selectMaxIdx = _lines[lineIdx].Length;
				}
				else if (lineIdx < _selection.End.LineIdx)
				{
					selectMinIdx = 0;
					selectMaxIdx = _lines[lineIdx].Length;
				}
				else if (lineIdx == _selection.End.LineIdx)
				{
					selectMinIdx = 0;
					selectMaxIdx = _selection.End.ColumnIdx;
				}
				else
				{
					selectMinIdx = 0;
					selectMaxIdx = 0;
				}
			}
			else if (_selection.Start.LineIdx > _selection.End.LineIdx)
			{
				if (lineIdx < _selection.End.LineIdx)
				{
					selectMinIdx = _lines[lineIdx].Length;
					selectMaxIdx = _lines[lineIdx].Length;
				}
				else if (lineIdx == _selection.End.LineIdx)
				{
					selectMinIdx = _selection.End.ColumnIdx;
					selectMaxIdx = _lines[lineIdx].Length;
				}
				else if (lineIdx < _selection.Start.LineIdx)
				{
					selectMinIdx = 0;
					selectMaxIdx = _lines[lineIdx].Length;
				}
				else if (lineIdx == _selection.Start.LineIdx)
				{
					selectMinIdx = 0;
					selectMaxIdx = _selection.Start.ColumnIdx;
				}
				else
				{
					selectMinIdx = 0;
					selectMaxIdx = 0;
				}
			}
			else
			{
				if (lineIdx == _selection.Start.LineIdx)
				{
					selectMinIdx = Math.Min(_selection.Start.ColumnIdx, _selection.End.ColumnIdx);
					selectMaxIdx = Math.Max(_selection.Start.ColumnIdx, _selection.End.ColumnIdx);
				}
				else
				{
					selectMinIdx = 0;
					selectMaxIdx = 0;
				}
			}
		}

		public void ApplyTheme(ApplicationThemeManager themeManager)
		{
			themeManager.DefaultApplyThemeToControl(this);

			BackColor = themeManager.Theme.InputBackgroundColor;
			ForeColor = themeManager.Theme.TextColor;
			_textColor = themeManager.Theme.TextColor;
			_warningColor= themeManager.Theme.WarningColor;
			_errorColor = themeManager.Theme.ErrorColor;
		}
	}

	public class LogControlTextWriter : ILogger
	{
		const LogLevel MinLogLevel = LogLevel.Information;

		class NullDisposable : IDisposable
		{
			public void Dispose() { }
		}

		readonly LogControl _logControl;

		public LogControlTextWriter(LogControl inLogControl)
		{
			_logControl = inLogControl;
		}

		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			if (logLevel < MinLogLevel)
			{
				return;
			}

			string message = formatter(state, exception);
			_logControl.AppendLine(message);

			if (exception != null)
			{
				foreach (string line in exception.ToString().Trim().Split('\n'))
				{
					_logControl.AppendLine(line);
				}
			}
		}

		public bool IsEnabled(LogLevel logLevel) => true;

		public IDisposable? BeginScope<TState>(TState state) where TState : notnull => new NullDisposable();
	}
}
