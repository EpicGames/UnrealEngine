// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Drawing;
using System.Windows.Forms;

namespace UnrealGameSync
{
	public class LogSplitContainer : SplitContainer, ICustomThemeHandler
	{
		const int WmSetcursor = 0x0020;

		bool _hoverOverClose;
		bool _mouseDownOverClose;
		bool _logVisible = true;
		bool _splitterMoved = false;
		bool _changingLogVisibility = false;

		const int CaptionPadding = 4;

		int _logHeight = 200;

		Color _textColor = SystemColors.ControlText;
		Color _borderColor = SystemColors.ControlDark;
		Color _activeCaption = SystemColors.ActiveCaption;
		Color _inactiveCaption = SystemColors.InactiveCaption;
		Color _inactiveCaptionText = SystemColors.InactiveCaptionText;
		readonly InvertibleImage _logIcon = new InvertibleImage(Properties.Resources.Log, false);

		public event Action<bool>? OnVisibilityChanged;

		[Browsable(true)]
		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public string Caption
		{
			get;
			set => field = value ?? string.Empty;
		}

		public LogSplitContainer()
		{
			DoubleBuffered = true;

			Caption = "Log";
			Orientation = System.Windows.Forms.Orientation.Horizontal;

			SplitterMoved += OnSplitterMoved;
		}

		protected static Font CaptionFont => SystemFonts.IconTitleFont!;

		protected override void Dispose(bool disposing)
		{
			base.Dispose(disposing);

			if (disposing)
			{
				_logIcon.Dispose();
			}
		}

		protected override void OnCreateControl()
		{
			base.OnCreateControl();

			UpdateMetrics();
		}

		protected override void OnKeyUp(KeyEventArgs e)
		{
			// Base implementation directly repaints the splitter control in response to this message, which bypasses our custom paint handler. Invalidate the window to get it drawn again.
			SuspendLayout();
			base.OnKeyUp(e);
			ResumeLayout();
			Invalidate();
		}

		private void OnSplitterMoved(object? obj, EventArgs args)
		{
			_splitterMoved = true;

			if (_logVisible && !_changingLogVisibility)
			{
				_logHeight = Height - (SplitterDistance + SplitterWidth);
			}

			Invalidate();
		}

		public void SetLogVisibility(bool visible)
		{
			if (visible != _logVisible)
			{
				SuspendLayout();

				_logVisible = visible;
				_changingLogVisibility = true;
				UpdateMetrics();
				if (_logVisible)
				{
					SplitterDistance = Math.Max(Height - _logHeight - SplitterWidth, Panel1MinSize);
				}
				else
				{
					SplitterDistance = Height - SplitterWidth;
				}

				_changingLogVisibility = false;

				ResumeLayout();

				if (OnVisibilityChanged != null)
				{
					OnVisibilityChanged(visible);
				}
			}
		}

		private void UpdateMetrics()
		{
			if (_logVisible)
			{
				IsSplitterFixed = false;
				SplitterWidth = CaptionFont.Height + (CaptionPadding * 2) + 4;
				Panel2MinSize = 50;
				Panel2.Show();
			}
			else
			{
				IsSplitterFixed = true;
				Panel2MinSize = 0;
				SplitterWidth = CaptionFont.Height + CaptionPadding + 4;
				Panel2.Hide();
			}
		}

		protected override void OnSizeChanged(EventArgs e)
		{
			base.OnSizeChanged(e);

			// Check it's not minimized; if our minimum size is not being respected, we can't resize properly
			if (Height >= SplitterWidth)
			{
				UpdateMetrics();

				if (!_logVisible)
				{
					SplitterDistance = Height - SplitterWidth;
				}
			}
		}

		protected override void OnLayout(LayoutEventArgs e)
		{
			base.OnLayout(e);

			if (_logVisible && !_changingLogVisibility)
			{
				_logHeight = Height - (SplitterDistance + SplitterWidth);
			}
		}

		public bool IsLogVisible()
		{
			return _logVisible;
		}

		protected override void OnFontChanged(EventArgs e)
		{
			base.OnFontChanged(e);

			UpdateMetrics();
		}

		protected override void OnMouseMove(MouseEventArgs e)
		{
			base.OnMouseMove(e);

			bool newHoverOverClose = CloseButtonRectangle.Contains(e.Location);
			if (_hoverOverClose != newHoverOverClose)
			{
				_hoverOverClose = newHoverOverClose;
				Invalidate();
			}
		}

		protected override void OnMouseLeave(EventArgs e)
		{
			base.OnMouseLeave(e);

			if (_hoverOverClose)
			{
				_hoverOverClose = false;
				_mouseDownOverClose = false;
				Invalidate();
			}
		}

		protected override void OnMouseDown(MouseEventArgs e)
		{
			_splitterMoved = false;

			if (CloseButtonRectangle.Contains(e.Location))
			{
				_mouseDownOverClose = _hoverOverClose;
			}
			else
			{
				base.OnMouseDown(e);
			}
		}

		protected override void OnMouseUp(MouseEventArgs e)
		{
			_mouseDownOverClose = false;

			base.OnMouseUp(e);

			if (e.Button.HasFlag(MouseButtons.Left) && !_splitterMoved)
			{
				SetLogVisibility(!IsLogVisible());
			}
		}

		protected override bool ShowFocusCues => false;

		protected override void OnResize(EventArgs e)
		{
			base.OnResize(e);

			Invalidate(false);
		}

		protected override void OnPaintBackground(PaintEventArgs e)
		{
		}

		protected override void OnPaint(PaintEventArgs e)
		{
			base.OnPaint(e);

			int captionMinY = SplitterDistance + CaptionPadding;
			int captionMaxY = SplitterDistance + SplitterWidth;
			int buttonSize = SplitterWidth - CaptionPadding - 2 - 4;
			if (IsLogVisible())
			{
				captionMaxY -= CaptionPadding;
				buttonSize -= CaptionPadding;
			}

			using (Pen captionBorderPen = new Pen(_borderColor, 1.0f))
			{
				e.Graphics.DrawRectangle(captionBorderPen, 0, captionMinY, ClientRectangle.Width - 1, captionMaxY - captionMinY - 1);
			}

			using (Brush backgroundBrush = new SolidBrush(BackColor))
			{
				e.Graphics.FillRectangle(backgroundBrush, 0, 0, ClientRectangle.Width, captionMinY);
				e.Graphics.FillRectangle(backgroundBrush, 0, captionMaxY, ClientRectangle.Width, Height - captionMaxY);
				e.Graphics.FillRectangle(backgroundBrush, 1, captionMinY + 1, ClientRectangle.Width - 2, captionMaxY - captionMinY - 2);
			}

			int crossX = ClientRectangle.Right - (buttonSize / 2) - 4;
			int crossY = (captionMinY + captionMaxY) / 2;
			if (_hoverOverClose)
			{
				using (Brush activeCaptionBrush = new SolidBrush(_activeCaption))
				{
					e.Graphics.FillRectangle(activeCaptionBrush, crossX - buttonSize / 2, crossY - buttonSize / 2, buttonSize, buttonSize);
				}
			}
			else if (_mouseDownOverClose)
			{
				using (Brush inactiveCaptionBrush = new SolidBrush(_inactiveCaption))
				{
					e.Graphics.FillRectangle(inactiveCaptionBrush, crossX - buttonSize / 2, crossY - buttonSize / 2, buttonSize - 2,buttonSize - 2);
				}
			}
			if (_hoverOverClose || _mouseDownOverClose)
			{
				using (Pen borderPen = new Pen(_inactiveCaptionText, 1.0f))
				{
					e.Graphics.DrawRectangle(borderPen, crossX - buttonSize / 2, crossY - buttonSize / 2, buttonSize, buttonSize);
				}
			}

			if (SplitterDistance >= Height - SplitterWidth)
			{
				e.Graphics.DrawImage(_logIcon.ResolveImage() ?? Properties.Resources.Log, new Rectangle(crossX - (buttonSize / 2) - 1, crossY - (buttonSize / 2) - 1, buttonSize + 2, buttonSize + 2), new Rectangle(16, 0, 16, 16), GraphicsUnit.Pixel);
			}
			else
			{
				e.Graphics.DrawImage(_logIcon.ResolveImage() ?? Properties.Resources.Log, new Rectangle(crossX - (buttonSize / 2) - 1, crossY - (buttonSize / 2) - 1, buttonSize + 2, buttonSize + 2), new Rectangle(0, 0, 16, 16), GraphicsUnit.Pixel);
			}

			TextRenderer.DrawText(e.Graphics, Caption, CaptionFont, new Rectangle(2, captionMinY, ClientRectangle.Width - 20, captionMaxY - captionMinY), _textColor, TextFormatFlags.Left | TextFormatFlags.VerticalCenter);
		}

		protected override void WndProc(ref Message message)
		{
			base.WndProc(ref message);

			switch (message.Msg)
			{
				case WmSetcursor:
					if (RectangleToScreen(CloseButtonRectangle).Contains(Cursor.Position))
					{
						Cursor.Current = Cursors.Default;
						message.Result = new IntPtr(1);
					}
					return;
			}
		}

		public void ApplyTheme(ApplicationThemeManager themeManager)
		{
			themeManager.DefaultApplyThemeToControl(this);

			BackColor = themeManager.Theme.InputBackgroundColor;
			ForeColor = themeManager.Theme.TextColor;

			_textColor = themeManager.Theme.TextColor;
			_borderColor = themeManager.Theme.InputBorderColor;
			_activeCaption = themeManager.Theme.ActiveCaption;
			_inactiveCaption = themeManager.Theme.InactiveCaption;
			_inactiveCaptionText = themeManager.Theme.InactiveCaptionText;
		}

		Rectangle CloseButtonRectangle => new Rectangle(ClientRectangle.Right - 20, SplitterDistance, 20, SplitterWidth);
	}
}
