// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Windows.Forms;

namespace UnrealGameSync.Controls;

public class ReadOnlyCheckbox : CheckBox
{
	private bool _readOnly = false;
	[System.ComponentModel.Category("Behavior")]
	[System.ComponentModel.DefaultValue(false)]
	public bool ReadOnly
	{
		get => _readOnly;
		set
		{
			_readOnly = value;
			Cursor = _readOnly ? Cursors.No : Cursors.Default;
			ProgramApplicationContext.GetThemeManager()?.ApplyThemeRecursively(this);
		}
	}

	protected override void OnMouseEnter(EventArgs e)
	{
		// Disable highlight when the cursor is over the CheckBox
		if (!ReadOnly)
		{
			base.OnMouseEnter(e);
		}
	}
	protected override void OnMouseDown(MouseEventArgs e)
	{
		// Disable reacting (logically or visibly) to a mouse click
		if (!ReadOnly)
		{
			base.OnMouseDown(e);
		}
	}
	protected override void OnKeyDown(KeyEventArgs e)
	{
		// Suppress space key to disable checking/unchecking
		if (!ReadOnly || e.KeyData != Keys.Space)
		{
			base.OnKeyDown(e);
		}
	}
}