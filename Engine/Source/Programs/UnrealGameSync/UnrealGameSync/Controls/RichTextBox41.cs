// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;
using System.Windows.Forms;

namespace UnrealGameSync
{
	class RichTextBox41 : RichTextBox
	{
		[DllImport("kernel32.dll", CharSet = CharSet.Auto)]
		static extern IntPtr LoadLibrary(string lpFileName);

		protected override CreateParams CreateParams
		{
			get
			{
				CreateParams p = base.CreateParams;
				if (LoadLibrary("msftedit.dll") != IntPtr.Zero)
				{
					p.ClassName = "RICHEDIT50W";
				}
				return p;
			}
		}
	}
}
