// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Drawing;
using System.Drawing.Drawing2D;
using System.Linq;
using System.Runtime.InteropServices;
using System.Windows.Forms;

namespace UnrealGameSync
{
	// For some reason, WinForms uses embedded resources for certain cursors, which don't scale correctly for high DPI modes
	static class NativeCursors
	{
		[DllImport("user32.dll")]
		public static extern IntPtr LoadCursor(IntPtr hInstance, IntPtr lpCursorName);

		const int IdcHand = 32649;

		public static readonly Cursor Hand = new Cursor(LoadCursor(IntPtr.Zero, new IntPtr(IdcHand)));
	}

	class StatusElementResources : IDisposable
	{
		readonly Dictionary<FontStyle, Font> _fontCache = new Dictionary<FontStyle, Font>();
		public readonly Font BadgeFont;

		public StatusElementResources(Font baseFont)
		{
			_fontCache.Add(FontStyle.Regular, baseFont);

			using (Graphics graphics = Graphics.FromHwnd(IntPtr.Zero))
			{
				BadgeFont = new Font(baseFont.FontFamily, 7.0f/* * Graphics.DpiY / 96.0f*/, FontStyle.Bold);
			}
		}

		public void Dispose()
		{
			foreach (KeyValuePair<FontStyle, Font> fontPair in _fontCache)
			{
				if (fontPair.Key != FontStyle.Regular)
				{
					fontPair.Value.Dispose();
				}
			}

			BadgeFont.Dispose();
		}

		public Font FindOrAddFont(FontStyle style)
		{
			Font? result;
			if (!_fontCache.TryGetValue(style, out result))
			{
				result = new Font(_fontCache[FontStyle.Regular], style);
				_fontCache[style] = result;
			}
			return result;
		}
	}

	abstract class StatusElement
	{
		public Cursor Cursor { get; set; } = Cursors.Arrow;
		public bool MouseOver { get; set; }
		public bool MouseDown { get; set; }
		public Rectangle Bounds { get; set; }

		public string? ToolTip
		{
			get;
			set;
		}

		public Point Layout(Graphics graphics, Point location, StatusElementResources resources)
		{
			Size size = Measure(graphics, resources);
			Bounds = new Rectangle(location.X, location.Y - (size.Height + 1) / 2, size.Width, size.Height);
			return new Point(location.X + size.Width, location.Y);
		}

		public abstract Size Measure(Graphics graphics, StatusElementResources resources);
		public abstract void Draw(Graphics grahpics, StatusElementResources resources);

		public virtual void OnClick(Point location)
		{
		}
	}

	class IconStatusElement : StatusElement
	{
		readonly Image _icon;

		public IconStatusElement(Image inIcon)
		{
			_icon = inIcon;
		}

		public override Size Measure(Graphics graphics, StatusElementResources resources)
		{
			return new Size((int)(_icon.Width * graphics.DpiX / 96.0f), (int)(_icon.Height * graphics.DpiY / 96.0f));
		}

		public override void Draw(Graphics graphics, StatusElementResources resources)
		{
			graphics.DrawImage(_icon, Bounds.Location);
		}
	}

	class IconStripStatusElement : StatusElement
	{
		readonly Image _strip;
		readonly Size _iconSize;
		readonly int _index;
		readonly bool _applyDpiScaling;

		public IconStripStatusElement(Image strip, Size iconSize, int index, bool applyDpiScaling = true)
		{
			_strip = strip;
			_iconSize = iconSize;
			_index = index;
			_applyDpiScaling = applyDpiScaling;
		}

		public override Size Measure(Graphics graphics, StatusElementResources resources)
		{
			float dpiScaling = _applyDpiScaling ? (graphics.DpiX / 96.0f) : 1.0f;

			return new Size((int)(_iconSize.Width * dpiScaling), (int)(_iconSize.Height * dpiScaling));
		}

		public override void Draw(Graphics graphics, StatusElementResources resources)
		{
			graphics.DrawImage(_strip, Bounds, new Rectangle(_iconSize.Width * _index, 0, _iconSize.Width, _iconSize.Height), GraphicsUnit.Pixel);
		}
	}

	class TextStatusElement : StatusElement
	{
		readonly string _text;
		readonly Color _color;
		readonly FontStyle _style;

		public TextStatusElement(string inText, Color inColor, FontStyle inStyle)
		{
			_text = inText;
			_color = inColor;
			_style = inStyle;
		}

		public override Size Measure(Graphics graphics, StatusElementResources resources)
		{
			return TextRenderer.MeasureText(graphics, _text, resources.FindOrAddFont(_style), new Size(Int32.MaxValue, Int32.MaxValue), TextFormatFlags.NoPadding);
		}

		public override void Draw(Graphics graphics, StatusElementResources resources)
		{
			TextRenderer.DrawText(graphics, _text, resources.FindOrAddFont(_style), Bounds.Location, _color, TextFormatFlags.NoPadding);
		}
	}

	class LinkStatusElement : StatusElement
	{
		readonly string _text;
		readonly FontStyle _style;
		readonly Action<Point, Rectangle> _linkAction;

		public LinkStatusElement(string text, FontStyle style, Action<Point, Rectangle> linkAction)
		{
			_text = text;
			_style = style;
			_linkAction = linkAction;
			Cursor = NativeCursors.Hand;
		}

		public override void OnClick(Point location)
		{
			_linkAction(location, Bounds);
		}

		public override Size Measure(Graphics graphics, StatusElementResources resources)
		{
			return TextRenderer.MeasureText(graphics, _text, resources.FindOrAddFont(_style), new Size(Int32.MaxValue, Int32.MaxValue), TextFormatFlags.NoPadding | TextFormatFlags.NoPrefix);
		}

		public override void Draw(Graphics graphics, StatusElementResources resources)
		{
			Color textColor = ProgramApplicationContext.GetTheme().LinkColor;
			if (MouseDown)
			{
				textColor = ProgramApplicationContext.GetTheme().LinkColorActive;
			}
			else if (MouseOver)
			{
				textColor = ProgramApplicationContext.GetTheme().LinkColorHover;
			}
			TextRenderer.DrawText(graphics, _text, resources.FindOrAddFont(_style), Bounds.Location, textColor, TextFormatFlags.NoPadding | TextFormatFlags.NoPrefix);
		}
	}

	class BadgeStatusElement : StatusElement
	{
		readonly string _name;
		readonly Color _backgroundColor;
		readonly Color _hoverBackgroundColor;
		readonly Action<Point, Rectangle>? _clickAction;

		public bool MergeLeft
		{
			get; set;
		}

		public bool MergeRight
		{
			get; set;
		}

		public BadgeStatusElement(string inName, Color inBackgroundColor, Action<Point, Rectangle>? inClickAction)
		{
			_name = inName;
			_backgroundColor = inBackgroundColor;
			_clickAction = inClickAction;

			if (_clickAction == null)
			{
				_hoverBackgroundColor = _backgroundColor;
			}
			else
			{
				_hoverBackgroundColor = Color.FromArgb(Math.Min(_backgroundColor.R + 32, 255), Math.Min(_backgroundColor.G + 32, 255), Math.Min(_backgroundColor.B + 32, 255));
			}
			if (_clickAction != null)
			{
				Cursor = NativeCursors.Hand;
			}
		}

		public override void OnClick(Point location)
		{
			if (_clickAction != null)
			{
				_clickAction(location, Bounds);
			}
		}

		public override Size Measure(Graphics graphics, StatusElementResources resources)
		{
			Size labelSize = TextRenderer.MeasureText(_name, resources.BadgeFont);
			int badgeHeight = resources.BadgeFont.Height + 1;
			return new Size(labelSize.Width + 1 + badgeHeight - 4, badgeHeight);
		}

		public override void Draw(Graphics graphics, StatusElementResources resources)
		{
			SmoothingMode prevSmoothingMode = graphics.SmoothingMode;
			graphics.SmoothingMode = SmoothingMode.HighQuality;

			Color badgeColor = MouseOver ? _hoverBackgroundColor : _backgroundColor;
			using (GraphicsPath path = new GraphicsPath())
			{
				path.StartFigure();
				path.AddLine(Bounds.Left + (MergeLeft ? 1 : 0), Bounds.Top, Bounds.Left - (MergeLeft ? 1 : 0), Bounds.Bottom);
				path.AddLine(Bounds.Left - (MergeLeft ? 1 : 0), Bounds.Bottom, Bounds.Right - 2 - (MergeRight ? 1 : 0), Bounds.Bottom);
				path.AddLine(Bounds.Right - 2 - (MergeRight ? 1 : 0), Bounds.Bottom, Bounds.Right - 2 + (MergeRight ? 1 : 0), Bounds.Top);
				path.AddLine(Bounds.Right - 2 + (MergeRight ? 1 : 0), Bounds.Top, Bounds.Left + (MergeLeft ? 1 : 0), Bounds.Top);
				path.CloseFigure();

				using (SolidBrush brush = new SolidBrush(badgeColor))
				{
					graphics.FillPath(brush, path);
				}
			}

			Color textColor = ApplicationTheme.IsColorLight(badgeColor) ? Color.Black : Color.White;
			graphics.SmoothingMode = prevSmoothingMode;
			TextRenderer.DrawText(graphics, _name, resources.BadgeFont, Bounds, textColor, TextFormatFlags.HorizontalCenter | TextFormatFlags.VerticalCenter | TextFormatFlags.SingleLine | TextFormatFlags.NoPrefix | TextFormatFlags.PreserveGraphicsClipping);
		}
	}

	class ProgressBarStatusElement : StatusElement
	{
		readonly float _progress;

		public ProgressBarStatusElement(float inProgress)
		{
			_progress = inProgress;
		}

		public override Size Measure(Graphics graphics, StatusElementResources resources)
		{
			int height = (int)(resources.FindOrAddFont(FontStyle.Regular).Height * 0.9f);
			return new Size(height * 16, height);
		}

		public override void Draw(Graphics graphics, StatusElementResources resources)
		{
			graphics.SmoothingMode = System.Drawing.Drawing2D.SmoothingMode.None;

			graphics.DrawRectangle(Pens.Black, Bounds.Left, Bounds.Top, Bounds.Width - 1, Bounds.Height - 1);
			graphics.FillRectangle(Brushes.White, Bounds.Left + 1, Bounds.Top + 1, Bounds.Width - 2, Bounds.Height - 2);

			int progressX = Bounds.Left + 2 + (int)((Bounds.Width - 4) * _progress);
			using (Brush progressBarBrush = new SolidBrush(Color.FromArgb(112, 146, 190)))
			{
				graphics.FillRectangle(progressBarBrush, Bounds.Left + 2, Bounds.Y + 2, progressX - (Bounds.Left + 2), Bounds.Height - 4);
			}
		}
	}

	class StatusLine
	{
		bool _modified;
		readonly List<StatusElement> _elements = new List<StatusElement>();

		public StatusLine()
		{
			LineHeight = 1.0f;
		}

		public float LineHeight
		{
			get;
			set;
		}

		public Rectangle Bounds
		{
			get;
			private set;
		}

		public bool RequiresLayout()
		{
			return _modified;
		}

		public void Clear()
		{
			_elements.Clear();
			_modified = true;
		}

		public string? ToolTip
		{ 
			get;
			set;
		}

		public void Add(StatusElement element, string? toolTip = null)
		{
			element.ToolTip = toolTip;
			_elements.Add(element);
			_modified = true;
		}

		public void AddIcon(Image inIcon, string? toolTip = null)
		{
			Add(new IconStatusElement(inIcon), toolTip);
		}

		public void AddIcon(Image inStrip, Size inIconSize, int inIndex, bool applyDpiScaling = true, string? toolTip = null)
		{
			Add(new IconStripStatusElement(inStrip, inIconSize, inIndex, applyDpiScaling), toolTip);
		}

		public void AddText(string inText, FontStyle inStyle = FontStyle.Regular, string? toolTip = null)
		{
			Color textColor = ProgramApplicationContext.GetTheme().TextColor;
			Add(new TextStatusElement(inText, textColor, inStyle), toolTip);
		}

		public void AddText(string inText, Color inColor, FontStyle inStyle = FontStyle.Regular, string? toolTip = null)
		{
			Add(new TextStatusElement(inText, inColor, inStyle), toolTip);
		}

		public void AddLink(string inText, FontStyle inStyle, Action inLinkAction, string? toolTip = null)
		{
			Add(new LinkStatusElement(inText, inStyle, (p, r) => { inLinkAction(); }), toolTip);
		}

		public void AddLink(string inText, FontStyle inStyle, Action<Point, Rectangle> inLinkAction, string? toolTip = null)
		{
			Add(new LinkStatusElement(inText, inStyle, inLinkAction), toolTip);
		}

		public void AddBadge(string inText, Color inBackgroundColor, Action<Point, Rectangle>? inClickAction, string? toolTip = null)
		{
			BadgeStatusElement element = new BadgeStatusElement(inText, inBackgroundColor, inClickAction);
			element.ToolTip = toolTip;

			_elements.Add(element);
			if (_elements.Count >= 2)
			{
				BadgeStatusElement? prevBadge = _elements[^2] as BadgeStatusElement;
				BadgeStatusElement? nextBadge = _elements[^1] as BadgeStatusElement;
				if (prevBadge != null && nextBadge != null)
				{
					prevBadge.MergeRight = true;
					nextBadge.MergeLeft = true;
				}
			}
			_modified = true;
		}

		public void AddProgressBar(float progress, string? toolTip = null)
		{
			Add(new ProgressBarStatusElement(progress), toolTip);
		}

		public bool HitTest(Point location, [NotNullWhen(true)] out StatusElement? outElement)
		{
			outElement = null;
			if (Bounds.Contains(location))
			{
				foreach (StatusElement element in _elements)
				{
					if (element.Bounds.Contains(location))
					{
						outElement = element;
						return true;
					}
				}
			}
			return false;
		}

		public void Layout(Graphics graphics, Point location, StatusElementResources resources)
		{
			Bounds = new Rectangle(location, new Size(0, 0));

			Point nextLocation = location;
			foreach (StatusElement element in _elements)
			{
				nextLocation = element.Layout(graphics, nextLocation, resources);
				Bounds = Rectangle.Union(Bounds, element.Bounds);
			}

			_modified = false;
		}

		public void Draw(Graphics graphics, StatusElementResources resources)
		{
			foreach (StatusElement element in _elements)
			{
				element.Draw(graphics, resources);
			}
		}
	}

	class StatusPanel : Panel, ICustomThemeHandler
	{
		const float LineSpacing = 1.35f;
		const int InfoMargin = 15;
		const int TopMargin = 10;

		InvertibleImage _projectLogo = new InvertibleImage(Properties.Resources.DefaultProjectLogo, false);
		Rectangle _projectLogoBounds;
		StatusElementResources? _resources;
		readonly List<StatusLine> _lines = new List<StatusLine>();
		readonly List<StatusLine> _infoLines = new List<StatusLine>();
		StatusLine? _caption;
		Pen? _alertDividerPen;
		int _alertDividerY;
		StatusLine? _alert;
		InvertibleColor? _tintColor;
		Point? _mouseOverLocation;
		StatusElement? _mouseOverElement;
		Point? _mouseDownLocation;
		StatusElement? _mouseDownElement;
		int _contentWidth = 400;
		int _suspendDisplayCount;
		readonly System.Windows.Forms.ToolTip _toolTip = new System.Windows.Forms.ToolTip();
		string? _currentToolTip = null;

		private bool _showInfoLines = true;
		
		public StatusPanel()
		{
			DoubleBuffered = true;

			_toolTip.ShowAlways = true;
			_toolTip.InitialDelay = 1000;
			_toolTip.ReshowDelay = 500;

			_alertDividerPen = new Pen(Color.Black);
			_alertDividerPen.DashPattern = new float[] { 2, 2 };
		}

		public void SuspendDisplay()
		{
			_suspendDisplayCount++;
		}

		public void ResumeDisplay()
		{
			_suspendDisplayCount--;
		}

		public void SetContentWidth(int newContentWidth)
		{
			if (_contentWidth != newContentWidth)
			{
				_contentWidth = newContentWidth;
				LayoutElements();
				Invalidate();
			}
		}

		protected override void Dispose(bool disposing)
		{
			base.Dispose(disposing);

			if (disposing)
			{
				if (_alertDividerPen != null)
				{
					_alertDividerPen.Dispose();
					_alertDividerPen = null;
				}

				_toolTip.Dispose();
				
				_projectLogo.Dispose();
				
				ResetFontCache();
			}
		}

		private void ResetFontCache()
		{
			if (_resources != null)
			{
				_resources.Dispose();
				_resources = null;
			}
			_resources = new StatusElementResources(Font);
		}

		public void SetProjectLogo(Image newProjectLogo, bool dispose)
		{
			_projectLogo.Dispose();
			_projectLogo = new InvertibleImage(newProjectLogo, dispose);
			
			Invalidate();
		}

		public void Clear()
		{
			InvalidateElements();
			_lines.Clear();
			_infoLines.Clear();
			_caption = null;
		}

		public void Set(IEnumerable<StatusLine> newLines, IEnumerable<StatusLine> newInfoLines, StatusLine? newCaption, StatusLine? newAlert, Color? newTintColor)
		{
			_resources ??= new StatusElementResources(Font);
			
			if (((_tintColor == null) != (newTintColor == null)) || (_tintColor != null && _tintColor.StandardColor != newTintColor))
			{
				Invalidate();
				_tintColor = newTintColor.HasValue ? new InvertibleColor(newTintColor.Value) : null;
			}

			InvalidateElements();
			_lines.Clear();
			_lines.AddRange(newLines);
			_infoLines.Clear();
			_infoLines.AddRange(newInfoLines);
			_caption = newCaption;
			_alert = newAlert;
			LayoutElements();
			InvalidateElements();

			_mouseOverElement = null;
			_mouseDownElement = null;
			SetMouseOverLocation(_mouseOverLocation);
			SetMouseDownLocation(_mouseDownLocation);
		}

		protected void InvalidateElements()
		{
			Invalidate(_projectLogoBounds);

			foreach (StatusLine line in _lines)
			{
				Invalidate(line.Bounds);
			}
			foreach(StatusLine line in _infoLines)
			{
				Invalidate(line.Bounds);
			}
			if (_caption != null)
			{
				Invalidate(_caption.Bounds);
			}
			if (_alert != null)
			{
				Invalidate(_alert.Bounds);
			}
		}

		protected bool HitTest(Point location, [NotNullWhen(true)] out StatusElement? outElement, out StatusLine? outLine)
		{
			outElement = null;
			outLine = null;

			foreach (StatusLine line in _lines)
			{
				if (line.HitTest(location, out outElement))
				{
					outLine = line;

					return true;
				}
			}
			foreach (StatusLine line in _infoLines)
			{
				if (line.HitTest(location, out outElement))
				{
					outLine = line;

					return true;
				}
			}
			if (_caption != null)
			{
				if (_caption.HitTest(location, out outElement))
				{
					outLine = _caption;

					return true;
				}
			}
			if (_alert != null)
			{
				if (_alert.HitTest(location, out outElement))
				{
					outLine = _alert;

					return true;
				}
			}

			return false;
		}

		protected override void OnFontChanged(EventArgs e)
		{
			base.OnFontChanged(e);

			ResetFontCache();
			LayoutElements();
		}

		protected void LayoutElements()
		{
			using (Graphics graphics = CreateGraphics())
			{
				LayoutElements(graphics);
			}
		}

		protected void LayoutElements(Graphics graphics)
		{
			// Layout the alert message
			int bodyHeight = Height;
			if (_alert != null)
			{
				_alertDividerY = Height - (int)(Font.Height * 2);
				_alert.Layout(graphics, Point.Empty, _resources!);
				_alert.Layout(graphics, new Point((Width - _alert.Bounds.Width) / 2, (Height + _alertDividerY) / 2), _resources!);
				bodyHeight = _alertDividerY;
			}

			// Get the logo size
			Image drawProjectLogo = _projectLogo.ResolveImage() ?? Properties.Resources.DefaultProjectLogo;
			float logoScale = Math.Min((float)bodyHeight - ((_caption != null) ? Font.Height : 0) / drawProjectLogo.Height, graphics.DpiY / 96.0f);
			int logoWidth = (int)(drawProjectLogo.Width * logoScale);
			int logoHeight = (int)(drawProjectLogo.Height * logoScale);

			// Figure out where the split between content and the logo is going to be
			int dividerX = ((Width - logoWidth - _contentWidth) / 2) + logoWidth;

			if (_infoLines.Any())
			{
				int maxWidth = 0;

				float tempTotalInfoLineHeight = _infoLines.Sum(x => x.LineHeight);
				float tempInfoLineY = (bodyHeight - InfoMargin - tempTotalInfoLineHeight * (int)(Font.Height * LineSpacing));
				foreach (StatusLine line in _infoLines)
				{
					line.Layout(graphics, new Point(InfoMargin, (int)tempInfoLineY), _resources!);
					maxWidth = Math.Max(maxWidth, line.Bounds.Right);
				}

				// init to logo position
				int minContentX = dividerX - logoWidth;

				if ( minContentX <= maxWidth)
				{
					_showInfoLines = false;
				}
				else
				{
					_showInfoLines = true;
					dividerX = Math.Max(dividerX, maxWidth);
				}
			}

			// Get the logo position
			int logoX = dividerX - logoWidth;
			int logoY = (bodyHeight - logoHeight) / 2;

			// Layout the caption. We may move the logo to make room for this.
			logoY -= Font.Height / 2;
			if (_caption != null)
			{
				_caption.Layout(graphics, Point.Empty, _resources!);
				int captionWidth = _caption.Bounds.Width;
				_caption.Layout(graphics, new Point(Math.Min(logoX + (logoWidth / 2) - (captionWidth / 2), dividerX - captionWidth), logoY + logoHeight), _resources!);
			}

			// Set the logo rectangle
			_projectLogoBounds = new Rectangle(logoX, logoY, logoWidth, logoHeight);

			// Measure up all the line height
			float totalLineHeight = _lines.Sum(x => x.LineHeight);

			// Space out all the lines
			float lineY = ((bodyHeight - totalLineHeight * (int)(Font.Height * LineSpacing)) / 2) + TopMargin;
			foreach (StatusLine line in _lines)
			{
				lineY += (int)(Font.Height * LineSpacing * line.LineHeight * 0.5f);
				line.Layout(graphics, new Point(dividerX + 5, (int)lineY), _resources!);
				lineY += (int)(Font.Height * LineSpacing * line.LineHeight * 0.5f);
			}

			// Measure up all the info line height
			float totalInfoLineHeight = _infoLines.Sum(x => x.LineHeight);

			// Space out all the info lines
			float infoLineY = (bodyHeight - InfoMargin - totalInfoLineHeight * (int)(Font.Height * LineSpacing));
			foreach (StatusLine line in _infoLines)
			{
				infoLineY += (int)(Font.Height * LineSpacing * line.LineHeight * 0.5f);
				line.Layout(graphics, new Point(InfoMargin, (int)infoLineY), _resources!);
				infoLineY += (int)(Font.Height * LineSpacing * line.LineHeight * 0.5f);
			}
		}

		protected override void OnMouseMove(MouseEventArgs e)
		{
			base.OnMouseMove(e);

			SetMouseOverLocation(e.Location);
		}

		protected override void OnMouseDown(MouseEventArgs e)
		{
			base.OnMouseDown(e);

			if (e.Button == MouseButtons.Left)
			{
				SetMouseDownLocation(e.Location);
			}
		}

		protected override void OnMouseUp(MouseEventArgs e)
		{
			base.OnMouseUp(e);

			if (_mouseDownElement != null && _mouseOverElement == _mouseDownElement)
			{
				_mouseDownElement.OnClick(e.Location);
			}

			SetMouseDownLocation(null);
		}

		protected override void OnMouseLeave(EventArgs e)
		{
			base.OnMouseLeave(e);

			SetMouseDownLocation(null);
			SetMouseOverLocation(null);
		}

		protected override void OnResize(EventArgs eventargs)
		{
			base.OnResize(eventargs);

			LayoutElements();
			Invalidate();
		}

		protected override void OnPaintBackground(PaintEventArgs e)
		{
			base.OnPaintBackground(e);

			if (_tintColor != null)
			{
				int tintSize = Width / 2;
				using (LinearGradientBrush backgroundBrush = new LinearGradientBrush(new Point(Width, 0), new Point(Width - tintSize, tintSize), _tintColor.ResolveColor(), BackColor))
				{
					backgroundBrush.WrapMode = WrapMode.TileFlipXY;
					using (GraphicsPath path = new GraphicsPath())
					{
						path.StartFigure();
						path.AddLine(Width, 0, Width - tintSize * 2, 0);
						path.AddLine(Width, tintSize * 2, Width, 0);
						path.CloseFigure();

						e.Graphics.FillPath(backgroundBrush, path);
					}
				}
			}
		}

		protected override void OnPaint(PaintEventArgs e)
		{
			base.OnPaint(e);

			if (_suspendDisplayCount == 0)
			{
				e.Graphics.DrawImage(_projectLogo.ResolveImage() ?? Properties.Resources.DefaultProjectLogo, _projectLogoBounds);

				e.Graphics.SmoothingMode = System.Drawing.Drawing2D.SmoothingMode.None;

				foreach (StatusLine line in _lines)
				{
					line.Draw(e.Graphics, _resources!);
				}

				if (_showInfoLines)
				{
					foreach (StatusLine line in _infoLines)
					{
						line.Draw(e.Graphics, _resources!);
					}
				}
				
				_caption?.Draw(e.Graphics, _resources!);

				if (_alert != null)
				{
					e.Graphics.DrawLine(_alertDividerPen!, 0, _alertDividerY, Width, _alertDividerY);
					_alert.Draw(e.Graphics, _resources!);
				}
			}
		}

		protected void SetMouseOverLocation(Point? newMouseOverLocation)
		{
			_mouseOverLocation = newMouseOverLocation;

			StatusElement? newMouseOverElement = null;
			StatusLine? newMouseOverLine = null;

			if (_mouseOverLocation.HasValue)
			{
				HitTest(_mouseOverLocation.Value, out newMouseOverElement, out newMouseOverLine);
			}

			if (newMouseOverElement != _mouseOverElement)
			{
				if (_mouseOverElement != null)
				{
					_mouseOverElement.MouseOver = false;
					Cursor = Cursors.Arrow;

					if (newMouseOverElement == null)
					{
						if (_currentToolTip != null)
						{
							_currentToolTip = null;
							_toolTip.Hide(this);
						}
					}

					Invalidate(_mouseOverElement.Bounds);
				}

				_mouseOverElement = newMouseOverElement;

				if (_mouseOverElement != null)
				{
					_mouseOverElement.MouseOver = true;
					Cursor = _mouseOverElement.Cursor;
					string? newToolTip = null;

					if (String.IsNullOrEmpty(_mouseOverElement.ToolTip) == false)
					{
						newToolTip = _mouseOverElement.ToolTip;
					}
					else if (String.IsNullOrEmpty(newMouseOverLine?.ToolTip) == false)
					{
						newToolTip = newMouseOverLine.ToolTip;
					}

					if (String.IsNullOrEmpty(newToolTip) == false)
					{
						if (newToolTip != _currentToolTip)
						{
							_currentToolTip = newToolTip;
							_toolTip.Show(newToolTip, this, _mouseOverLocation!.Value.X, _mouseOverElement.Bounds.Bottom + 5);
						}
					}
					else
					{
						if (_currentToolTip != null)
						{
							_currentToolTip = null;
							_toolTip.Hide(this);
						}
					}

					Invalidate(_mouseOverElement.Bounds);
				}
			}
		}

		protected void SetMouseDownLocation(Point? newMouseDownLocation)
		{
			_mouseDownLocation = newMouseDownLocation;

			StatusElement? newMouseDownElement = null;

			if (_mouseDownLocation.HasValue)
			{
				HitTest(_mouseDownLocation.Value, out newMouseDownElement, out _);
			}

			if (newMouseDownElement != _mouseDownElement)
			{
				if (_mouseDownElement != null)
				{
					_mouseDownElement.MouseDown = false;
					Invalidate(_mouseDownElement.Bounds);
				}

				_mouseDownElement = newMouseDownElement;

				if (_mouseDownElement != null)
				{
					_mouseDownElement.MouseDown = true;
					Invalidate(_mouseDownElement.Bounds);
				}
			}
		}

		public void ApplyTheme(ApplicationThemeManager themeManager)
		{
			themeManager.DefaultApplyThemeToControl(this);

			BackColor = themeManager.Theme.InputBackgroundColor;
			ForeColor = themeManager.Theme.TextColor;
		}
	}
}
