// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.IO;
using System.Linq;
using System.Text;
using System.Windows.Forms;

namespace UnrealGameSync.Controls
{
	public partial class SyncFilterControl : UserControl
	{
		private class ViewItem
		{
			public bool Locked { get; set; } = false;
			public string Value { get; set; } = String.Empty;

			public ViewItem(bool locked, string value)
			{
				Locked = locked;
				Value = value;
			}
		}

		private readonly BindingList<ViewItem> _views = new BindingList<ViewItem>();

		private int _categoriesCheckListHoverIndex = -1;
		private Timer _categoriesCheckListooltipTimer { get; } = new Timer { Interval = 300 };

		// static readonly HashSet<string> s_stripLines = new HashSet<string>()
		// {
		// 	"; Rules are specified one per line, and may use any standard Perforce wildcards:",
		// 	";    ?    Matches one character.",
		// 	";    *    Matches any sequence of characters, except for a directory separator.",
		// 	";    ...  Matches any sequence of characters, including directory separators.",
		// 	"; Patterns may match any file fragment (eg. *.pdb), or may be rooted to the branch (eg. /Engine/Binaries/.../*.pdb).",
		// 	"; To exclude files which match a pattern, prefix the rule with the '-' character (eg. -/Engine/Documentation/...)",
		// 	"; Global rules are applied to the files being synced first, followed by any workspace-specific patterns.",
		// };

		public SyncFilterControl()
		{
			InitializeComponent();
			ViewDataGrid.RowHeadersVisible = false;
			ViewDataGrid.ColumnHeadersVisible = false;
			ViewDataGrid.AutoGenerateColumns = true;
			ViewDataGrid.EditMode = DataGridViewEditMode.EditOnEnter;
			ViewDataGrid.AllowUserToAddRows = true;
			ViewDataGrid.DefaultValuesNeeded += ViewDataGrid_DefaultValuesNeeded;
			ViewDataGrid.CellBeginEdit += ViewDataGrid_CellBeginEdit;
			ViewDataGrid.UserDeletingRow += ViewDataGrid_UserDeletingRow;

			_views.AddingNew += new AddingNewEventHandler(Views_AddingNew);

			ViewDataGrid.DataSource = _views;

			CategoriesCheckList.MouseMove += CheckedListBox_MouseMove;

			CategoriesCheckList.MouseLeave += (_, __) => CategoriesToolTip.Hide(CategoriesCheckList);
			CategoriesCheckList.MouseWheel += (_, __) => CategoriesToolTip.Hide(CategoriesCheckList);
			CategoriesCheckList.MouseDown += (_, __) => CategoriesToolTip.Hide(CategoriesCheckList);

			_categoriesCheckListooltipTimer.Tick += TooltipTimer_Tick;
		}

		private void CheckedListBox_MouseMove(object? sender, MouseEventArgs e)
		{
			int index = CategoriesCheckList.IndexFromPoint(e.Location);

			if (index == _categoriesCheckListHoverIndex)
			{
				return;

			}

			if (index >= 0 && index < CategoriesCheckList.Items.Count)
			{
				_categoriesCheckListHoverIndex = index;
				_categoriesCheckListooltipTimer.Start();
			}
			else
			{
				CategoriesToolTip.SetToolTip(CategoriesCheckList, String.Empty);
				CategoriesToolTip.Hide(CategoriesCheckList);
			}
		}

		private void TooltipTimer_Tick(object? sender, EventArgs e)
		{
			_categoriesCheckListooltipTimer.Stop();

			if (_categoriesCheckListHoverIndex >= 0 && _categoriesCheckListHoverIndex < CategoriesCheckList.Items.Count)
			{
				object item = CategoriesCheckList.Items[_categoriesCheckListHoverIndex];
				System.Reflection.PropertyInfo? property = item.GetType().GetProperty("ToolTip");

				if (property != null)
				{
					if (property.GetValue(item) is string toolTip)
					{
						CategoriesToolTip.ToolTipTitle = item.ToString();
						CategoriesToolTip.SetToolTip(CategoriesCheckList, toolTip);
					}
				}
			}
		}

		void Views_AddingNew(object? sender, AddingNewEventArgs e)
		{
			e.NewObject = new ViewItem(false, "change me");
		}

		private void ViewDataGrid_CellBeginEdit(object? sender, DataGridViewCellCancelEventArgs e)
		{
			if (e.RowIndex >= 0 && e.RowIndex < ViewDataGrid.Rows.Count)
			{
				DataGridViewRow? row = ViewDataGrid.Rows[e.RowIndex];
				if (row.DataBoundItem is ViewItem item)
				{
					e.Cancel = item.Locked;
				}
			}
		}

		private void ViewDataGrid_UserDeletingRow(object? sender, DataGridViewRowCancelEventArgs e)
		{
			if (e.Row != null && e.Row.DataBoundItem is ViewItem item)
			{
				e.Cancel = item.Locked;
			}
		}

		private void ViewDataGrid_DefaultValuesNeeded(object? sender, System.Windows.Forms.DataGridViewRowEventArgs e)
		{
			e.Row.Cells[nameof(ViewItem.Locked)].Value = false;
			e.Row.Cells[nameof(ViewItem.Value)].Value = String.Empty;
		}

		private void SyntaxButton_LinkClicked(object sender, LinkLabelLinkClickedEventArgs e)
		{
			using SyncFilterSyntax dialog = new SyncFilterSyntax();
			dialog.ShowDialog(ParentForm);
		}

		public void SetView(string[] views)
		{
			_views.Clear();

			foreach (string view in views)
			{
				_views.Add(new ViewItem(false, view));
			}

			foreach (DataGridViewColumn column in ViewDataGrid.Columns)
			{
				if (column.Name.Equals(nameof(ViewItem.Locked), StringComparison.Ordinal))
				{
					column.Visible = false;
				}

				if (column.Name.Equals(nameof(ViewItem.Value), StringComparison.Ordinal))
				{
					column.AutoSizeMode = DataGridViewAutoSizeColumnMode.Fill;
				}
			}

			foreach (DataGridViewRow? row in ViewDataGrid.Rows)
			{
				if (row == null)
				{
					continue;
				}

				if (row.DataBoundItem is ViewItem item)
				{
					if (item.Locked)
					{
						row.ReadOnly = true;
						foreach (DataGridViewCell cell in row.Cells)
						{
							cell.ReadOnly = true;
						}
					}
				}
			}
		}

		public string[] GetView()
		{
			List<string> views = new List<string>();
			foreach (ViewItem item in _views)
			{
				if (item.Locked)
				{
					continue;
				}

				if (String.IsNullOrEmpty(item.Value) == false)
				{
					views.Add(item.Value);
				}
			}

			return views.ToArray();
		}
		private void CopyCustomViewsToClipboard_Click(object sender, EventArgs e)
		{
			try
			{
				if (!_views.Any())
				{
					return;
				}

				StringBuilder sb = new StringBuilder();

				foreach (ViewItem item in _views)
				{
					if (String.IsNullOrEmpty(item.Value) == false)
					{
						sb.AppendLine(item.Value);
					}
				}

				string? text = sb.ToString();
				if (String.IsNullOrWhiteSpace(text))
				{
					return;
				}

				Clipboard.SetText(sb.ToString());
			}
			catch
			{
				// ignored
			}
		}

		private async void PasteCustomViewsToClipboard_Click(object sender, EventArgs e)
		{
			try
			{
				string text = Clipboard.GetText();
				if (String.IsNullOrWhiteSpace(text))
				{
					return;
				}

				_views.Clear();

				HashSet<string> uniques = new HashSet<string>();

				using StringReader reader = new StringReader(text);

				while (reader.Peek() != -1)
				{
					string? line = await reader.ReadLineAsync();
					if (String.IsNullOrWhiteSpace(line))
					{
						continue;
					}

					uniques.Add(line);
					
				}

				foreach (string unique in uniques)
				{
					_views.Add(new ViewItem(false, unique));
				}
			}
			catch
			{
				// ignored
			}
		}

		private void ClearCustomViewsToClipboard_Click(object sender, EventArgs e)
		{
			_views.Clear();
		}
	}
}
