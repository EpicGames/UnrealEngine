// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Drawing;
using System.Drawing.Drawing2D;
using System.Linq;
using System.Windows.Forms;

namespace UnrealGameSync
{
	class TabControl : Control, ICustomThemeHandler
	{
		const int TabPadding = 13;//18;
		const int TabCloseButtonWidth = 8 + 13;

		class TabData
		{
			public string _name;
			public object _data;
			public int _minX;
			public int _width;
			public int _closeButtonWidth;
			public Size _textSize;
			public Tuple<Color, float>? _highlight;
			public Color? _tintColor;

			public TabData(string name, object data)
			{
				_name = name;
				_data = data;
			}
		}

		class TabDragData
		{
			public TabData _tab;
			public int _initialIdx;
			public int _mouseX;
			public int _relativeMouseX;

			public TabDragData(TabData tab, int initialIdx, int mouseX, int relativeMouseX)
			{
				_tab = tab;
				_initialIdx = initialIdx;
				_mouseX = mouseX;
				_relativeMouseX = relativeMouseX;
			}
		}

		readonly List<TabData> _tabs = new List<TabData>();
		int _selectedTabIdx;
		int _hoverTabIdx;
		int _highlightTabIdx;
		bool _isHoveringCloseBtn = false;
		TabDragData? _dragState;
		Color _textColor = SystemColors.WindowText;
		Color _selectedColor = SystemColors.HotTrack;
		Color _borderColor = SystemColors.Window;
		Color _hoverColor = SystemColors.Window;
		Color _hoverStripeColor = Color.FromArgb(192, 192, 192);
		Color _closeColor = SystemColors.HotTrack;
		Color _closeColorHovered = Color.FromArgb(SystemColors.HotTrack.B, SystemColors.HotTrack.G, SystemColors.HotTrack.R);

		public delegate void OnTabChangedDelegate(object? newTabData);
		public event OnTabChangedDelegate? OnTabChanged;

		public delegate void OnNewTabClickDelegate(Point location, MouseButtons buttons);
		public event OnNewTabClickDelegate? OnNewTabClick;

		public delegate void OnTabClickedDelegate(object? tabData, Point location, MouseButtons buttons);
		public event OnTabClickedDelegate? OnTabClicked;

		public delegate bool OnTabClosingDelegate(object tabData);
		public event OnTabClosingDelegate? OnTabClosing;

		public delegate void OnTabClosedDelegate(object tabData);
		public event OnTabClosedDelegate? OnTabClosed;

		public delegate void OnTabReorderDelegate();
		public event OnTabReorderDelegate? OnTabReorder;

		public delegate void OnButtonClickDelegate(int buttonIdx, Point location, MouseButtons buttons);
		public event OnButtonClickDelegate? OnButtonClick;

		public TabControl()
		{
			DoubleBuffered = true;
			_tabs.Add(new TabData("+", "+"));
			_selectedTabIdx = -1;
			_hoverTabIdx = -1;
			_highlightTabIdx = -1;
		}

		public void LockHover()
		{
			_highlightTabIdx = _hoverTabIdx;
		}

		public void UnlockHover()
		{
			_highlightTabIdx = -1;
			Invalidate();
		}

		public int FindTabIndex(object data)
		{
			int result = -1;
			for (int tabIdx = 0; tabIdx < _tabs.Count - 1; tabIdx++)
			{
				if (_tabs[tabIdx]._data == data)
				{
					result = tabIdx;
					break;
				}
			}
			return result;
		}

		public void SetHighlight(int tabIdx, Tuple<Color, float>? highlight)
		{
			Tuple<Color, float>? currentHighlight = _tabs[tabIdx]._highlight;
			if (highlight == null || currentHighlight == null)
			{
				if (highlight != currentHighlight)
				{
					_tabs[tabIdx]._highlight = highlight;
					Invalidate();
				}
			}
			else
			{
				if (highlight.Item1 != currentHighlight.Item1 || highlight.Item2 != currentHighlight.Item2)
				{
					_tabs[tabIdx]._highlight = highlight;
					Invalidate();
				}
			}
		}

		public void SetTint(int tabIdx, Color? tintColor)
		{
			if (_tabs[tabIdx]._tintColor != tintColor)
			{
				_tabs[tabIdx]._tintColor = tintColor;
				Invalidate();
			}
		}

		public int InsertTab(int insertIdx, string name, object data, Color? tintColor)
		{
			int idx = _tabs.Count - 1;
			if (insertIdx >= 0 && insertIdx < _tabs.Count - 1)
			{
				idx = insertIdx;
			}
			if (_selectedTabIdx >= idx)
			{
				_selectedTabIdx++;
			}

			_tabs.Insert(idx, new TabData(name, data) { _tintColor = tintColor });
			LayoutTabs();
			Invalidate();
			return idx;
		}

		public int GetTabCount()
		{
			return _tabs.Count - 1;
		}

		public void SetTabName(int tabIdx, string name)
		{
			_tabs[tabIdx]._name = name;
			LayoutTabs();
			Invalidate();
		}

		public object GetTabData(int tabIdx)
		{
			return _tabs[tabIdx]._data;
		}

		public void SetTabData(int tabIdx, object data)
		{
			_tabs[tabIdx]._data = data;
		}

		public int GetSelectedTabIndex()
		{
			return _selectedTabIdx;
		}

		public object GetSelectedTabData()
		{
			return _tabs[_selectedTabIdx]._data;
		}

		public void SelectTab(int tabIdx)
		{
			if (tabIdx != _selectedTabIdx)
			{
				ForceSelectTab(tabIdx);
			}
		}

		private void ForceSelectTab(int tabIdx)
		{
			_selectedTabIdx = tabIdx;
			LayoutTabs();
			Invalidate();

			if (OnTabChanged != null)
			{
				if (_selectedTabIdx == -1)
				{
					OnTabChanged(null);
				}
				else
				{
					OnTabChanged(_tabs[_selectedTabIdx]._data);
				}
			}
		}

		public void RemoveTab(int tabIdx)
		{
			object tabData = _tabs[tabIdx]._data;
			if (OnTabClosing == null || OnTabClosing(tabData))
			{
				_tabs.RemoveAt(tabIdx);

				if (_hoverTabIdx == tabIdx)
				{
					_hoverTabIdx = -1;
				}

				if (_highlightTabIdx == tabIdx)
				{
					_highlightTabIdx = -1;
				}

				if (_selectedTabIdx == tabIdx)
				{
					if (_selectedTabIdx < _tabs.Count - 1)
					{
						ForceSelectTab(_selectedTabIdx);
					}
					else
					{
						ForceSelectTab(_selectedTabIdx - 1);
					}
				}
				else
				{
					if (_selectedTabIdx > tabIdx)
					{
						_selectedTabIdx--;
					}
				}

				LayoutTabs();
				Invalidate();

				OnTabClosed?.Invoke(tabData);
			}
		}

		void LayoutTabs()
		{
			using (Graphics graphics = CreateGraphics())
			{
				float dpiScaleX = graphics.DpiX / 96.0f;
				float dpiScaleY = graphics.DpiY / 96.0f;

				for (int idx = 0; idx < _tabs.Count; idx++)
				{
					TabData tab = _tabs[idx];
					tab._textSize = TextRenderer.MeasureText(graphics, tab._name, Font, new Size(Int32.MaxValue, Int32.MaxValue), TextFormatFlags.NoPadding);
					tab._width = (int)(TabPadding * dpiScaleX) + tab._textSize.Width + (int)(TabPadding * dpiScaleX);
					if (idx == _selectedTabIdx)
					{
						tab._closeButtonWidth = (int)(TabCloseButtonWidth * dpiScaleX);
						tab._width += tab._closeButtonWidth;
					}
				}

				int leftX = 0;
				for (int idx = 0; idx < _tabs.Count; idx++)
				{
					TabData tab = _tabs[idx];
					tab._minX = leftX;
					leftX += tab._width;
				}

				int rightX = Width - 1;
				if (leftX > rightX)
				{
					int usedWidth = _tabs.Take(_tabs.Count - 1).Sum(x => x._width);
					int remainingWidth = rightX + 1 - _tabs[^1]._width;
					if (_selectedTabIdx != -1)
					{
						usedWidth -= _tabs[_selectedTabIdx]._width;
						remainingWidth -= _tabs[_selectedTabIdx]._width;
					}

					int newX = 0;
					for (int idx = 0; idx < _tabs.Count; idx++)
					{
						_tabs[idx]._minX = newX;
						int prevWidth = _tabs[idx]._width;
						if (idx != _selectedTabIdx && idx != _tabs.Count - 1)
						{
							_tabs[idx]._width = Math.Max((_tabs[idx]._width * remainingWidth) / usedWidth, TabPadding * 3);
						}
						_tabs[idx]._textSize.Width -= prevWidth - _tabs[idx]._width;
						newX += _tabs[idx]._width;
					}
				}

				if (_dragState != null)
				{
					int minOffset = -_dragState._tab._minX;
					int maxOffset = _tabs[^1]._minX - _dragState._tab._width - _dragState._tab._minX;

					int offset = (_dragState._mouseX - _dragState._relativeMouseX) - _dragState._tab._minX;
					offset = Math.Max(offset, minOffset);
					offset = Math.Min(offset, maxOffset);

					_dragState._tab._minX += offset;
				}
			}
		}

		protected override void OnResize(EventArgs e)
		{
			base.OnResize(e);

			LayoutTabs();
			Invalidate();
		}

		protected override void OnMouseDown(MouseEventArgs e)
		{
			base.OnMouseDown(e);

			OnMouseMove(e);

			if (_hoverTabIdx != -1)
			{
				if (e.Button == MouseButtons.Left)
				{
					if (_hoverTabIdx == _selectedTabIdx)
					{
						if (e.Location.X > _tabs[_hoverTabIdx]._minX + _tabs[_hoverTabIdx]._width - _tabs[_hoverTabIdx]._closeButtonWidth)
						{
							RemoveTab(_hoverTabIdx);
						}
						else
						{
							_dragState = new TabDragData(_tabs[_selectedTabIdx], _selectedTabIdx, e.Location.X, e.Location.X - _tabs[_selectedTabIdx]._minX);
						}
					}
					else
					{
						if (_hoverTabIdx > _tabs.Count - 1)
						{
							OnButtonClick?.Invoke(_hoverTabIdx - _tabs.Count, e.Location, e.Button);
						}
						else if (_hoverTabIdx == _tabs.Count - 1)
						{
							OnNewTabClick?.Invoke(e.Location, e.Button);
						}
						else
						{
							SelectTab(_hoverTabIdx);
						}
					}
				}
				else if (e.Button == MouseButtons.Middle)
				{
					if (_hoverTabIdx < _tabs.Count - 1)
					{
						RemoveTab(_hoverTabIdx);
					}
				}
			}

			if (OnTabClicked != null)
			{
				object? tabData = (_hoverTabIdx == -1) ? null : _tabs[_hoverTabIdx]._data;
				OnTabClicked(tabData, e.Location, e.Button);
			}
		}

		protected override void OnMouseUp(MouseEventArgs e)
		{
			base.OnMouseUp(e);

			if (_dragState != null)
			{
				if (OnTabReorder != null)
				{
					OnTabReorder();
				}

				_dragState = null;

				LayoutTabs();
				Invalidate();
			}
		}

		protected override void OnMouseMove(MouseEventArgs e)
		{
			base.OnMouseMove(e);

			if (_dragState != null)
			{
				_dragState._mouseX = e.Location.X;

				_tabs.Remove(_dragState._tab);

				int totalWidth = 0;
				for (int insertIdx = 0; ; insertIdx++)
				{
					if (insertIdx == _tabs.Count - 1 || _dragState._mouseX - _dragState._relativeMouseX < totalWidth + _tabs[insertIdx]._width / 2)
					{
						_hoverTabIdx = _selectedTabIdx = insertIdx;
						_tabs.Insert(insertIdx, _dragState._tab);
						break;
					}
					totalWidth += _tabs[insertIdx]._width;
				}

				LayoutTabs();
				Invalidate();
			}
			else
			{
				int newHoverTabIdx = -1;
				if (ClientRectangle.Contains(e.Location))
				{
					for (int idx = 0; idx < _tabs.Count; idx++)
					{
						if (e.Location.X > _tabs[idx]._minX && e.Location.X < _tabs[idx]._minX + _tabs[idx]._width)
						{
							newHoverTabIdx = idx;
							break;
						}
					}
				}

				if (_hoverTabIdx != newHoverTabIdx)
				{
					_hoverTabIdx = newHoverTabIdx;
					LayoutTabs();
					Invalidate();

					if (_hoverTabIdx != -1)
					{
						Capture = true;
					}
					else
					{
						Capture = false;
					}
				}
			}

			if (_selectedTabIdx != -1)
			{
				bool isNowHoveringCloseBtn = _hoverTabIdx == _selectedTabIdx
								  && ((e.Location.X) > (_tabs[_hoverTabIdx]._minX + _tabs[_hoverTabIdx]._width - _tabs[_hoverTabIdx]._closeButtonWidth));
				if (_isHoveringCloseBtn != isNowHoveringCloseBtn)
				{
					_isHoveringCloseBtn = isNowHoveringCloseBtn;
					Invalidate();
				}
			}
		}

		protected override void OnMouseCaptureChanged(EventArgs e)
		{
			base.OnMouseCaptureChanged(e);

			if (_hoverTabIdx != -1)
			{
				_hoverTabIdx = -1;
				Invalidate();
			}
		}

		protected override void OnPaint(PaintEventArgs e)
		{
			base.OnPaint(e);

			LayoutTabs();

			using (SolidBrush backBrush = new SolidBrush(BackColor))
			using (SolidBrush hoverBrush = new SolidBrush(_hoverColor))
			using (SolidBrush hoverStripeBrush = new SolidBrush(_hoverStripeColor))
			{
				for (int idx = 0; idx < _tabs.Count; idx++)
				{
					TabData tab = _tabs[idx];
					if (idx == _hoverTabIdx || idx == _highlightTabIdx)
					{
						DrawBackground(e.Graphics, tab, hoverBrush, hoverStripeBrush, tab._highlight);
					}
					else
					{
						DrawBackground(e.Graphics, tab, backBrush, backBrush, tab._highlight);
					}
				}
			}

			using (Pen separatorPen = new Pen(Color.FromArgb(192, SystemColors.ControlDarkDark)))
			{
				for (int idx = 0; idx < _tabs.Count; idx++)
				{
					TabData tab = _tabs[idx];
					if ((idx > 0 && idx < _tabs.Count - 1) || idx >= _tabs.Count || idx == _selectedTabIdx || idx == _hoverTabIdx || idx == _highlightTabIdx || _tabs.Count == 1)
					{
						e.Graphics.DrawLine(separatorPen, tab._minX, 0, tab._minX, ClientSize.Height - 2);
					}
					if (idx < _tabs.Count - 1 || idx >= _tabs.Count || idx == _hoverTabIdx || idx == _highlightTabIdx || _tabs.Count == 1)
					{
						e.Graphics.DrawLine(separatorPen, tab._minX + tab._width, 0, tab._minX + tab._width, ClientSize.Height - 2);
					}
					if (idx == _hoverTabIdx || idx == _highlightTabIdx || idx >= _tabs.Count)
					{
						e.Graphics.DrawLine(separatorPen, tab._minX, 0, tab._minX + tab._width, 0);
					}
				}
			}

			for (int idx = 0; idx < _tabs.Count; idx++)
			{
				if (idx != _selectedTabIdx)
				{
					DrawText(e.Graphics, _tabs[idx]);
				}
			}

			if (_selectedTabIdx != -1)
			{
				TabData selectedTab = _tabs[_selectedTabIdx];

				Color tabBackColor = _hoverColor;

				using (SolidBrush backBrush = new SolidBrush(tabBackColor))
				using (SolidBrush selectedBrush = new SolidBrush(_selectedColor))
				{
					DrawBackground(e.Graphics, selectedTab, backBrush, selectedBrush, selectedTab._highlight);
				}

				DrawText(e.Graphics, selectedTab);

				// Draw the close button
				SmoothingMode prevSmoothingMode = e.Graphics.SmoothingMode;
				e.Graphics.SmoothingMode = System.Drawing.Drawing2D.SmoothingMode.AntiAlias;

				float dpiScaleX = e.Graphics.DpiX / 96.0f;
				float dpiScaleY = e.Graphics.DpiY / 96.0f;

				int closeMidX = selectedTab._minX + selectedTab._width - (int)((TabPadding + TabCloseButtonWidth) * dpiScaleX / 2);// + (13 / 2);
				int closeMidY = (ClientSize.Height - (int)(4 * dpiScaleY)) / 2;

				Color closeColor = _isHoveringCloseBtn ? _closeColorHovered : _closeColor;
				Rectangle closeButton = new Rectangle(closeMidX - (int)((13 / 2) * dpiScaleX), closeMidY - (int)((13 / 2) * dpiScaleY), (int)(13 * dpiScaleX), (int)(13 * dpiScaleY));
				using (SolidBrush backBrush = new SolidBrush(closeColor))
				{
					e.Graphics.FillEllipse(backBrush, closeButton);
				}

				using(Pen crossPen = new Pen(tabBackColor, 2.0f * dpiScaleX))
				{
					float indentX = 3.5f * dpiScaleX;
					float indentY = 3.5f * dpiScaleY;
					e.Graphics.DrawLine(crossPen, closeButton.Left + indentX, closeButton.Top + indentY, closeButton.Right - indentX, closeButton.Bottom - indentY);
					e.Graphics.DrawLine(crossPen, closeButton.Left + indentX, closeButton.Bottom - indentY, closeButton.Right - indentX, closeButton.Top + indentY);
				}
				e.Graphics.SmoothingMode = prevSmoothingMode;

				// Border
				using (Pen borderPen = new Pen(_borderColor))
				{
					e.Graphics.DrawLine(borderPen, selectedTab._minX, 0, selectedTab._minX + selectedTab._width, 0);
					e.Graphics.DrawLine(borderPen, selectedTab._minX, 0, selectedTab._minX, ClientSize.Height - 2);
					e.Graphics.DrawLine(borderPen, selectedTab._minX + selectedTab._width, 0, selectedTab._minX + selectedTab._width, ClientSize.Height - 2);
				}
			}

			using (Pen borderPen = new Pen(_borderColor))
			{
				e.Graphics.DrawLine(SystemPens.ControlDarkDark, 0, ClientSize.Height - 2, ClientSize.Width, ClientSize.Height - 2);
				e.Graphics.DrawLine(borderPen, 0, ClientSize.Height - 1, ClientSize.Width, ClientSize.Height - 1);
			}
		}

		void DrawBackground(Graphics graphics, TabData tab, Brush backgroundBrush, Brush stripeBrush, Tuple<Color, float>? highlight)
		{
			float dpiScaleY = graphics.DpiY / 96.0f;

			graphics.SmoothingMode = System.Drawing.Drawing2D.SmoothingMode.None;

			graphics.FillRectangle(backgroundBrush, tab._minX + 1, 1, tab._width - 1, ClientSize.Height - 3);
			graphics.FillRectangle(stripeBrush, tab._minX + 1, ClientSize.Height - (int)(5 * dpiScaleY), tab._width - 1, (int)(5 * dpiScaleY) - 2);

			if (tab._tintColor.HasValue)
			{
				int midX = tab._minX + tab._width / 2;
				using (LinearGradientBrush brush = new LinearGradientBrush(new Rectangle(midX, 1, 1, ClientSize.Height), Color.FromArgb(64, tab._tintColor.Value), Color.FromArgb(32, tab._tintColor.Value), LinearGradientMode.Vertical))
				{
					graphics.FillRectangle(brush, tab._minX + 1, 1, tab._width - 1, ClientSize.Height - 3);
				}
			}

			if (highlight != null && highlight.Item2 > 0.0f)
			{
				using (SolidBrush brush = new SolidBrush(highlight.Item1))
				{
					graphics.FillRectangle(brush, tab._minX + 1, ClientSize.Height - (int)(5 * dpiScaleY), (int)((tab._width - 2) * highlight.Item2), (int)(5 * dpiScaleY) - 2);
				}
			}
		}

		void DrawText(Graphics graphics, TabData tab)
		{
			float dpiScaleX = graphics.DpiX / 96.0f;
			float dpiScaleY = graphics.DpiY / 96.0f;
			TextRenderer.DrawText(graphics, tab._name, Font, new Rectangle(tab._minX + (int)(TabPadding * dpiScaleX), 0, tab._textSize.Width, ClientSize.Height - (int)(3 * dpiScaleY)), _textColor, TextFormatFlags.NoPadding | TextFormatFlags.EndEllipsis | TextFormatFlags.VerticalCenter);
		}

		public void ApplyTheme(ApplicationThemeManager themeManager)
		{
			_textColor = themeManager.Theme.TextColor;
			_hoverColor = themeManager.Theme.InputBackgroundColor;
			_selectedColor = themeManager.Theme.LinkColor;
			_borderColor = themeManager.Theme.InputBorderColor;
			_closeColor = themeManager.Theme.LinkColor;
			_closeColorHovered = themeManager.Theme.LinkColorHover;

			float hoverWeight = 0.4f;
			_hoverStripeColor = Color.FromArgb(
				(int)Math.Round(_hoverColor.R * (1 - hoverWeight) + Color.Gray.R * hoverWeight),
				(int)Math.Round(_hoverColor.G * (1 - hoverWeight) + Color.Gray.G * hoverWeight),
				(int)Math.Round(_hoverColor.B * (1 - hoverWeight) + Color.Gray.B * hoverWeight));

			themeManager.DefaultApplyThemeToControl(this);
		}
	}
}
