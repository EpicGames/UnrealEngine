// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Runtime.InteropServices;
using System.Windows.Forms;

namespace UnrealGameSync
{
	class TextBoxWithCueBanner : TextBox
	{
		private const int EmSetcuebanner = 0x1501;

		[DllImport("user32.dll", CharSet = CharSet.Unicode)]
		private static extern IntPtr SendMessage(IntPtr hWnd, int msg, IntPtr wparam, string lparam);

		private string _cueBannerValue = String.Empty;

		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public string CueBanner
		{
			get => _cueBannerValue;
			set
			{
				_cueBannerValue = value;
				UpdateCueBanner();
			}
		}

		private void UpdateCueBanner()
		{
			if (IsHandleCreated)
			{
				SendMessage(Handle, EmSetcuebanner, (IntPtr)1, _cueBannerValue);
			}
		}

		protected override void OnHandleCreated(EventArgs e)
		{
			base.OnHandleCreated(e);
			UpdateCueBanner();
		}
	}
}
