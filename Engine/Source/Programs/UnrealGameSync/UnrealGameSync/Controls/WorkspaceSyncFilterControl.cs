// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Drawing;
using System.IO;
using System.Linq;
using System.Text;
using System.Windows.Forms;

namespace UnrealGameSync.Controls
{
	public partial class WorkspaceSyncFilterControl : UserControl
	{
		private int _categoriesCheckListHoverIndex = -1;

		private Timer _categoriesCheckListTooltipTimer { get; } = new Timer { Interval = 300 };

		private Timer _syncBaseContentTooltipTimer { get; } = new Timer { Interval = 300 };

		private class ViewItem
		{
			public bool Locked { get; set; } = false;
			public string Value { get; set; } = String.Empty;

			public ViewItem(bool locked, string value)
			{
				Locked = locked;
				Value = value;
			}
		}

		private readonly BindingList<ViewItem> _views = new BindingList<ViewItem>();
		private readonly List<ViewItem> _presetViews = new List<ViewItem>();

		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public string SyncBaseContentToolTipTitle { get; set; } = String.Empty;
		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public string SyncBaseContentToolTipContent { get; set; } = String.Empty;

		public WorkspaceSyncFilterControl()
		{
			InitializeComponent();
			ViewDataGrid.RowHeadersVisible = false;
			ViewDataGrid.ColumnHeadersVisible = false;
			ViewDataGrid.AutoGenerateColumns = true;
			ViewDataGrid.EditMode = DataGridViewEditMode.EditOnEnter;
			ViewDataGrid.AllowUserToAddRows = true;
			ViewDataGrid.DefaultValuesNeeded += ViewDataGrid_DefaultValuesNeeded;
			ViewDataGrid.CellBeginEdit += ViewDataGrid_CellBeginEdit;
			ViewDataGrid.UserDeletingRow += ViewDataGrid_UserDeletingRow;

			_views.AddingNew += new AddingNewEventHandler(Views_AddingNew);

			ViewDataGrid.DataSource = _views;

			SyncBaseContentCheckBox.MouseHover += SyncBaseContentCheckBox_MouseHover;
			SyncBaseContentCheckBox.MouseLeave += (_, __) => SyncBaseContentToolTip.Hide(SyncBaseContentCheckBox);

			CategoriesCheckList.MouseMove += CheckedListBox_MouseMove;
			CategoriesCheckList.MouseLeave += (_, __) => CategoriesToolTip.Hide(CategoriesCheckList);
			CategoriesCheckList.MouseWheel += (_, __) => CategoriesToolTip.Hide(CategoriesCheckList);
			CategoriesCheckList.MouseDown += (_, __) => CategoriesToolTip.Hide(CategoriesCheckList);

			_categoriesCheckListTooltipTimer.Tick += CategoriesCheckListTooltipTimer_Tick;
			_syncBaseContentTooltipTimer.Tick += BaseSyncContentTooltipTimer_Tick;

			SyncBaseContentFlowLayout.MouseMove += SyncBaseContentFlowLayout_MouseMove;
		}

		private void SyncBaseContentFlowLayout_MouseMove(object? sender, MouseEventArgs e)
		{
			Control? control = SyncBaseContentFlowLayout.GetChildAtPoint(e.Location);
			if (control == null)
			{
				_syncBaseContentTooltipTimer.Stop();
				SyncBaseContentToolTip.Hide(SyncBaseContentCheckBox);
				return;
			}

			if (control.Name != SyncBaseContentCheckBox.Name)
			{
				_syncBaseContentTooltipTimer.Stop();
				SyncBaseContentToolTip.Hide(SyncBaseContentCheckBox);
				return;
			}

			if (control.Enabled)
			{
				return;
			}

			SyncBaseContentToolTip.ToolTipTitle = SyncBaseContentToolTipTitle;
			SyncBaseContentToolTip.SetToolTip(SyncBaseContentCheckBox, SyncBaseContentToolTipContent);

			_syncBaseContentTooltipTimer.Start();
		}

		private void SyncBaseContentCheckBox_MouseHover(object? sender, EventArgs e)
		{
			SyncBaseContentToolTip.ToolTipTitle = SyncBaseContentToolTipTitle;
			SyncBaseContentToolTip.SetToolTip(SyncBaseContentCheckBox, SyncBaseContentToolTipContent);
		}

		private void CheckedListBox_MouseMove(object? sender, MouseEventArgs e)
		{
			int index = CategoriesCheckList.IndexFromPoint(e.Location);

			if (index == _categoriesCheckListHoverIndex)
			{
				return;

			}

			if (index >= 0 && index < CategoriesCheckList.Items.Count)
			{
				_categoriesCheckListHoverIndex = index;
				_categoriesCheckListTooltipTimer.Start();
			}
			else
			{
				CategoriesToolTip.SetToolTip(CategoriesCheckList, String.Empty);
				CategoriesToolTip.Hide(CategoriesCheckList);
			}
		}

		private void CategoriesCheckListTooltipTimer_Tick(object? sender, EventArgs e)
		{
			_categoriesCheckListTooltipTimer.Stop();

			if (_categoriesCheckListHoverIndex >= 0 && _categoriesCheckListHoverIndex < CategoriesCheckList.Items.Count)
			{
				object item = CategoriesCheckList.Items[_categoriesCheckListHoverIndex];
				System.Reflection.PropertyInfo? property = item.GetType().GetProperty("ToolTip");

				if (property != null)
				{
					if (property.GetValue(item) is string toolTip)
					{
						if (SyncBaseContentCheckBox.Visible && SyncBaseContentCheckBox.Checked == false)
						{
							toolTip = $"{toolTip}\n\n" +
									  $@"(Click the Sync Base Content checkbox to allow enabling/disabling of sync categories.)";
						}

						CategoriesToolTip.ToolTipTitle = item.ToString();
						CategoriesToolTip.SetToolTip(CategoriesCheckList, toolTip);
					}
				}
			}
		}

		private void BaseSyncContentTooltipTimer_Tick(object? sender, EventArgs e)
		{
			_syncBaseContentTooltipTimer.Stop();

			Point coordinates = CategoriesTableLayoutPanel.PointToClient(Cursor.Position);
			SyncBaseContentToolTip.Show(
				SyncBaseContentToolTipContent,
				CategoriesTableLayoutPanel,
				coordinates.X,
				coordinates.Y);
		}

		void Views_AddingNew(object? sender, AddingNewEventArgs e)
		{
			e.NewObject = new ViewItem(false, "change me");
		}

		private void ViewDataGrid_CellBeginEdit(object? sender, DataGridViewCellCancelEventArgs e)
		{
			if (e.RowIndex >= 0 && e.RowIndex < ViewDataGrid.Rows.Count)
			{
				DataGridViewRow? row = ViewDataGrid.Rows[e.RowIndex];
				if (row.DataBoundItem is ViewItem item)
				{
					e.Cancel = item.Locked;
				}
			}
		}

		private void ViewDataGrid_UserDeletingRow(object? sender, DataGridViewRowCancelEventArgs e)
		{
			if (e.Row != null && e.Row.DataBoundItem is ViewItem item)
			{
				e.Cancel = item.Locked;
			}
		}

		private void ViewDataGrid_DefaultValuesNeeded(object? sender, System.Windows.Forms.DataGridViewRowEventArgs e)
		{
			e.Row.Cells[nameof(ViewItem.Locked)].Value = false;
			e.Row.Cells[nameof(ViewItem.Value)].Value = String.Empty;
		}

		private void SyntaxButton_LinkClicked(object sender, LinkLabelLinkClickedEventArgs e)
		{
			using SyncFilterSyntax dialog = new SyncFilterSyntax();
			dialog.ShowDialog(ParentForm);
		}

		public void SetView(string preset, string[] presetViews, string[] views)
		{
			_views.Clear();

			if (!String.IsNullOrWhiteSpace(preset))
			{
				foreach (string presetView in presetViews)
				{
					_views.Add(new ViewItem(true, $"{presetView} [Preset: {preset} - Locked]"));
					_presetViews.Add(new ViewItem(true, $"{presetView} [Preset: {preset} - Locked]"));
				}
			}

			foreach (string view in views)
			{
				_views.Add(new ViewItem(false, view));
			}

			foreach (DataGridViewColumn column in ViewDataGrid.Columns)
			{
				if (column.Name.Equals(nameof(ViewItem.Locked), StringComparison.Ordinal))
				{
					column.Visible = false;
				}

				if (column.Name.Equals(nameof(ViewItem.Value), StringComparison.Ordinal))
				{
					column.AutoSizeMode = DataGridViewAutoSizeColumnMode.Fill;
				}
			}

			foreach (DataGridViewRow? row in ViewDataGrid.Rows)
			{
				if (row == null)
				{
					continue;
				}

				if (row.DataBoundItem is ViewItem item)
				{
					if (item.Locked)
					{
						row.ReadOnly = true;
						foreach (DataGridViewCell cell in row.Cells)
						{
							cell.ReadOnly = true;
						}
					}
				}
			}
		}

		public string[] GetView()
		{
			List<string> views = new List<string>();
			foreach (ViewItem item in _views)
			{
				if (item.Locked)
				{
					continue;
				}

				if (String.IsNullOrEmpty(item.Value) == false)
				{
					views.Add(item.Value);
				}
			}

			return views.ToArray();
		}

		private void CopyViewsToClipBoard_Click(object sender, EventArgs e)
		{
			try
			{
				if (!_views.Any())
				{
					return;
				}

				StringBuilder sb = new StringBuilder();

				foreach (ViewItem item in _views)
				{
					if (String.IsNullOrEmpty(item.Value) == false)
					{
						sb.AppendLine(item.Value);
					}
				}

				string? text = sb.ToString();
				if (String.IsNullOrWhiteSpace(text))
				{
					return;
				}

				Clipboard.SetText(sb.ToString());
			}
			catch
			{
				// ignored
			}
		}

		private async void PasteCustomViewsToClipboard_Click(object sender, EventArgs e)
		{
			try
			{
				string text = Clipboard.GetText();
				if (String.IsNullOrWhiteSpace(text))
				{
					return;
				}

				_views.Clear();

				HashSet<string> uniques = new HashSet<string>();

				foreach (ViewItem presetView in _presetViews)
				{
					_views.Add(presetView);
				}

				using StringReader reader = new StringReader(text);

				while (reader.Peek() != -1)
				{
					string? line = await reader.ReadLineAsync();
					if (String.IsNullOrWhiteSpace(line))
					{
						continue;
					}

					if (_presetViews.Any(pv => pv.Value.Equals(line, StringComparison.OrdinalIgnoreCase)))
					{
						continue;
					}

					uniques.Add(line);
				}

				foreach (string unique in uniques)
				{
					_views.Add(new ViewItem(false, unique));
				}
			}
			catch
			{
				// ignored
			}
		}

		private void ClearCustomViewsToClipboard_Click(object sender, EventArgs e)
		{
			_views.Clear();

			foreach (ViewItem presetView in _presetViews)
			{
				_views.Add(presetView);
			}
		}
	}
}
