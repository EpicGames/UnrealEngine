// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	static class DeleteFilesTask
	{
		public static async Task RunAsync(IPerforceSettings perforceSettings, List<FileInfo> filesToSync, List<FileInfo> filesToDelete, List<DirectoryInfo> directoriesToDelete, ILogger logger, CancellationToken cancellationToken)
		{
			StringBuilder failMessage = new StringBuilder();

			if (filesToSync.Count > 0)
			{
				// Do batches to avoid blowing out the RPC, 10K files take around 50 seconds to sync.
				const int BatchSize = 10_000;
				for (int startIndex = 0; startIndex < filesToSync.Count; startIndex += BatchSize)
				{
					logger.LogInformation("syncing: {StartIndex}/{Count}, percent {Percent}", startIndex, filesToSync.Count,
						(int)Math.Round(100 * (float)startIndex / filesToSync.Count));

					int batchCount = Math.Min(BatchSize, filesToSync.Count - startIndex);
					List<FileInfo> batchFilesToSync = filesToSync.GetRange(startIndex, batchCount);
					using IPerforceConnection perforce = await PerforceConnection.CreateAsync(perforceSettings, logger);
					List<string> revisionsToSync = new List<string>();
					foreach (FileInfo fileToSync in batchFilesToSync)
					{
						revisionsToSync.Add(String.Format("{0}#have", PerforceUtils.EscapePath(fileToSync.FullName)));
					}
					List<PerforceResponse<SyncRecord>> failedRecords = await perforce.TrySyncAsync(SyncOptions.Force, revisionsToSync, cancellationToken).Where(x => x.Failed).ToListAsync(cancellationToken);
					foreach (PerforceResponse<SyncRecord> failedRecord in failedRecords)
					{
						failMessage.Append(failedRecord.ToString());
					}
				}
			}

			foreach (FileInfo fileToDelete in filesToDelete)
			{
				try
				{
					fileToDelete.IsReadOnly = false;
					fileToDelete.Delete();
				}
				catch (Exception ex)
				{
					logger.LogWarning(ex, "Unable to delete {File}", fileToDelete.FullName);
					failMessage.AppendFormat("{0} ({1})\r\n", fileToDelete.FullName, ex.Message.Trim());
				}
			}
			foreach (DirectoryInfo directoryToDelete in directoriesToDelete)
			{
				try
				{
					directoryToDelete.Delete(true);
				}
				catch (Exception ex)
				{
					logger.LogWarning(ex, "Unable to delete {Directory}", directoryToDelete.FullName);
					failMessage.AppendFormat("{0} ({1})\r\n", directoryToDelete.FullName, ex.Message.Trim());
				}
			}

			if (failMessage.Length > 0)
			{
				throw new UserErrorException(failMessage.ToString());
			}
		}
	}
}
