// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	enum EventType
	{
		Syncing,

		// Reviews
		Compiles,
		DoesNotCompile,
		Good,
		Bad,
		Unknown,

		// Starred builds
		Starred,
		Unstarred,

		// Investigating events
		Investigating,
		Resolved,
	}

	class LatestData
	{
		public int Version { get; set; } = 1;
		public long LastEventId { get; set; }
		public long LastCommentId { get; set; }
		public long LastBuildId { get; set; }
	}

	class EventData
	{
		public long Id { get; set; }
		public int Change { get; set; }
		public string UserName { get; set; } = String.Empty;
		public EventType Type { get; set; }
		public string Project { get; set; } = String.Empty;
	}

	class CommentData
	{
		public long Id { get; set; }
		public int ChangeNumber { get; set; }
		public string UserName { get; set; } = String.Empty;
		public string Text { get; set; } = String.Empty;
		public string Project { get; set; } = String.Empty;
	}

	enum BadgeResult
	{
		Starting,
		Failure,
		Warning,
		Success,
		Skipped,
	}

	class Link
	{
		public string Text { get; set; } = String.Empty;
		public string Url { get; set; } = String.Empty;
	}

	class BadgeMetadata
	{
		public List<Link>? Links { get; set; } = new List<Link>();
	}

	class BadgeData
	{
		public long Id { get; set; }
		public int ChangeNumber { get; set; }
		public string BuildType { get; set; } = String.Empty;
		public BadgeResult Result { get; set; }
		public string Url { get; set; } = String.Empty;
		public string Project { get; set; } = String.Empty;
		public BadgeMetadata? Metadata { get; set; }

		public bool IsSuccess => Result == BadgeResult.Success || Result == BadgeResult.Warning;

		public bool IsFailure => Result == BadgeResult.Failure;

		public string BadgeName
		{
			get
			{
				if (BuildType == null)
				{
					return "Unknown";
				}

				int idx = BuildType.IndexOf(':', StringComparison.Ordinal);
				if (idx == -1)
				{
					return BuildType;
				}
				else
				{
					return BuildType.Substring(0, idx);
				}
			}
		}

		public string BadgeLabel
		{
			get
			{
				if (BuildType == null)
				{
					return "Unknown";
				}

				int idx = BuildType.IndexOf(':', StringComparison.Ordinal);
				if (idx == -1)
				{
					return BuildType;
				}
				else
				{
					return BuildType.Substring(idx + 1);
				}
			}
		}
	}

	enum ReviewVerdict
	{
		Unknown,
		Good,
		Bad,
		Mixed,
	}

	public enum UgsUserVote
	{
		None,
		CompileSuccess,
		CompileFailure,
		Good,
		Bad
	}

	class GetUserDataResponseV2
	{
		public string User { get; set; } = String.Empty;
		public long? SyncTime { get; set; }
		public UgsUserVote? Vote { get; set; }
		public string Comment { get; set; } = String.Empty;
		public bool? Investigating { get; set; }
		public bool? Starred { get; set; }
	}

	class GetBadgeDataResponseV2
	{
		public string Name { get; set; } = String.Empty;
		public string Url { get; set; } = String.Empty;
		public BadgeResult State { get; set; }
	}

	class GetMetadataResponseV2
	{
		public int Change { get; set; }
		public string Project { get; set; } = String.Empty;
		public List<GetUserDataResponseV2> Users { get; set; } = new List<GetUserDataResponseV2>();
		public List<GetBadgeDataResponseV2> Badges { get; set; } = new List<GetBadgeDataResponseV2>();
	}

	class GetMetadataListResponseV2
	{
		public long SequenceNumber { get; set; }
		public List<GetMetadataResponseV2> Items { get; set; } = new List<GetMetadataResponseV2>();
	}

	class UpdateMetadataRequestV2
	{
		public string? Stream { get; set; }
		public int Change { get; set; }
		public string? Project { get; set; }
		public string? UserName { get; set; }
		public bool? Synced { get; set; }
		public string? Vote { get; set; }
		public bool? Investigating { get; set; }
		public bool? Starred { get; set; }
		public string? Comment { get; set; }
	}

	class EventSummary
	{
		public int ChangeNumber;
		public ReviewVerdict Verdict;
		public List<EventData> SyncEvents = new List<EventData>();
		public List<EventData> Reviews = new List<EventData>();
		public List<string> CurrentUsers = new List<string>();
		public EventData? LastStarReview;
		public List<BadgeData> Badges = new List<BadgeData>();
		public List<CommentData> Comments = new List<CommentData>();

		public GetMetadataResponseV2? SharedMetadata;
		public GetMetadataResponseV2? ProjectMetadata;
	}

	class EventMonitor : IDisposable
	{
		readonly string? _apiUrl;
		int _apiVersion;
		readonly string _project;
		readonly string _currentUserName;
		readonly SynchronizationContext _synchronizationContext;
#pragma warning disable CA2213 // warning CA2213: 'EventMonitor' contains field '_cancellationSource' that is of IDisposable type 'CancellationTokenSource', but it is never disposed. Change the Dispose method on 'EventMonitor' to call Close or Dispose on this field.
		readonly CancellationTokenSource _cancellationSource;
#pragma warning restore CA2213
		Task? _workerTask;
		readonly AsyncEvent _refreshEvent = new AsyncEvent();
		readonly ConcurrentQueue<EventData> _outgoingEvents = new ConcurrentQueue<EventData>();
		readonly ConcurrentQueue<EventData> _incomingEvents = new ConcurrentQueue<EventData>();
		readonly ConcurrentQueue<CommentData> _outgoingComments = new ConcurrentQueue<CommentData>();
		readonly ConcurrentQueue<CommentData> _incomingComments = new ConcurrentQueue<CommentData>();
		readonly ConcurrentQueue<BadgeData> _incomingBadges = new ConcurrentQueue<BadgeData>();
		readonly SortedDictionary<int, EventSummary> _changeNumberToSummary = new SortedDictionary<int, EventSummary>();
		readonly Dictionary<string, EventData> _userNameToLastSyncEvent = new Dictionary<string, EventData>(StringComparer.InvariantCultureIgnoreCase);
		readonly Dictionary<string, BadgeData> _badgeNameToLatestData = new Dictionary<string, BadgeData>();
		readonly ILogger _logger;
		readonly IAsyncDisposer _asyncDisposer;
		readonly LatestData _latestIds;
		HashSet<int> _filterChangeNumbers = new HashSet<int>();

		// Investigation events are collected in descending CL# order
		readonly SortedSet<EventData> _investigationEvents = new SortedSet<EventData>(
			Comparer<EventData>.Create((first, second) =>
			{
				int clOrder = second.Change.CompareTo(first.Change);
				return clOrder != 0 ? clOrder : first.Id.CompareTo(second.Id);
			}));
		SortedSet<EventData>? _activeInvestigations = null;

		// MetadataV2
		readonly string _metadataStream;
		readonly string _metadataProject;
		readonly ConcurrentQueue<GetMetadataResponseV2> _incomingMetadata = new ConcurrentQueue<GetMetadataResponseV2>();
		int _minChange;
		int _newMinChange;
		long _metadataSequenceNumber;

		public Action? OnUpdatesReady;

		public EventMonitor(string? inApiUrl, string inProject, string inCurrentUserName, IServiceProvider serviceProvider)
		{
			_apiUrl = inApiUrl;
			_project = inProject;
			_currentUserName = inCurrentUserName;
			_logger = serviceProvider.GetRequiredService<ILogger<EventMonitor>>();
			_asyncDisposer = serviceProvider.GetRequiredService<IAsyncDisposer>();
			_synchronizationContext = SynchronizationContext.Current!;
			_cancellationSource = new CancellationTokenSource();

			_latestIds = new LatestData { LastBuildId = 0, LastCommentId = 0, LastEventId = 0 };

			_metadataProject = String.Empty;
			_metadataStream = _project.ToLowerInvariant().TrimEnd('/');
			if (_metadataStream.StartsWith("//", StringComparison.Ordinal))
			{
				int nextIdx = _metadataStream.IndexOf('/', 2);
				if (nextIdx != -1)
				{
					nextIdx = _metadataStream.IndexOf('/', nextIdx + 1);
					if (nextIdx != -1)
					{
						_metadataProject = _metadataStream.Substring(nextIdx + 1);
						_metadataStream = _metadataStream.Substring(0, nextIdx);
					}
				}
			}

			if (_apiUrl == null)
			{
				LastStatusMessage = "Database functionality disabled due to empty ApiUrl.";
			}
			else
			{
				_logger.LogInformation("Using connection string: {ApiUrl}", _apiUrl);
			}
		}

		public void Start()
		{
			_workerTask ??= Task.Run(() => PollForUpdatesAsync(_cancellationSource.Token));
		}

		public void Dispose()
		{
			OnUpdatesReady = null;

			if (_workerTask != null)
			{
				_cancellationSource.Cancel();
				_asyncDisposer.Add(_workerTask.ContinueWith(_ => _cancellationSource.Dispose(), CancellationToken.None, TaskContinuationOptions.None, TaskScheduler.Default));
				_workerTask = null;
			}
		}

		public void FilterChanges(IEnumerable<int> changeNumbers)
		{
			// Build a lookup for all the change numbers
			_filterChangeNumbers = new HashSet<int>(changeNumbers);

			// Figure out the minimum changelist number to fetch
			int prevNewMinChange = _newMinChange;
			if (changeNumbers.Any())
			{
				_newMinChange = changeNumbers.Min(x => x);
			}
			else
			{
				_newMinChange = 0;
			}

			// Remove any changes which are no longer relevant
			if (_apiVersion == 2)
			{
				while (_changeNumberToSummary.Count > 0)
				{
					int firstChange = _changeNumberToSummary.Keys.First();
					if (firstChange >= _newMinChange)
					{
						break;
					}
					_changeNumberToSummary.Remove(firstChange);
				}
			}

			// Clear out the list of active users for each review we have
			_userNameToLastSyncEvent.Clear();
			foreach (EventSummary summary in _changeNumberToSummary.Values)
			{
				summary.CurrentUsers.Clear();
			}

			// Add all the user reviews back in again
			foreach (EventSummary summary in _changeNumberToSummary.Values)
			{
				foreach (EventData syncEvent in summary.SyncEvents)
				{
					ApplyFilteredUpdate(syncEvent);
				}
			}

			// Clear the list of active investigations, since this depends on the changes we're showing
			_activeInvestigations = null;

			// Trigger an update if there's something to do
			if (_newMinChange < prevNewMinChange || (_newMinChange != 0 && prevNewMinChange == 0))
			{
				_refreshEvent.Set();
			}
		}

		protected EventSummary FindOrAddSummary(int changeNumber)
		{
			EventSummary? summary;
			if (!_changeNumberToSummary.TryGetValue(changeNumber, out summary))
			{
				summary = new EventSummary();
				summary.ChangeNumber = changeNumber;
				_changeNumberToSummary.Add(changeNumber, summary);
			}
			return summary;
		}

		public string LastStatusMessage
		{
			get;
			private set;
		} = String.Empty;

		public void ApplyUpdates()
		{
			GetMetadataResponseV2? metadata;
			while (_incomingMetadata.TryDequeue(out metadata))
			{
				ConvertMetadataToEvents(metadata);
			}

			EventData? evt;
			while (_incomingEvents.TryDequeue(out evt))
			{
				ApplyEventUpdate(evt);
			}

			BadgeData? badge;
			while (_incomingBadges.TryDequeue(out badge))
			{
				ApplyBadgeUpdate(badge);
			}

			CommentData? comment;
			while (_incomingComments.TryDequeue(out comment))
			{
				ApplyCommentUpdate(comment);
			}
		}

		void ApplyEventUpdate(EventData evt)
		{
			EventSummary summary = FindOrAddSummary(evt.Change);
			if (evt.Type == EventType.Starred || evt.Type == EventType.Unstarred)
			{
				// If it's a star or un-star review, process that separately
				if (summary.LastStarReview == null || evt.Id > summary.LastStarReview.Id)
				{
					summary.LastStarReview = evt;
				}
			}
			else if (evt.Type == EventType.Investigating || evt.Type == EventType.Resolved)
			{
				// Events are automatically sorted by descending CL#
				_investigationEvents.Add(evt);
				_activeInvestigations = null;
			}
			else if (evt.Type == EventType.Syncing)
			{
				summary.SyncEvents.RemoveAll(x => String.Equals(x.UserName, evt.UserName, StringComparison.OrdinalIgnoreCase));
				summary.SyncEvents.Add(evt);
				ApplyFilteredUpdate(evt);
			}
			else if (IsReview(evt.Type))
			{
				// Try to find an existing review by this user. If we already have a newer review, ignore this one. Otherwise remove it.
				EventData? existingReview = summary.Reviews.Find(x => String.Equals(x.UserName, evt.UserName, StringComparison.OrdinalIgnoreCase));
				if (existingReview != null)
				{
					if (existingReview.Id <= evt.Id)
					{
						summary.Reviews.Remove(existingReview);
					}
					else
					{
						return;
					}
				}

				// Add the new review, and find the new verdict for this change
				summary.Reviews.Add(evt);
				summary.Verdict = GetVerdict(summary.Reviews, summary.Badges);
			}
			else
			{
				// Unknown type
			}
		}

		void ApplyBadgeUpdate(BadgeData badge)
		{
			EventSummary summary = FindOrAddSummary(badge.ChangeNumber);

			BadgeData? existingBadge = summary.Badges.Find(x => x.ChangeNumber == badge.ChangeNumber && x.BuildType == badge.BuildType);
			if (existingBadge != null)
			{
				if (existingBadge.Id <= badge.Id)
				{
					summary.Badges.Remove(existingBadge);
				}
				else
				{
					return;
				}
			}

			summary.Badges.Add(badge);
			summary.Verdict = GetVerdict(summary.Reviews, summary.Badges);

			BadgeData? latestBadge;
			if (!_badgeNameToLatestData.TryGetValue(badge.BadgeName, out latestBadge) || badge.ChangeNumber > latestBadge.ChangeNumber || (badge.ChangeNumber == latestBadge.ChangeNumber && badge.Id > latestBadge.Id))
			{
				_badgeNameToLatestData[badge.BadgeName] = badge;
			}
		}

		void ApplyCommentUpdate(CommentData comment)
		{
			EventSummary summary = FindOrAddSummary(comment.ChangeNumber);
			if (String.Equals(comment.UserName, _currentUserName, StringComparison.OrdinalIgnoreCase) && summary.Comments.Count > 0 && summary.Comments.Last().Id == Int64.MaxValue)
			{
				// This comment was added by PostComment(), to mask the latency of a round trip to the server. Remove it now we have the sorted comment.
				summary.Comments.RemoveAt(summary.Comments.Count - 1);
			}
			AddPerUserItem(summary.Comments, comment, x => x.Id, x => x.UserName);
		}

		static bool AddPerUserItem<T>(List<T> items, T newItem, Func<T, long> idSelector, Func<T, string> userSelector)
		{
			int insertIdx = items.Count;

			for (; insertIdx > 0 && idSelector(items[insertIdx - 1]) >= idSelector(newItem); insertIdx--)
			{
				if (String.Equals(userSelector(items[insertIdx - 1]), userSelector(newItem), StringComparison.OrdinalIgnoreCase))
				{
					return false;
				}
			}

			items.Insert(insertIdx, newItem);

			for (; insertIdx > 0; insertIdx--)
			{
				if (String.Equals(userSelector(items[insertIdx - 1]), userSelector(newItem), StringComparison.OrdinalIgnoreCase))
				{
					items.RemoveAt(insertIdx - 1);
				}
			}

			return true;
		}

		static EventType? GetEventTypeFromVote(UgsUserVote? state)
		{
			if (state != null)
			{
				switch (state.Value)
				{
					case UgsUserVote.CompileSuccess:
						return EventType.Compiles;
					case UgsUserVote.CompileFailure:
						return EventType.DoesNotCompile;
					case UgsUserVote.Good:
						return EventType.Good;
					case UgsUserVote.Bad:
						return EventType.Bad;
				}
			}
			return null;
		}

		void ConvertMetadataToEvents(GetMetadataResponseV2 metadata)
		{
			EventSummary newSummary = new EventSummary();
			newSummary.ChangeNumber = metadata.Change;

			EventSummary? summary;
			if (_changeNumberToSummary.TryGetValue(metadata.Change, out summary))
			{
				foreach (string currentUser in summary.CurrentUsers)
				{
					_userNameToLastSyncEvent.Remove(currentUser);
				}

				newSummary.SharedMetadata = summary.SharedMetadata;
				newSummary.ProjectMetadata = summary.ProjectMetadata;
			}

			if (String.IsNullOrEmpty(metadata.Project))
			{
				newSummary.SharedMetadata = metadata;
			}
			else
			{
				newSummary.ProjectMetadata = metadata;
			}

			_changeNumberToSummary[newSummary.ChangeNumber] = newSummary;

			if (newSummary.SharedMetadata != null)
			{
				PostEvents(newSummary.SharedMetadata);
			}

			if (newSummary.ProjectMetadata != null)
			{
				PostEvents(newSummary.ProjectMetadata);
			}
		}

		void PostEvents(GetMetadataResponseV2 metadata)
		{
			if (metadata.Badges != null)
			{
				foreach (GetBadgeDataResponseV2 badgeData in metadata.Badges)
				{
					BadgeData badge = new BadgeData();
					badge.Id = ++_latestIds.LastBuildId;
					badge.ChangeNumber = metadata.Change;
					badge.BuildType = badgeData.Name;
					badge.Result = badgeData.State;
					badge.Url = badgeData.Url;
					badge.Project = metadata.Project;
					_incomingBadges.Enqueue(badge);
				}
			}

			if (metadata.Users != null)
			{
				foreach (GetUserDataResponseV2 userData in metadata.Users)
				{
					if (userData.SyncTime != null)
					{
						EventData evt = new EventData { Id = userData.SyncTime.Value, Change = metadata.Change, Project = metadata.Project, UserName = userData.User, Type = EventType.Syncing };
						_incomingEvents.Enqueue(evt);
					}

					EventType? type = GetEventTypeFromVote(userData.Vote);
					if (type != null)
					{
						EventData evt = new EventData { Id = ++_latestIds.LastEventId, Change = metadata.Change, Project = metadata.Project, UserName = userData.User, Type = type.Value };
						_incomingEvents.Enqueue(evt);
					}

					if (userData.Investigating != null)
					{
						EventType investigationEventType = (userData.Investigating.Value ? EventType.Investigating : EventType.Resolved);
						EventData evt = new EventData { Id = ++_latestIds.LastEventId, Change = metadata.Change, Project = metadata.Project, UserName = userData.User, Type = investigationEventType };
						_incomingEvents.Enqueue(evt);
					}

					if (userData.Starred != null)
					{
						EventType starEventType = (userData.Starred.Value ? EventType.Starred : EventType.Unstarred);
						EventData evt = new EventData { Id = ++_latestIds.LastEventId, Change = metadata.Change, Project = metadata.Project, UserName = userData.User, Type = starEventType };
						_incomingEvents.Enqueue(evt);
					}

					if (userData.Comment != null)
					{
						CommentData comment = new CommentData { Id = ++_latestIds.LastCommentId, ChangeNumber = metadata.Change, Project = metadata.Project, UserName = userData.User, Text = userData.Comment };
						_incomingComments.Enqueue(comment);
					}
				}
			}
		}

		static ReviewVerdict GetVerdict(IEnumerable<EventData> events, IEnumerable<BadgeData> badges)
		{
			int numPositiveReviews = events.Count(x => x.Type == EventType.Good);
			int numNegativeReviews = events.Count(x => x.Type == EventType.Bad);
			if (numPositiveReviews > 0 || numNegativeReviews > 0)
			{
				return GetVerdict(numPositiveReviews, numNegativeReviews);
			}

			int numCompiles = events.Count(x => x.Type == EventType.Compiles);
			int numFailedCompiles = events.Count(x => x.Type == EventType.DoesNotCompile);
			if (numCompiles > 0 || numFailedCompiles > 0)
			{
				return GetVerdict(numCompiles, numFailedCompiles);
			}

			int numBadges = badges.Count(x => x.BuildType == "Editor" && x.IsSuccess);
			int numFailedBadges = badges.Count(x => x.BuildType == "Editor" && x.IsFailure);
			if (numBadges > 0 || numFailedBadges > 0)
			{
				return GetVerdict(numBadges, numFailedBadges);
			}

			return ReviewVerdict.Unknown;
		}

		static ReviewVerdict GetVerdict(int numPositive, int numNegative)
		{
			if (numPositive > (int)(numNegative * 1.5))
			{
				return ReviewVerdict.Good;
			}
			else if (numPositive >= numNegative)
			{
				return ReviewVerdict.Mixed;
			}
			else
			{
				return ReviewVerdict.Bad;
			}
		}

		void ApplyFilteredUpdate(EventData evt)
		{
			if (evt.Type == EventType.Syncing && _filterChangeNumbers.Contains(evt.Change) && !String.IsNullOrEmpty(evt.UserName))
			{
				// Update the active users list for this change
				EventData? lastSync;
				if (_userNameToLastSyncEvent.TryGetValue(evt.UserName, out lastSync))
				{
					if (evt.Id > lastSync.Id)
					{
						_changeNumberToSummary[lastSync.Change].CurrentUsers.RemoveAll(x => String.Equals(x, evt.UserName, StringComparison.OrdinalIgnoreCase));
						FindOrAddSummary(evt.Change).CurrentUsers.Add(evt.UserName);
						_userNameToLastSyncEvent[evt.UserName] = evt;
					}
				}
				else
				{
					FindOrAddSummary(evt.Change).CurrentUsers.Add(evt.UserName);
					_userNameToLastSyncEvent[evt.UserName] = evt;
				}
			}
		}

		async Task PollForUpdatesAsync(CancellationToken cancellationToken)
		{
			EventData? evt = null;
			CommentData? comment = null;
			bool updateThrottledRequests = true;
			double requestThrottle = 90; // seconds to wait for throttled request;
			Stopwatch timer = Stopwatch.StartNew();
			while (!cancellationToken.IsCancellationRequested)
			{
				Task refreshTask = _refreshEvent.Task;

				// If there's no connection string, just empty out the queue
				if (_apiUrl != null)
				{
					// Post all the reviews to the database. We don't send them out of order, so keep the review outside the queue until the next update if it fails
					while (evt != null || _outgoingEvents.TryDequeue(out evt))
					{
						await SendEventToBackendAsync(evt, cancellationToken);
						evt = null;
					}

					// Post all the comments to the database.
					while (comment != null || _outgoingComments.TryDequeue(out comment))
					{
						await SendCommentToBackendAsync(comment, cancellationToken);
						comment = null;
					}

					if (timer.Elapsed > TimeSpan.FromSeconds(requestThrottle))
					{
						updateThrottledRequests = true;
						timer.Restart();
					}

					// Read all the new reviews, pass whether or not to fire the throttled requests
					await ReadEventsFromBackendAsync(updateThrottledRequests, cancellationToken);

					// Send a notification that we're ready to update
					if (!_incomingMetadata.IsEmpty || !_incomingEvents.IsEmpty || !_incomingBadges.IsEmpty || !_incomingComments.IsEmpty)
					{
						_synchronizationContext.Post(_ => OnUpdatesReady?.Invoke(), null);
					}
				}

				// Wait for something else to do
				Task delayTask = Task.Delay(TimeSpan.FromSeconds(30.0), cancellationToken);
				if (await Task.WhenAny(delayTask, refreshTask) == delayTask)
				{
					updateThrottledRequests = true;
				}
			}
		}

		public static bool HasNetCore3()
		{
			DirectoryInfo baseDirInfo = new DirectoryInfo(Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.ProgramFiles), "dotnet", "shared", "Microsoft.NETCore.App"));
			if (baseDirInfo.Exists)
			{
				foreach (DirectoryInfo subDir in baseDirInfo.EnumerateDirectories())
				{
					if (subDir.Name.StartsWith("3.", StringComparison.Ordinal))
					{
						return true;
					}
				}
			}
			return false;
		}

		async Task<bool> SendEventToBackendAsync(EventData evt, CancellationToken cancellationToken)
		{
			try
			{
				Stopwatch timer = Stopwatch.StartNew();
				_logger.LogInformation("Posting event... ({Change}, {UserName}, {Type})", evt.Change, evt.UserName, evt.Type);
				if (_apiVersion == 2)
				{
					await SendMetadataUpdateUpdateV2Async(evt.Change, evt.UserName, evt.Type, null, cancellationToken);
				}
				else
				{
					await RestApi.PostAsync($"{_apiUrl}/api/event", JsonSerializer.Serialize(evt), cancellationToken);
				}
				return true;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed with exception.");
				return false;
			}
		}

		async Task<bool> SendCommentToBackendAsync(CommentData comment, CancellationToken cancellationToken)
		{
			try
			{
				Stopwatch timer = Stopwatch.StartNew();
				_logger.LogInformation("Posting comment... ({Change}, {User}, {Text}, {Project})", comment.ChangeNumber, comment.UserName, comment.Text, comment.Project);
				if (_apiVersion == 2)
				{
					await SendMetadataUpdateUpdateV2Async(comment.ChangeNumber, comment.UserName, null, comment.Text, cancellationToken);
				}
				else
				{
					await RestApi.PostAsync($"{_apiUrl}/api/comment", JsonSerializer.Serialize(comment), cancellationToken);
				}
				_logger.LogInformation("Done in {Time}ms.", timer.ElapsedMilliseconds);
				return true;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed with exception.");
				return false;
			}
		}

		async Task SendMetadataUpdateUpdateV2Async(int change, string userName, EventType? evt, string? comment, CancellationToken cancellationToken)
		{
			UpdateMetadataRequestV2 update = new UpdateMetadataRequestV2();
			update.Stream = _metadataStream;
			update.Project = _metadataProject;
			update.Change = change;
			update.UserName = userName;

			if (evt != null)
			{
				switch (evt)
				{
					case EventType.Syncing:
						update.Synced = true;
						break;
					case EventType.Compiles:
						update.Vote = nameof(UgsUserVote.CompileSuccess);
						break;
					case EventType.DoesNotCompile:
						update.Vote = nameof(UgsUserVote.CompileFailure);
						break;
					case EventType.Good:
						update.Vote = nameof(UgsUserVote.Good);
						break;
					case EventType.Bad:
						update.Vote = nameof(UgsUserVote.Bad);
						break;
					case EventType.Unknown:
						update.Vote = nameof(UgsUserVote.None);
						break;
					case EventType.Starred:
						update.Starred = true;
						break;
					case EventType.Unstarred:
						update.Starred = false;
						break;
					case EventType.Investigating:
						update.Investigating = true;
						break;
					case EventType.Resolved:
						update.Investigating = false;
						break;
				}
			}
			update.Comment = comment;

			await RestApi.PostAsync($"{_apiUrl}/api/metadata", JsonSerializer.Serialize(update), cancellationToken);
		}

		async Task<bool> ReadEventsFromBackendAsync(bool fireThrottledRequests, CancellationToken cancellationToken)
		{
			try
			{
				Stopwatch timer = Stopwatch.StartNew();
				_logger.LogInformation("Polling for events...");

				//////////////
				// Initial Ids 
				//////////////
				if (_apiVersion == 0)
				{
					LatestData initialIds = await RestApi.GetAsync<LatestData>($"{_apiUrl}/api/latest?project={_project}", cancellationToken);
					_apiVersion = (initialIds.Version == 0) ? 1 : initialIds.Version;
					_latestIds.LastBuildId = initialIds.LastBuildId;
					_latestIds.LastCommentId = initialIds.LastCommentId;
					_latestIds.LastEventId = initialIds.LastEventId;
				}

				if (_apiVersion == 2)
				{
					int newMinChangeCopy = _newMinChange;
					if (newMinChangeCopy != 0)
					{
						// If the range of changes has decreased, update the MinChange value before we fetch anything
						_minChange = Math.Max(_minChange, newMinChangeCopy);

						// Get the first part of the query
						string commonArgs = String.Format("stream={0}", _metadataStream);
						if (_metadataProject != null)
						{
							commonArgs += String.Format("&project={0}", _metadataProject);
						}

						// Fetch any updates in the current range of changes
						if (_minChange != 0)
						{
							GetMetadataListResponseV2 newEventList = await RestApi.GetAsync<GetMetadataListResponseV2>($"{_apiUrl}/api/metadata?{commonArgs}&minchange={_minChange}&sequence={_metadataSequenceNumber}", cancellationToken);
							foreach (GetMetadataResponseV2 newEvent in newEventList.Items)
							{
								_incomingMetadata.Enqueue(newEvent);
							}
							_metadataSequenceNumber = Math.Max(newEventList.SequenceNumber, _metadataSequenceNumber);
						}

						// Fetch any new changes
						if (newMinChangeCopy < _minChange)
						{
							GetMetadataListResponseV2 newEvents = await RestApi.GetAsync<GetMetadataListResponseV2>($"{_apiUrl}/api/metadata?{commonArgs}&minchange={newMinChangeCopy}&maxchange={_minChange}", cancellationToken);
							foreach (GetMetadataResponseV2 newEvent in newEvents.Items)
							{
								_incomingMetadata.Enqueue(newEvent);
							}
							_minChange = newMinChangeCopy;
						}
					}
				}
				else
				{
					//////////////
					// Builds
					//////////////
					List<BadgeData> builds = await RestApi.GetAsync<List<BadgeData>>($"{_apiUrl}/api/build?project={_project}&lastbuildid={_latestIds.LastBuildId}", cancellationToken);
					foreach (BadgeData build in builds)
					{
						_incomingBadges.Enqueue(build);
						_latestIds.LastBuildId = Math.Max(_latestIds.LastBuildId, build.Id);
					}

					//////////////////////////
					// Throttled Requests
					//////////////////////////
					if (fireThrottledRequests)
					{
						//////////////
						// Reviews 
						//////////////
						List<EventData> events = await RestApi.GetAsync<List<EventData>>($"{_apiUrl}/api/event?project={_project}&lasteventid={_latestIds.LastEventId}", cancellationToken);
						foreach (EventData review in events)
						{
							_incomingEvents.Enqueue(review);
							_latestIds.LastEventId = Math.Max(_latestIds.LastEventId, review.Id);
						}

						//////////////
						// Comments 
						//////////////
						List<CommentData> comments = await RestApi.GetAsync<List<CommentData>>($"{_apiUrl}/api/comment?project={_project}&lastcommentid={_latestIds.LastCommentId}", cancellationToken);
						foreach (CommentData comment in comments)
						{
							_incomingComments.Enqueue(comment);
							_latestIds.LastCommentId = Math.Max(_latestIds.LastCommentId, comment.Id);
						}
					}
				}

				LastStatusMessage = String.Format("Last update took {0}ms", timer.ElapsedMilliseconds);
				_logger.LogInformation("Done in {Time}ms.", timer.ElapsedMilliseconds);
				return true;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed with exception.");
				LastStatusMessage = String.Format("Last update failed: ({0})", ex.ToString());
				return false;
			}
		}

		public void PostEvent(int changeNumber, EventType type)
		{
			if (_apiUrl != null)
			{
				EventData evt = new EventData();
				evt.Id = ++_latestIds.LastEventId;
				evt.Change = changeNumber;
				evt.UserName = _currentUserName;
				evt.Type = type;
				evt.Project = _project;
				_outgoingEvents.Enqueue(evt);

				ApplyEventUpdate(evt);

				_refreshEvent.Set();
			}
		}

		public void PostComment(int changeNumber, string text)
		{
			if (_apiUrl != null)
			{
				CommentData comment = new CommentData();
				comment.Id = Int64.MaxValue;
				comment.ChangeNumber = changeNumber;
				comment.UserName = _currentUserName;
				comment.Text = text;
				comment.Project = _project;
				_outgoingComments.Enqueue(comment);

				ApplyCommentUpdate(comment);

				_refreshEvent.Set();
			}
		}

		public bool GetCommentByCurrentUser(int changeNumber, [NotNullWhen(true)] out string? commentText)
		{
			EventSummary? summary = GetSummaryForChange(changeNumber);
			if (summary == null)
			{
				commentText = null;
				return false;
			}

			CommentData? comment = summary.Comments.Find(x => String.Equals(x.UserName, _currentUserName, StringComparison.OrdinalIgnoreCase));
			if (comment == null || String.IsNullOrWhiteSpace(comment.Text))
			{
				commentText = null;
				return false;
			}

			commentText = comment.Text;
			return true;
		}

		public EventData? GetReviewByCurrentUser(int changeNumber)
		{
			EventSummary? summary = GetSummaryForChange(changeNumber);
			if (summary == null)
			{
				return null;
			}

			EventData? evt = summary.Reviews.FirstOrDefault(x => String.Equals(x.UserName, _currentUserName, StringComparison.OrdinalIgnoreCase));
			if (evt == null || evt.Type == EventType.Unknown)
			{
				return null;
			}

			return evt;
		}

		public EventSummary? GetSummaryForChange(int changeNumber)
		{
			EventSummary? summary;
			_changeNumberToSummary.TryGetValue(changeNumber, out summary);
			return summary;
		}

		public bool TryGetLatestBadge(string buildType, [NotNullWhen(true)] out BadgeData? badgeData)
		{
			return _badgeNameToLatestData.TryGetValue(buildType, out badgeData);
		}

		public static bool IsReview(EventType type)
		{
			return IsPositiveReview(type) || IsNegativeReview(type) || type == EventType.Unknown;
		}

		public static bool IsPositiveReview(EventType type)
		{
			return type == EventType.Good || type == EventType.Compiles;
		}

		public static bool IsNegativeReview(EventType type)
		{
			return type == EventType.DoesNotCompile || type == EventType.Bad;
		}

		public bool WasSyncedByCurrentUser(int changeNumber)
		{
			EventSummary? summary = GetSummaryForChange(changeNumber);
			return (summary != null && summary.SyncEvents.Any(x => x.Type == EventType.Syncing && String.Equals(x.UserName, _currentUserName, StringComparison.OrdinalIgnoreCase)));
		}

		public void StartInvestigating(int changeNumber)
		{
			PostEvent(changeNumber, EventType.Investigating);
		}

		public void FinishInvestigating(int changeNumber)
		{
			PostEvent(changeNumber, EventType.Resolved);
		}

		protected void UpdateActiveInvestigations()
		{
			if (_activeInvestigations == null)
			{
				// Collected investigation events are sorted by descending CL#
				// Use the comparison logic below to only keep the last such event for each user
				_activeInvestigations = new SortedSet<EventData>(
					Comparer<EventData>.Create((first, second) =>
						first.UserName.Equals(second.UserName, StringComparison.OrdinalIgnoreCase)
							? 0
							: first.Change.CompareTo(second.Change)));
				foreach (EventData investigationEvent in _investigationEvents)
				{
					_activeInvestigations.Add(investigationEvent);
				}

				// Only keep investigation start events
				_activeInvestigations.RemoveWhere(e => e.Type == EventType.Resolved);
			}
		}

		public bool IsUnderInvestigation(int changeNumber)
		{
			UpdateActiveInvestigations();
			return _activeInvestigations?.Any(x => x.Change <= changeNumber) ?? false;
		}

		public bool IsUnderInvestigationByCurrentUser(int changeNumber)
		{
			UpdateActiveInvestigations();
			return _activeInvestigations?.Any(x => x.Change <= changeNumber && String.Equals(x.UserName, _currentUserName, StringComparison.OrdinalIgnoreCase)) ?? false;
		}

		public IEnumerable<string> GetInvestigatingUsers(int changeNumber)
		{
			UpdateActiveInvestigations();
			return _activeInvestigations?.Where(x => changeNumber >= x.Change).Select(x => x.UserName) ?? Array.Empty<string>();
		}

		public int GetInvestigationStartChangeNumber(int lastChangeNumber)
		{
			UpdateActiveInvestigations();

			int startChangeNumber = -1;
			if (_activeInvestigations != null)
			{
				foreach (EventData activeInvestigation in _activeInvestigations)
				{
					if (String.Equals(activeInvestigation.UserName, _currentUserName, StringComparison.OrdinalIgnoreCase))
					{
						if (activeInvestigation.Change <= lastChangeNumber && (startChangeNumber == -1 || activeInvestigation.Change < startChangeNumber))
						{
							startChangeNumber = activeInvestigation.Change;
						}
					}
				}
			}
			return startChangeNumber;
		}
	}
}
