// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	class FolderToClean
	{
		public DirectoryInfo _directory;
		public Dictionary<string, FolderToClean> _nameToSubFolder = new Dictionary<string, FolderToClean>(StringComparer.InvariantCultureIgnoreCase);
		public Dictionary<string, FileInfo> _nameToFile = new Dictionary<string, FileInfo>(StringComparer.InvariantCultureIgnoreCase);
		public List<FileInfo> _filesToDelete = new List<FileInfo>();
		public List<FileInfo> _filesToSync = new List<FileInfo>();
		public bool _emptyLeaf = false;
		public bool _emptyAfterClean = true;

		public FolderToClean(DirectoryInfo inDirectory)
		{
			_directory = inDirectory;
		}

		public string Name => _directory.Name;

		public override string ToString() => _directory.FullName;
	}

	class FindFoldersToCleanTask : IDisposable
	{
		class PerforceHaveFolder
		{
			public Dictionary<string, PerforceHaveFolder> _nameToSubFolder = new Dictionary<string, PerforceHaveFolder>(StringComparer.OrdinalIgnoreCase);
			public Dictionary<string, FStatRecord> _nameToFile = new Dictionary<string, FStatRecord>(StringComparer.OrdinalIgnoreCase);
		}

		readonly IPerforceSettings _perforceSettings;
		readonly string _clientRootPath;
		readonly IReadOnlyList<string> _syncPaths;
		readonly ILogger _logger;
		readonly FolderToClean _rootFolderToClean;

		int _remainingFoldersToScan;
		ManualResetEvent _finishedScan = new ManualResetEvent(false);
		bool _abortScan;
		string? _scanError;

		public List<string> FileNames = new List<string>();

		public FindFoldersToCleanTask(IPerforceSettings inPerforceSettings, FolderToClean inRootFolderToClean, string inClientRootPath, IReadOnlyList<string> inSyncPaths, ILogger logger)
		{
			_perforceSettings = inPerforceSettings;
			_clientRootPath = inClientRootPath.TrimEnd('/') + "/";
			_syncPaths = new List<string>(inSyncPaths);
			_logger = logger;
			_rootFolderToClean = inRootFolderToClean;
			_finishedScan = new ManualResetEvent(true);
		}

		void QueueFolderToPopulate(FolderToClean folder)
		{
			if (Interlocked.Increment(ref _remainingFoldersToScan) == 1)
			{
				_finishedScan.Reset();
			}
			ThreadPool.QueueUserWorkItem(x => PopulateFolder(folder));
		}

		void PopulateFolder(FolderToClean folder)
		{
			if (!_abortScan)
			{
				try
				{
					if ((folder._directory.Attributes & FileAttributes.ReparsePoint) == 0)
					{
						foreach (DirectoryInfo subDirectory in folder._directory.EnumerateDirectories())
						{
							FolderToClean subFolder = new FolderToClean(subDirectory);
							folder._nameToSubFolder[subFolder.Name] = subFolder;
							QueueFolderToPopulate(subFolder);
						}
						foreach (FileInfo file in folder._directory.EnumerateFiles())
						{
							FileAttributes attributes = file.Attributes; // Force the value to be cached.
							folder._nameToFile[file.Name] = file;
						}
					}
				}
				catch (Exception ex)
				{
					string newError = String.Format("Unable to enumerate contents of {0} due to an error:\n\n{1}", folder._directory.FullName, ex);
					Interlocked.CompareExchange(ref _scanError, newError, null);
					_abortScan = true;
				}
			}

			if (Interlocked.Decrement(ref _remainingFoldersToScan) == 0)
			{
				_finishedScan.Set();
			}
		}

		static void MergeTrees(FolderToClean localFolder, PerforceHaveFolder? perforceFolder, HashSet<string> openClientPaths, string? perforceConfigFile)
		{
			if (perforceFolder == null)
			{
				// Loop through all the local sub-folders
				foreach (FolderToClean localSubFolder in localFolder._nameToSubFolder.Values)
				{
					MergeTrees(localSubFolder, null, openClientPaths, perforceConfigFile);
				}

				// Delete everything
				localFolder._filesToDelete.AddRange(localFolder._nameToFile.Values);
			}
			else
			{
				// Loop through all the local sub-folders
				foreach (FolderToClean localSubFolder in localFolder._nameToSubFolder.Values)
				{
					PerforceHaveFolder? perforceSubFolder;
					perforceFolder._nameToSubFolder.TryGetValue(localSubFolder.Name, out perforceSubFolder);
					MergeTrees(localSubFolder, perforceSubFolder, openClientPaths, perforceConfigFile);
				}

				// Also merge all the Perforce folders that no longer exist
				foreach (KeyValuePair<string, PerforceHaveFolder> perforceSubFolderPair in perforceFolder._nameToSubFolder)
				{
					FolderToClean? localSubFolder;
					if (!localFolder._nameToSubFolder.TryGetValue(perforceSubFolderPair.Key, out localSubFolder))
					{
						localSubFolder = new FolderToClean(new DirectoryInfo(Path.Combine(localFolder._directory.FullName, perforceSubFolderPair.Key)));
						MergeTrees(localSubFolder, perforceSubFolderPair.Value, openClientPaths, perforceConfigFile);
						localFolder._nameToSubFolder.Add(localSubFolder.Name, localSubFolder);
					}
				}

				// Find all the files that need to be re-synced
				foreach (KeyValuePair<string, FStatRecord> filePair in perforceFolder._nameToFile)
				{
					FileInfo? localFile;
					if (!localFolder._nameToFile.TryGetValue(filePair.Key, out localFile))
					{
						localFolder._filesToSync.Add(new FileInfo(Path.Combine(localFolder._directory.FullName, filePair.Key)));
					}
					else if (!IsFileTypeWritable(filePair.Value?.HeadType ?? filePair.Value?.Type) && (localFile.Attributes & FileAttributes.ReadOnly) == 0 && !openClientPaths.Contains(filePair.Value?.ClientFile ?? ""))
					{
						localFolder._filesToSync.Add(localFile);
					}
				}

				// Find all the files that should be deleted
				foreach (FileInfo localFileInfo in localFolder._nameToFile.Values)
				{
					if (!perforceFolder._nameToFile.ContainsKey(localFileInfo.Name) && !openClientPaths.Contains(localFileInfo.FullName))
					{
						localFolder._filesToDelete.Add(localFileInfo);
					}
				}
			}

			// Remove any config files
			if (perforceConfigFile != null)
			{
				localFolder._filesToDelete.RemoveAll(x => String.Equals(x.Name, perforceConfigFile, StringComparison.OrdinalIgnoreCase));
			}

			// Figure out if this folder is just an empty directory that needs to be removed
			localFolder._emptyLeaf = localFolder._nameToFile.Count == 0 && localFolder._nameToSubFolder.Count == 0 && localFolder._filesToSync.Count == 0;

			// Figure out if it the folder will be empty after the clean operation
			localFolder._emptyAfterClean = localFolder._nameToSubFolder.Values.All(x => x._emptyAfterClean) && localFolder._filesToDelete.Count == localFolder._nameToFile.Count && localFolder._filesToSync.Count == 0;
		}

		static bool IsFileTypeWritable(string? type)
		{
			if (type != null)
			{
				int idx = type.IndexOf('+', StringComparison.Ordinal);
				if (idx != -1)
				{
					return type.IndexOf('w', idx) != -1;
				}
			}
			return false;
		}

		static void RemoveEmptyFolders(FolderToClean folder)
		{
			foreach (FolderToClean subFolder in folder._nameToSubFolder.Values)
			{
				RemoveEmptyFolders(subFolder);
			}

			folder._nameToSubFolder = folder._nameToSubFolder.Values.Where(x => x._nameToSubFolder.Count > 0 || x._filesToSync.Count > 0 || x._filesToDelete.Count > 0 || x._emptyLeaf).ToDictionary(x => x.Name, x => x, StringComparer.InvariantCultureIgnoreCase);
		}

		public void Dispose()
		{
			_abortScan = true;

			if (_finishedScan != null)
			{
				_finishedScan.WaitOne();
				_finishedScan.Dispose();
				_finishedScan = null!;
			}
		}

		public async Task RunAsync(CancellationToken cancellationToken)
		{
			using IPerforceConnection perforceClient = await PerforceConnection.CreateAsync(_perforceSettings, _logger);

			_logger.LogInformation("Finding files in workspace...");
			_logger.LogInformation("");

			// Clear the current error
			_scanError = null;

			// Start enumerating all the files that exist locally
			foreach (string syncPath in _syncPaths)
			{
				Debug.Assert(syncPath.StartsWith(_clientRootPath, StringComparison.OrdinalIgnoreCase));
				if (syncPath.StartsWith(_clientRootPath, StringComparison.OrdinalIgnoreCase))
				{
					string[] fragments = syncPath.Substring(_clientRootPath.Length).Split('/');

					FolderToClean syncFolder = _rootFolderToClean;
					for (int idx = 0; idx < fragments.Length - 1; idx++)
					{
						FolderToClean? nextSyncFolder;
						if (!syncFolder._nameToSubFolder.TryGetValue(fragments[idx], out nextSyncFolder))
						{
							nextSyncFolder = new FolderToClean(new DirectoryInfo(Path.Combine(syncFolder._directory.FullName, fragments[idx])));
							syncFolder._nameToSubFolder[nextSyncFolder.Name] = nextSyncFolder;
						}
						syncFolder = nextSyncFolder;
					}

					string wildcard = fragments[^1];
					if (wildcard == "...")
					{
						QueueFolderToPopulate(syncFolder);
					}
					else
					{
						if (syncFolder._directory.Exists)
						{
							foreach (FileInfo file in syncFolder._directory.EnumerateFiles(wildcard))
							{
								syncFolder._nameToFile[file.Name] = file;
							}
						}
					}
				}
			}

			// Get the prefix for any local file
			string localRootPrefix = _rootFolderToClean._directory.FullName.Replace(Path.AltDirectorySeparatorChar, Path.DirectorySeparatorChar);
			if (!localRootPrefix.EndsWith(Path.DirectorySeparatorChar))
			{
				localRootPrefix += Path.DirectorySeparatorChar;
			}

			// Query the have table and build a separate tree from it
			PerforceHaveFolder rootHaveFolder = new PerforceHaveFolder();
			foreach (string syncPath in _syncPaths)
			{
				List<FStatRecord> fileRecords = await perforceClient.FStatAsync($"{syncPath}#have", cancellationToken).ToListAsync(cancellationToken);
				foreach (FStatRecord fileRecord in fileRecords)
				{
					if (fileRecord.ClientFile == null || !fileRecord.ClientFile.StartsWith(localRootPrefix, StringComparison.InvariantCultureIgnoreCase))
					{
						throw new UserErrorException($"Failed to get have table; file '{fileRecord.ClientFile}' doesn't start with root path ('{_rootFolderToClean._directory.FullName}')");
					}

					string[] tokens = fileRecord.ClientFile.Substring(localRootPrefix.Length).Split('/', '\\');

					PerforceHaveFolder fileFolder = rootHaveFolder;
					for (int idx = 0; idx < tokens.Length - 1; idx++)
					{
						PerforceHaveFolder? nextFileFolder;
						if (!fileFolder._nameToSubFolder.TryGetValue(tokens[idx], out nextFileFolder))
						{
							nextFileFolder = new PerforceHaveFolder();
							fileFolder._nameToSubFolder.Add(tokens[idx], nextFileFolder);
						}
						fileFolder = nextFileFolder;
					}
					fileFolder._nameToFile[tokens[^1]] = fileRecord;
				}
			}

			// Find all the files which are currently open for edit. We don't want to force sync these.
			List<OpenedRecord> openFileRecords = await perforceClient.OpenedAsync(OpenedOptions.None, -1, null, null, -1, "//...", cancellationToken).ToListAsync(cancellationToken);

			// Build a set of all the open local files
			HashSet<string> openLocalFiles = new HashSet<string>(StringComparer.InvariantCultureIgnoreCase);
			foreach (OpenedRecord openFileRecord in openFileRecords)
			{
				if (!openFileRecord.ClientFile.StartsWith(_clientRootPath, StringComparison.InvariantCultureIgnoreCase))
				{
					throw new UserErrorException($"Failed to get open file list; file '{openFileRecord.ClientFile}' doesn't start with client root path ('{_clientRootPath}')");
				}
				openLocalFiles.Add(localRootPrefix + PerforceUtils.UnescapePath(openFileRecord.ClientFile).Substring(_clientRootPath.Length).Replace('/', Path.DirectorySeparatorChar));
			}

			// Wait to finish scanning the directory
			_finishedScan.WaitOne();

			// Check if there was an error
			if (_scanError != null)
			{
				throw new UserErrorException(_scanError);
			}

			// Find the value of the P4CONFIG variable
			string? perforceConfigFile = PerforceEnvironment.Default.GetValue("P4CONFIG");

			// Merge the trees
			MergeTrees(_rootFolderToClean, rootHaveFolder, openLocalFiles, perforceConfigFile);

			// Remove all the empty folders
			RemoveEmptyFolders(_rootFolderToClean);
		}
	}
}
