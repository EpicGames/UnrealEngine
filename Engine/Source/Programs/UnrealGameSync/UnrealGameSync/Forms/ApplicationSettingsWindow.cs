// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;

namespace UnrealGameSync
{
	partial class ApplicationSettingsWindow : ThemedForm
	{
		public enum Result
		{
			Cancel,
			Ok,
			Quit,
			Restart,
			RestartAndConfigureUpdate,
		}

		static class PerforceTestConnectionTask
		{
			public static async Task RunAsync(IPerforceConnection perforce, string depotPath, CancellationToken cancellationToken)
			{
				string checkFilePath = String.Format("{0}/Release/UnrealGameSync.exe", depotPath);

				List<FStatRecord> fileRecords = await perforce.FStatAsync(checkFilePath, cancellationToken).ToListAsync(cancellationToken);
				if (fileRecords.Count == 0)
				{
					throw new UserErrorException($"Unable to find {checkFilePath}");
				}
			}
		}

		readonly string _originalExecutableFileName;
		readonly UserSettings _settings;
		readonly ILogger _logger;

		readonly int _initialAutomationPortNumber;
		readonly ProtocolHandlerState _initialProtocolHandlerState;

		readonly ToolUpdateMonitor _toolUpdateMonitor;
		readonly Tuple<bool, string> _toolUpdateStatus;

		Result _result = Result.Ok;

		class ToolItem
		{
			public Guid Id => Definition.Id;

			public int Index { get; }
			public ToolInfo Definition { get; }
			public List<ToolItem> RequiresTools { get; } = new List<ToolItem>();

			public bool Enabled { get; set; }
			public int DependencyRefCount { get; set; }

			public ToolItem(int index, ToolInfo definition, bool enabled)
			{
				Index = index;
				Definition = definition;
				Enabled = enabled;
			}

			public CheckState GetCheckState()
			{
				if (Enabled)
				{
					return CheckState.Checked;
				}
				else if (DependencyRefCount > 0)
				{
					return CheckState.Indeterminate;
				}
				else
				{
					return CheckState.Unchecked;
				}
			}

			public override string ToString()
			{
				return Definition.Description;
			}
		}

		private ApplicationSettingsWindow(IPerforceSettings defaultPerforceSettings, string originalExecutableFileName, UserSettings settings, ToolUpdateMonitor toolUpdateMonitor, ILogger<ApplicationSettingsWindow> logger)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_originalExecutableFileName = originalExecutableFileName;
			_settings = settings;
			_toolUpdateMonitor = toolUpdateMonitor;
			_toolUpdateStatus = toolUpdateMonitor.LastStatus;
			_logger = logger;

			LauncherSettings launcherSettings = new LauncherSettings();
			launcherSettings.Read();

			_initialAutomationPortNumber = AutomationServer.GetPortNumber();
			_initialProtocolHandlerState = ProtocolHandlerUtils.GetState();

			AutomaticallyRunAtStartupCheckBox.Checked = IsAutomaticallyRunAtStartup();
			KeepInTrayCheckBox.Checked = settings.KeepInTray;

			foreach (KeyValuePair<string, string> themeIdNamePair in settings.ThemeList)
			{
				ThemeComboBox.Items.Add(themeIdNamePair);

				if (_settings.Theme?.ThemeId == themeIdNamePair.Key)
				{
					ThemeComboBox.SelectedItem = themeIdNamePair;
				}
			}

			HordeServerTextBox.Text = launcherSettings.HordeServer;
			HordeServerTextBox.Select(HordeServerTextBox.TextLength, 0);
			HordeServerTextBox.CueBanner = launcherSettings.HordeServer ?? String.Empty;

			ServerTextBox.Text = launcherSettings.PerforceServerAndPort;
			ServerTextBox.Select(ServerTextBox.TextLength, 0);
			ServerTextBox.CueBanner = $"Default ({defaultPerforceSettings.ServerAndPort})";

			UserNameTextBox.Text = launcherSettings.PerforceUserName;
			UserNameTextBox.Select(UserNameTextBox.TextLength, 0);
			UserNameTextBox.CueBanner = $"Default ({defaultPerforceSettings.UserName})";

			ParallelSyncThreadsSpinner.Value = Math.Max(Math.Min(settings.SyncOptions.NumThreads ?? PerforceSyncOptions.DefaultNumThreads, ParallelSyncThreadsSpinner.Maximum), ParallelSyncThreadsSpinner.Minimum);

			if (_initialAutomationPortNumber > 0)
			{
				EnableAutomationCheckBox.Checked = true;
				AutomationPortTextBox.Enabled = true;
				AutomationPortTextBox.Text = _initialAutomationPortNumber.ToString();
			}
			else
			{
				EnableAutomationCheckBox.Checked = false;
				AutomationPortTextBox.Enabled = false;
				AutomationPortTextBox.Text = AutomationServer.DefaultPortNumber.ToString();
			}

			if (_initialProtocolHandlerState == ProtocolHandlerState.Installed)
			{
				EnableProtocolHandlerCheckBox.CheckState = CheckState.Checked;
			}
			else if (_initialProtocolHandlerState == ProtocolHandlerState.NotInstalled)
			{
				EnableProtocolHandlerCheckBox.CheckState = CheckState.Unchecked;
			}
			else
			{
				EnableProtocolHandlerCheckBox.CheckState = CheckState.Indeterminate;
			}

			if (_toolUpdateStatus.Item1)
			{
				List<ToolItem> toolItems = new List<ToolItem>();
				Dictionary<Guid, ToolItem> idToToolItem = new Dictionary<Guid, ToolItem>();
				foreach (ToolInfo tool in toolUpdateMonitor.GetTools().OrderBy(x => x.Name))
				{
					ToolItem toolItem = new ToolItem(toolItems.Count, tool, settings.EnabledTools.Contains(tool.Id));
					idToToolItem[tool.Id] = toolItem;
					toolItems.Add(toolItem);
				}

				HashSet<ToolItem> dependsOnToolItems = new HashSet<ToolItem>();
				foreach (ToolItem toolItem in toolItems)
				{
					dependsOnToolItems.Clear();
					FindDependencies(toolItem, dependsOnToolItems, idToToolItem);
					toolItem.RequiresTools.AddRange(dependsOnToolItems);

					if (toolItem.Enabled)
					{
						foreach (ToolItem requiredTool in toolItem.RequiresTools)
						{
							requiredTool.DependencyRefCount++;
						}
					}
				}

				foreach (ToolItem toolItem in toolItems)
				{
					CustomToolsListBox.Items.Add(toolItem, toolItem.GetCheckState());
				}

				CustomToolsListBox.Enabled = true;
			}
			else
			{
				string message = Regex.Replace(_toolUpdateStatus.Item2, "\n.*$", "...");
				CustomToolsListBox.Items.Add(message);
				CustomToolsListBox.Enabled = _toolUpdateStatus.Item1;
			}
		}

		static void FindDependencies(ToolItem toolItem, HashSet<ToolItem> dependsOnToolItems, Dictionary<Guid, ToolItem> idToToolItem)
		{
			foreach (Guid dependsOnToolId in toolItem.Definition.DependsOnToolIds)
			{
				ToolItem? dependsOnToolItem;
				if (idToToolItem.TryGetValue(dependsOnToolId, out dependsOnToolItem) && dependsOnToolItems.Add(dependsOnToolItem))
				{
					FindDependencies(dependsOnToolItem, dependsOnToolItems, idToToolItem);
				}
			}
		}

		public static Result ShowModal(IWin32Window owner, IPerforceSettings defaultPerforceSettings, string originalExecutableFileName, UserSettings settings, ToolUpdateMonitor toolUpdateMonitor, ILogger<ApplicationSettingsWindow> logger)
		{
			using ApplicationSettingsWindow applicationSettings = new ApplicationSettingsWindow(defaultPerforceSettings, originalExecutableFileName, settings, toolUpdateMonitor, logger);
			applicationSettings.ShowDialog(owner);
			return applicationSettings._result;
		}

		private static bool IsAutomaticallyRunAtStartup()
		{
			RegistryKey? key = Registry.CurrentUser.OpenSubKey("SOFTWARE\\Microsoft\\Windows\\CurrentVersion\\Run");
			return (key?.GetValue("UnrealGameSync") != null);
		}

		private void UpdateSettingsBtn_Click(object sender, EventArgs e)
		{
			if (Path.GetFileName(_originalExecutableFileName).Contains("Launcher", StringComparison.OrdinalIgnoreCase))
			{
				if (MessageBox.Show("To configure update settings, quit UnrealGameSync and relaunch from the start menu while holding down the shift key.\n\nWould you like to quit now?", "Restart Required", MessageBoxButtons.YesNo) == DialogResult.Yes)
				{
					ApplySettings(Result.Quit);
				}
			}
			else
			{
				if (MessageBox.Show("UnrealGameSync must be restarted to configure update settings.\n\nWould you like to restart now?", "Restart Required", MessageBoxButtons.OKCancel) == DialogResult.OK)
				{
					ApplySettings(Result.RestartAndConfigureUpdate);
				}
			}
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			ApplySettings(Result.Ok);
		}

		private void ApplySettings(Result result)
		{
			LauncherSettings originalLauncherSettings = new LauncherSettings();
			originalLauncherSettings.Read();

			// Update the settings
			LauncherSettings launcherSettings = new LauncherSettings(originalLauncherSettings);

			launcherSettings.HordeServer = HordeServerTextBox.Text.Trim();
			if (launcherSettings.HordeServer.Length == 0)
			{
				launcherSettings.HordeServer = null;
			}

			launcherSettings.PerforceServerAndPort = ServerTextBox.Text.Trim();
			if (launcherSettings.PerforceServerAndPort.Length == 0)
			{
				launcherSettings.PerforceServerAndPort = null;
			}

			launcherSettings.PerforceUserName = UserNameTextBox.Text.Trim();
			if (launcherSettings.PerforceUserName.Length == 0)
			{
				launcherSettings.PerforceUserName = null;
			}

			int automationPortNumber;
			if (!EnableAutomationCheckBox.Checked || !Int32.TryParse(AutomationPortTextBox.Text, out automationPortNumber))
			{
				automationPortNumber = -1;
			}

			string themeId = (ThemeComboBox.SelectedItem != null) ? ((KeyValuePair<string, string>)ThemeComboBox.SelectedItem).Key : "";
			bool didThemeChange = (_settings.Theme == null || _settings.Theme.ThemeId != themeId);
			
			if (!String.Equals(launcherSettings.HordeServer, originalLauncherSettings.HordeServer, StringComparison.OrdinalIgnoreCase) ||
				!String.Equals(launcherSettings.PerforceServerAndPort, originalLauncherSettings.PerforceServerAndPort, StringComparison.OrdinalIgnoreCase) ||
				!String.Equals(launcherSettings.PerforceUserName, originalLauncherSettings.PerforceUserName, StringComparison.OrdinalIgnoreCase) ||
				automationPortNumber != _initialAutomationPortNumber ||
				didThemeChange)
			{
				if (result == Result.Ok)
				{
					if (MessageBox.Show("UnrealGameSync must be restarted to apply these settings.\n\nWould you like to restart now?", "Restart Required", MessageBoxButtons.OKCancel) != DialogResult.OK)
					{
						return;
					}
					else
					{
						result = Result.Restart;
					}
				}

				launcherSettings.Save();
				AutomationServer.SetPortNumber(automationPortNumber);
			}

			RegistryKey key = Registry.CurrentUser.CreateSubKey("SOFTWARE\\Microsoft\\Windows\\CurrentVersion\\Run");
			if (AutomaticallyRunAtStartupCheckBox.Checked)
			{
				key.SetValue("UnrealGameSync", String.Format("\"{0}\" -RestoreState", _originalExecutableFileName));
			}
			else
			{
				key.DeleteValue("UnrealGameSync", false);
			}

			if (_settings.KeepInTray != KeepInTrayCheckBox.Checked || _settings.SyncOptions.NumThreads != ParallelSyncThreadsSpinner.Value)
			{
				int numThreads = (int)ParallelSyncThreadsSpinner.Value;
				_settings.SyncOptions.NumThreads = (numThreads != PerforceSyncOptions.DefaultNumThreads) ? (int?)numThreads : null;
				_settings.KeepInTray = KeepInTrayCheckBox.Checked;
				_settings.Save(_logger);
			}

			if (didThemeChange)
			{
				_settings.Theme = _settings.LoadTheme(themeId);
				_settings.Save(_logger);
			}

			if (_toolUpdateStatus.Item1)
			{
				List<Guid> newEnabledTools = new List<Guid>();
				foreach (ToolItem? item in CustomToolsListBox.Items)
				{
					if (item != null && item.Enabled)
					{
						newEnabledTools.Add(item.Definition.Id);
					}
				}
				if (!newEnabledTools.SequenceEqual(_settings.EnabledTools))
				{
					_settings.EnabledTools.Clear();
					_settings.EnabledTools.UnionWith(newEnabledTools);
					_settings.Save(_logger);
					_toolUpdateMonitor.UpdateNow();
				}
			}

			if (EnableProtocolHandlerCheckBox.CheckState == CheckState.Checked)
			{
				if (_initialProtocolHandlerState != ProtocolHandlerState.Installed)
				{
					ProtocolHandlerUtils.Install();
				}
			}
			else if (EnableProtocolHandlerCheckBox.CheckState == CheckState.Unchecked)
			{
				if (_initialProtocolHandlerState != ProtocolHandlerState.NotInstalled)
				{
					ProtocolHandlerUtils.Uninstall();
				}
			}

			_result = result;
			Close();
		}

		private void CancelBtn_Click(object sender, EventArgs e)
		{
			_result = Result.Cancel;
			Close();
		}

		private void EnableAutomationCheckBox_CheckedChanged(object sender, EventArgs e)
		{
			AutomationPortTextBox.Enabled = EnableAutomationCheckBox.Checked;
		}

		private void AdvancedBtn_Click(object sender, EventArgs e)
		{
			using PerforceSyncSettingsWindow window = new PerforceSyncSettingsWindow(_settings, _logger);
			window.ShowDialog();
		}

		bool _recursiveItemCheck = false;

		private void CustomToolsListBox_ItemCheck(object sender, ItemCheckEventArgs e)
		{
			if (!_recursiveItemCheck)
			{
				_recursiveItemCheck = true;
				ToolItem toolItem = (ToolItem)CustomToolsListBox.Items[e.Index];

				bool newEnabled = (e.CurrentValue == CheckState.Unchecked || e.CurrentValue == CheckState.Indeterminate);
				if (newEnabled != toolItem.Enabled)
				{
					toolItem.Enabled = newEnabled;

					foreach (ToolItem requiredTool in toolItem.RequiresTools)
					{
						if (toolItem.Enabled)
						{
							requiredTool.DependencyRefCount++;
						}
						else
						{
							requiredTool.DependencyRefCount--;
						}

						CheckState newCheckState = requiredTool.GetCheckState();
						if (newCheckState != CustomToolsListBox.GetItemCheckState(requiredTool.Index))
						{
							CustomToolsListBox.SetItemCheckState(requiredTool.Index, newCheckState);
						}
					}
				}

				e.NewValue = toolItem.GetCheckState();
				_recursiveItemCheck = false;
			}
		}
	}
}
