// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Drawing;
using System.Runtime.InteropServices;
using System.Windows.Forms;
using System.Windows.Forms.VisualStyles;

namespace UnrealGameSync
{
	public partial class ArgumentsWindow : ThemedForm
	{
		const int LvmFirst = 0x1000;
		const int LvmEditlabelw = (LvmFirst + 118);

		const int EmSetsel = 0xb1;

		[DllImport("user32.dll", CharSet = CharSet.Auto)]
		static extern IntPtr SendMessage(IntPtr hWnd, uint msg, IntPtr wParam, IntPtr lParam);

		[DllImport("user32.dll")]
		static extern bool SetWindowText(IntPtr hWnd, string text);

		private readonly List<LockableEditorArgument> _defaultEditorArguments = new List<LockableEditorArgument>();

		public ArgumentsWindow(List<LockableEditorArgument> arguments, List<LockableEditorArgument> defaultArguments, bool promptBeforeLaunch)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			ActiveControl = ArgumentsList;

			ArgumentsList.Font = SystemFonts.IconTitleFont;

			ArgumentsList.Items.Clear();

			// Keep a local copy of the default arguments
			foreach (LockableEditorArgument defaultArgument in defaultArguments)
			{
				_defaultEditorArguments.Add(defaultArgument);
			}

			// Add any arguments that are in the default list first
			foreach (LockableEditorArgument defaultArgument in _defaultEditorArguments)
			{
				int index = arguments.FindIndex(x => x.Name == defaultArgument.Name);

				if (index != -1)
				{
					ListViewItem item = new ListViewItem(arguments[index].Name);
					item.Checked = arguments[index].Enabled;
					ArgumentsList.Items.Add(item);
				}
			}

			// Add any user arguments that are not defaults second
			foreach (LockableEditorArgument argument in arguments)
			{
				int index = _defaultEditorArguments.FindIndex(x => x.Name == argument.Name);

				if (index == -1)
				{
					ListViewItem item = new ListViewItem(argument.Name);
					item.Checked = argument.Enabled;
					ArgumentsList.Items.Add(item);
				}
			}

			ListViewItem addAnotherItem = new ListViewItem("Click to add an item...", 0);
			ArgumentsList.Items.Add(addAnotherItem);

			PromptBeforeLaunchCheckBox.Checked = promptBeforeLaunch;
		}

		public bool PromptBeforeLaunch => PromptBeforeLaunchCheckBox.Checked;

		public List<LockableEditorArgument> GetItems()
		{
			List<LockableEditorArgument> items = new List<LockableEditorArgument>();
			for (int idx = 0; idx < ArgumentsList.Items.Count - 1; idx++)
			{
				ListViewItem item = ArgumentsList.Items[idx];
				items.Add(new LockableEditorArgument(item.Text, item.Checked));
			}
			return items;
		}

		private void ClearButton_Click(object sender, EventArgs e)
		{
			//			ArgumentsTextBox.Text = "";
			DialogResult = System.Windows.Forms.DialogResult.OK;
			Close();
		}

		private void ArgumentsList_AfterLabelEdit(object sender, LabelEditEventArgs e)
		{
			// don't allow default editor arguments to have their text modified
			bool isItemDefaultArgument = (GetDefaultArgumentForItem(e.Item) != null);
			if (isItemDefaultArgument)
			{
				e.CancelEdit = true;
				return;
			}

			if ((e.Label == null && ArgumentsList.Items[e.Item].Text.Length == 0) || (e.Label != null && e.Label.Trim().Length == 0))
			{
				e.CancelEdit = true;
				ArgumentsList.Items.RemoveAt(e.Item);
			}
		}

		private void ArgumentsList_BeforeLabelEdit(object sender, LabelEditEventArgs e)
		{
			// don't allow default editor arguments to have their text modified
			bool isItemDefaultArgument = (GetDefaultArgumentForItem(e.Item) != null);
			if (isItemDefaultArgument)
			{
				e.CancelEdit = true;
				return;
			}

			if (e.Item == ArgumentsList.Items.Count - 1)
			{
				e.CancelEdit = true;
			}
		}

		private void ArgumentsList_MouseClick(object sender, MouseEventArgs e)
		{
			ListViewHitTestInfo info = ArgumentsList.HitTest(e.Location);
			if (info.Item == null)
			{
				return;
			}
			else if (info.Item.Index == ArgumentsList.Items.Count - 1)
			{
				ListViewItem newItem = new ListViewItem();
				newItem.Checked = true;
				newItem = ArgumentsList.Items.Insert(ArgumentsList.Items.Count - 1, newItem);
				newItem.BeginEdit();
			}
			else
			{
				// don't allow default editor arguments to have their text modified
				bool isItemDefaultArgument = (GetDefaultArgumentForItem(info.Item) != null);
				if (isItemDefaultArgument)
				{
					return;
				}

				using (Graphics graphics = ArgumentsList.CreateGraphics())
				{
					int labelOffset = e.X - CheckBoxPadding - CheckBoxRenderer.GetGlyphSize(graphics, CheckBoxState.CheckedNormal).Width - CheckBoxPadding;
					if (labelOffset >= 0 && labelOffset < TextRenderer.MeasureText(info.Item.Text, ArgumentsList.Font).Width)
					{
						info.Item.BeginEdit();
					}
				}
			}
		}

		const int CheckBoxPadding = 5;

		private void ArgumentsList_DrawItem(object sender, DrawListViewItemEventArgs e)
		{
			e.DrawBackground();
			if (e.ItemIndex < ArgumentsList.Items.Count - 1)
			{
				CheckBoxState state = e.Item.Checked ? CheckBoxState.CheckedNormal : CheckBoxState.UncheckedNormal;
				Size checkSize = CheckBoxRenderer.GetGlyphSize(e.Graphics, state);
				CheckBoxRenderer.DrawCheckBox(e.Graphics, new Point(e.Bounds.Left + 4, e.Bounds.Top + (e.Bounds.Height - checkSize.Height) / 2), state);
				DrawItemLabel(e.Graphics, Theme.TextColor, e.Item);
			}
			else
			{
				DrawItemLabel(e.Graphics, Theme.DimmedText, e.Item);
			}
		}

		private void DrawItemLabel(Graphics graphics, Color normalColor, ListViewItem item)
		{
			string extraInfo = GetDefaultArgumentDebugText(item);

			Rectangle labelRect = GetLabelRectangle(graphics, item);
			if (item.Selected)
			{
				graphics.FillRectangle(SystemBrushes.Highlight, labelRect);
				TextRenderer.DrawText(graphics, item.Text + extraInfo, ArgumentsList.Font, labelRect, SystemColors.HighlightText, SystemColors.Highlight, TextFormatFlags.Left | TextFormatFlags.VerticalCenter | TextFormatFlags.SingleLine);
			}
			else
			{
				TextRenderer.DrawText(graphics, item.Text + extraInfo, ArgumentsList.Font, labelRect, normalColor, TextFormatFlags.Left | TextFormatFlags.VerticalCenter | TextFormatFlags.SingleLine);
			}
		}

		private Rectangle GetLabelRectangle(Graphics graphics, ListViewItem item)
		{
			string extraInfo = GetDefaultArgumentDebugText(item);

			CheckBoxState state = item.Checked ? CheckBoxState.CheckedNormal : CheckBoxState.UncheckedNormal;
			Size checkSize = CheckBoxRenderer.GetGlyphSize(graphics, state);
			Size labelSize = TextRenderer.MeasureText(item.Text + extraInfo, ArgumentsList.Font);

			int labelIndent = CheckBoxPadding + checkSize.Width + CheckBoxPadding;
			return new Rectangle(item.Bounds.Left + labelIndent, item.Bounds.Top, labelSize.Width, item.Bounds.Height);
		}

		private void ArgumentsList_KeyUp(object sender, KeyEventArgs e)
		{
			if (e.KeyCode == Keys.Delete && ArgumentsList.SelectedIndices.Count == 1)
			{
				int index = ArgumentsList.SelectedIndices[0];

				// don't allow default editor arguments to be deleted
				bool isItemDefaultArgument = (GetDefaultArgumentForItem(index) != null);
				if (isItemDefaultArgument)
				{
					return;
				}

				ArgumentsList.Items.RemoveAt(index);
				if (index < ArgumentsList.Items.Count - 1)
				{
					ArgumentsList.Items[index].Selected = true;
				}
			}
		}

		private void ArgumentsList_KeyPress(object sender, KeyPressEventArgs e)
		{
			if (ArgumentsList.SelectedItems.Count == 1 && !Char.IsControl(e.KeyChar))
			{
				ListViewItem item = ArgumentsList.SelectedItems[0];

				// don't allow default editor arguments to have their text modified
				bool isItemDefaultArgument = (GetDefaultArgumentForItem(item) != null);
				if (isItemDefaultArgument)
				{
					return;
				}

				if (item.Index == ArgumentsList.Items.Count - 1)
				{
					item = new ListViewItem();
					item.Checked = true;
					item = ArgumentsList.Items.Insert(ArgumentsList.Items.Count - 1, item);
				}

				IntPtr newHandle = SendMessage(ArgumentsList.Handle, LvmEditlabelw, new IntPtr(item.Index), IntPtr.Zero);
				SetWindowText(newHandle, e.KeyChar.ToString());
				SendMessage(newHandle, EmSetsel, new IntPtr(1), new IntPtr(1));

				e.Handled = true;
			}
		}

		private void MoveUpButton_Click(object sender, EventArgs e)
		{
			if (ArgumentsList.SelectedIndices.Count == 1)
			{
				int index = ArgumentsList.SelectedIndices[0];
				if (index > 0)
				{
					ListViewItem item = ArgumentsList.Items[index];
					ArgumentsList.Items.RemoveAt(index);
					ArgumentsList.Items.Insert(index - 1, item);
				}
			}
		}

		private void MoveDownButton_Click(object sender, EventArgs e)
		{
			if (ArgumentsList.SelectedIndices.Count == 1)
			{
				int index = ArgumentsList.SelectedIndices[0];
				if (index < ArgumentsList.Items.Count - 2)
				{
					ListViewItem item = ArgumentsList.Items[index];
					ArgumentsList.Items.RemoveAt(index);
					ArgumentsList.Items.Insert(index + 1, item);
				}
			}
		}

		private void ResetToDefaultsButton_Click(object sender, EventArgs e)
		{
			foreach (ListViewItem item in ArgumentsList.Items)
			{
				LockableEditorArgument? defaultArgument = GetDefaultArgumentForItem(item);

				if (defaultArgument != null)
				{
					item.Checked = defaultArgument.Enabled;
				}
				else
				{
					item.Checked = false;
				}
			}
		}

		private void ArgumentsList_SelectedIndexChanged(object sender, EventArgs e)
		{
			UpdateMoveButtons();
		}

		private void UpdateMoveButtons()
		{
			MoveUpButton.Enabled = (ArgumentsList.SelectedIndices.Count == 1 && ArgumentsList.SelectedIndices[0] > 0);
			MoveDownButton.Enabled = (ArgumentsList.SelectedIndices.Count == 1 && ArgumentsList.SelectedIndices[0] < ArgumentsList.Items.Count - 2);
			ResetToDefaultsButton.Enabled = true;
		}

		private LockableEditorArgument? GetDefaultArgumentForItem(ListViewItem item)
		{
			int defaultArgumentindex = _defaultEditorArguments.FindIndex(x => x.Name == item.Text);

			if (defaultArgumentindex != -1)
			{
				return _defaultEditorArguments[defaultArgumentindex];
			}

			return null;
		}

		private LockableEditorArgument? GetDefaultArgumentForItem(int index)
		{
			if (index < 0 || index >= ArgumentsList.Items.Count)
			{
				return null;
			}

			return GetDefaultArgumentForItem(ArgumentsList.Items[index]);
		}

		private string GetDefaultArgumentDebugText(ListViewItem item)
		{
			string extraInfo = "";

			LockableEditorArgument? defaultArgument = GetDefaultArgumentForItem(item);

			if (defaultArgument != null)
			{
				extraInfo = extraInfo + "    Default";

				if (defaultArgument.Locked)
				{
					extraInfo = extraInfo + ": Locked";
				}
				else if (item.Checked != defaultArgument.Enabled)
				{
					extraInfo = extraInfo + ": Modified";
				}
			}

			return extraInfo;
		}

		private void ArgumentsList_ItemCheck(object? sender, ItemCheckEventArgs e)
		{
			LockableEditorArgument? defaultArgument = GetDefaultArgumentForItem(e.Index);

			if ((defaultArgument != null) && defaultArgument.Locked)
			{ 
				e.NewValue = defaultArgument.Enabled ? CheckState.Checked : CheckState.Unchecked;
			}
		}
	}
}
