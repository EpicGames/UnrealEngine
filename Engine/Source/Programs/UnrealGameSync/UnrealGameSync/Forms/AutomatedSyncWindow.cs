// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.Diagnostics.CodeAnalysis;
using System.Drawing;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace UnrealGameSync
{
	partial class AutomatedSyncWindow : ThemedForm
	{
		static class FindDefaultWorkspaceTask
		{
			public static async Task<string?> RunAsync(IPerforceConnection perforce, string streamName, CancellationToken cancellationToken)
			{
				InfoRecord info = await perforce.GetInfoAsync(InfoOptions.ShortOutput, cancellationToken);

				List<ClientsRecord> clients = await perforce.GetClientsAsync(ClientsOptions.None, perforce.Settings.UserName, cancellationToken);

				List<ClientsRecord> candidateClients = new List<ClientsRecord>();
				foreach (ClientsRecord client in clients)
				{
					if (client.Host == null || client.Host.Equals(info.ClientHost, StringComparison.OrdinalIgnoreCase))
					{
						if (client.Stream != null && client.Stream.Equals(streamName, StringComparison.OrdinalIgnoreCase))
						{
							candidateClients.Add(client);
						}
					}
				}

				string? workspaceName = null;
				if (candidateClients.Count >= 1)
				{
					workspaceName = candidateClients.OrderByDescending(x => x.Access).First().Name;
				}
				return workspaceName;
			}
		}

		class ValidateWorkspaceTask
		{
			public string WorkspaceName { get; }
			public string StreamName { get; }
			public bool RequiresStreamSwitch { get; private set; }
			public bool HasOpenFiles { get; private set; }

			public ValidateWorkspaceTask(string workspaceName, string streamName)
			{
				WorkspaceName = workspaceName;
				StreamName = streamName;
			}

			public async Task RunAsync(IPerforceConnection perforce, CancellationToken cancellationToken)
			{
				ClientRecord spec = await perforce.GetClientAsync(WorkspaceName, cancellationToken);

				string? currentStreamName = spec.Stream;
				if (currentStreamName == null || currentStreamName != StreamName)
				{
					RequiresStreamSwitch = true;

					List<PerforceResponse<OpenedRecord>> records = await perforce.TryOpenedAsync(OpenedOptions.None, FileSpecList.Any, cancellationToken).ToListAsync(cancellationToken);
					HasOpenFiles = records.Succeeded() && records.Count > 0;
				}
			}
		}

		public class WorkspaceInfo
		{
			public string ServerAndPort { get; }
			public string UserName { get; }
			public string WorkspaceName { get; }
			public bool RequiresStreamSwitch { get; }

			public WorkspaceInfo(string serverAndPort, string userName, string workspaceName, bool requiresStreamSwitch)
			{
				ServerAndPort = serverAndPort;
				UserName = userName;
				WorkspaceName = workspaceName;
				RequiresStreamSwitch = requiresStreamSwitch;
			}
		}

		readonly string _streamName;
		readonly IServiceProvider _serviceProvider;

		string? _serverAndPortOverride;
		string? _userNameOverride;
		readonly IPerforceSettings _defaultPerforceSettings;
		readonly UserSettings _settings;
		WorkspaceInfo? _selectedWorkspaceInfo;

		private AutomatedSyncWindow(string streamName, string projectPath, string? workspaceName, IPerforceSettings defaultPerforceSettings, UserSettings settings, IServiceProvider serviceProvider)
		{
			_streamName = streamName;
			_defaultPerforceSettings = defaultPerforceSettings;
			_serviceProvider = serviceProvider;
			_settings = settings;

			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			ActiveControl = WorkspaceNameTextBox;

			MinimumSize = Size;
			MaximumSize = new Size(32768, Size.Height);

			ProjectTextBox.Text = streamName + projectPath;

			if (workspaceName != null)
			{
				WorkspaceNameTextBox.Text = workspaceName;
				WorkspaceNameTextBox.Select(WorkspaceNameTextBox.Text.Length, 0);
			}

			UpdateServerLabel();
			UpdateOkButton();
		}

		private IPerforceSettings Perforce => Utility.OverridePerforceSettings(_defaultPerforceSettings, _serverAndPortOverride, _userNameOverride);

		public static string? FindDefaultWorkspace(IWin32Window owner, IPerforceSettings defaultPerforceSettings, string streamName, IServiceProvider serviceProvider)
		{
			ILogger logger = serviceProvider.GetRequiredService<ILogger<AutomatedSyncWindow>>();
			ModalTask<string?>? workspaceTask = PerforceModalTask.Execute(owner, "Finding workspace", "Finding default workspace, please wait...", defaultPerforceSettings, (p, c) => FindDefaultWorkspaceTask.RunAsync(p, streamName, c), logger);
			return (workspaceTask != null && workspaceTask.Succeeded) ? workspaceTask.Result : null;
		}

		public static bool ShowModal(IWin32Window owner, IPerforceSettings defaultPerforceSettings, string streamName, string projectPath, UserSettings settings, [NotNullWhen(true)] out WorkspaceInfo? workspaceInfo, IServiceProvider serviceProvider)
		{
			string? workspaceName = FindDefaultWorkspace(owner, defaultPerforceSettings, streamName, serviceProvider);

			using AutomatedSyncWindow window = new AutomatedSyncWindow(streamName, projectPath, workspaceName, defaultPerforceSettings, settings, serviceProvider);
			if (window.ShowDialog() == DialogResult.OK)
			{
				workspaceInfo = window._selectedWorkspaceInfo!;
				return true;
			}
			else
			{
				workspaceInfo = null;
				return false;
			}
		}

		private void ChangeLink_LinkClicked(object sender, LinkLabelLinkClickedEventArgs e)
		{
			if (ConnectWindow.ShowModal(this, _defaultPerforceSettings, ref _serverAndPortOverride, ref _userNameOverride, _serviceProvider))
			{
				UpdateServerLabel();
			}
		}

		private void UpdateServerLabel()
		{
			ServerLabel.Text = OpenProjectWindow.GetServerLabelText(_defaultPerforceSettings, _serverAndPortOverride, _userNameOverride);
			ChangeLink.Location = new Point(ServerLabel.Right + 5, ChangeLink.Location.Y);
		}

		private void WorkspaceNameNewBtn_Click(object sender, EventArgs e)
		{
			string? workspaceName;
			if (NewWorkspaceWindow.ShowModal(this, _defaultPerforceSettings, _streamName, WorkspaceNameTextBox.Text, _settings, _serviceProvider, out workspaceName))
			{
				WorkspaceNameTextBox.Text = workspaceName;
				UpdateOkButton();
			}
		}

		private void WorkspaceNameBrowseBtn_Click(object sender, EventArgs e)
		{
			string? workspaceName = WorkspaceNameTextBox.Text;
			if (SelectWorkspaceWindow.ShowModal(this, _defaultPerforceSettings, workspaceName, _serviceProvider, out workspaceName))
			{
				WorkspaceNameTextBox.Text = workspaceName;
				UpdateOkButton();
			}
		}

		private void UpdateOkButton()
		{
			OkBtn.Enabled = (WorkspaceNameTextBox.Text.Length > 0);
		}

		public static bool ValidateWorkspace(IWin32Window owner, IPerforceSettings perforce, string workspaceName, string streamName, IServiceProvider serviceProvider, [NotNullWhen(true)] out WorkspaceInfo? selectedWorkspaceInfo)
		{
			ValidateWorkspaceTask validateWorkspace = new ValidateWorkspaceTask(workspaceName, streamName);

			ModalTask? task = PerforceModalTask.Execute(owner, "Checking workspace", "Checking workspace, please wait...", perforce, validateWorkspace.RunAsync, serviceProvider.GetRequiredService<ILogger<ValidateWorkspaceTask>>());
			if (task != null && task.Succeeded)
			{
				if (validateWorkspace.RequiresStreamSwitch)
				{
					string message;
					if (validateWorkspace.HasOpenFiles)
					{
						message = String.Format("You have files open for edit in this workspace. If you switch this workspace to {0}, you will not be able to submit them until you switch back.\n\nContinue switching streams?", streamName);
					}
					else
					{
						message = String.Format("Switch this workspace to {0}?", streamName);
					}
					if (MessageBox.Show(message, "Switch Streams", MessageBoxButtons.YesNo) != DialogResult.Yes)
					{
						selectedWorkspaceInfo = null;
						return false;
					}
				}

				selectedWorkspaceInfo = new WorkspaceInfo(perforce.ServerAndPort, perforce.UserName, validateWorkspace.WorkspaceName, validateWorkspace.RequiresStreamSwitch);
				return true;
			}

			selectedWorkspaceInfo = null;
			return false;
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			if (ValidateWorkspace(this, Perforce, WorkspaceNameTextBox.Text, _streamName, _serviceProvider, out _selectedWorkspaceInfo))
			{
				DialogResult = DialogResult.OK;
				Close();
			}
		}
	}
}
