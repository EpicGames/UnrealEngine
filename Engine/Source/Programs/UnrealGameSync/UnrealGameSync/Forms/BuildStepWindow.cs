// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Drawing;
using System.IO;
using System.Linq;
using System.Windows.Forms;
using EpicGames.Core;

namespace UnrealGameSync
{
	partial class BuildStepWindow : ThemedForm
	{
		readonly BuildStep _step;
		readonly List<string> _targetNames;
		readonly DirectoryReference _baseDirectory;
		readonly IReadOnlyDictionary<string, string> _variables;
#pragma warning disable CA2213 // warning CA2213: 'BuildStepWindow' contains field '_variablesWindow' that is of IDisposable type 'VariablesWindow?', but it is never disposed. Change the Dispose method on 'BuildStepWindow' to call Close or Dispose on this field.
		VariablesWindow? _variablesWindow;
#pragma warning restore CA2213

		public BuildStepWindow(BuildStep inTask, List<string> inTargetNames, DirectoryReference inBaseDirectory, IReadOnlyDictionary<string, string> inVariables)
		{
			_step = inTask;
			_targetNames = inTargetNames;
			_baseDirectory = inBaseDirectory;
			_variables = inVariables;

			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			MaximumSize = new System.Drawing.Size(32768, Height);
			MinimumSize = new System.Drawing.Size(Width, Height);
		}

		private void BuildTaskWindow_Load(object sender, EventArgs e)
		{
			string defaultTargetName = _targetNames.FirstOrDefault(x => !x.EndsWith("Editor", StringComparison.OrdinalIgnoreCase) && !x.EndsWith("Client", StringComparison.OrdinalIgnoreCase) && !x.EndsWith("Server", StringComparison.OrdinalIgnoreCase)) ?? ((_targetNames.Count > 0) ? _targetNames[0] : "");
			CompileTargetComboBox.Items.AddRange(_targetNames.ToArray());
			CompileTargetComboBox.Text = String.IsNullOrEmpty(_step.Target) ? defaultTargetName : _step.Target;
			StatusPanelLinkTextBox.Text = _step.StatusPanelLink;

			DescriptionTextBox.Text = _step.Description;
			StatusTextTextBox.Text = _step.StatusText;
			DurationTextBox.Text = _step.EstimatedDuration.ToString();

			switch (_step.Type)
			{
				case BuildStepType.Compile:
					CompileRadioButton.Checked = true;
					CompilePlatformComboBox.Text = _step.Platform;
					CompileConfigComboBox.Text = _step.Configuration;
					CompileArgumentsTextBox.Text = _step.Arguments;
					break;
				case BuildStepType.Cook:
					CookRadioButton.Checked = true;
					CookFileNameTextBox.Text = _step.FileName;
					break;
				case BuildStepType.Other:
					OtherRadioButton.Checked = true;
					OtherFileNameTextBox.Text = _step.FileName;
					OtherArgumentsTextBox.Text = _step.Arguments;
					OtherUseLogWindowCheckBox.Checked = _step.UseLogWindow;
					OtherWorkingDirTextBox.Text = _step.WorkingDir;
					break;
			}

			if (String.IsNullOrEmpty(CompilePlatformComboBox.Text))
			{
				CompilePlatformComboBox.Text = "Win64";
			}

			if (String.IsNullOrEmpty(CompileConfigComboBox.Text))
			{
				CompileConfigComboBox.Text = "Development";
			}

			UpdateType();
		}

		public void UpdateType()
		{
			bool isCompile = CompileRadioButton.Checked;
			CompileTargetComboBox.Enabled = isCompile;
			CompilePlatformComboBox.Enabled = isCompile;
			CompileConfigComboBox.Enabled = isCompile;
			CompileArgumentsTextBox.Enabled = isCompile;

			bool isCook = CookRadioButton.Checked;
			CookFileNameTextBox.Enabled = isCook;
			CookFileNameButton.Enabled = isCook;

			bool isOther = OtherRadioButton.Checked;
			OtherFileNameTextBox.Enabled = isOther;
			OtherFileNameButton.Enabled = isOther;
			OtherWorkingDirTextBox.Enabled = isOther;
			OtherWorkingDirButton.Enabled = isOther;
			OtherArgumentsTextBox.Enabled = isOther;
			OtherUseLogWindowCheckBox.ReadOnly = !isOther;
		}

		private void CompileRadioButton_CheckedChanged(object sender, EventArgs e)
		{
			UpdateType();
		}

		private void CookRadioButton_CheckedChanged(object sender, EventArgs e)
		{
			UpdateType();
		}

		private void OtherRadioButton_CheckedChanged(object sender, EventArgs e)
		{
			UpdateType();
		}

		private void OkButton_Click(object sender, EventArgs e)
		{
			_step.Description = DescriptionTextBox.Text;
			_step.StatusText = StatusTextTextBox.Text;
			_step.StatusPanelLink = StatusPanelLinkTextBox.Text;

			int estimatedDuration;
			if (!Int32.TryParse(DurationTextBox.Text, out estimatedDuration))
			{
				estimatedDuration = 1;
			}
			_step.EstimatedDuration = estimatedDuration;

			if (CompileRadioButton.Checked)
			{
				_step.Type = BuildStepType.Compile;
				_step.Target = CompileTargetComboBox.Text;
				_step.Platform = CompilePlatformComboBox.Text;
				_step.Configuration = CompileConfigComboBox.Text;
				_step.FileName = null;
				_step.Arguments = CompileArgumentsTextBox.Text;
				_step.UseLogWindow = true;
			}
			else if (CookRadioButton.Checked)
			{
				_step.Type = BuildStepType.Cook;
				_step.Target = null;
				_step.Platform = null;
				_step.Configuration = null;
				_step.FileName = CookFileNameTextBox.Text;
				_step.Arguments = null;
				_step.UseLogWindow = true;
			}
			else
			{
				_step.Type = BuildStepType.Other;
				_step.Target = null;
				_step.Platform = null;
				_step.Configuration = null;
				_step.FileName = OtherFileNameTextBox.Text;
				_step.WorkingDir = OtherWorkingDirTextBox.Text;
				_step.Arguments = OtherArgumentsTextBox.Text;
				_step.UseLogWindow = OtherUseLogWindowCheckBox.Checked;
			}

			DialogResult = DialogResult.OK;
			Close();
		}

		private void NewCancelButton_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
			Close();
		}

		private void CookFileNameButton_Click(object sender, EventArgs e)
		{
			using OpenFileDialog dialog = new OpenFileDialog();
			dialog.Filter = "Cook/Launch Profiles (*.ulp2)|*.ulp2";
			dialog.FileName = AddBaseDirectory(CookFileNameTextBox.Text);
			if (dialog.ShowDialog() == DialogResult.OK)
			{
				CookFileNameTextBox.Text = RemoveBaseDirectory(dialog.FileName);
			}
		}

		private void OtherFileNameButton_Click(object sender, EventArgs e)
		{
			using OpenFileDialog dialog = new OpenFileDialog();
			dialog.Filter = "Executable Files (*.exe,*.bat)|*.exe;*.bat|All files (*.*)|*.*";
			dialog.FileName = AddBaseDirectory(OtherFileNameTextBox.Text);
			if (dialog.ShowDialog() == DialogResult.OK)
			{
				OtherFileNameTextBox.Text = RemoveBaseDirectory(dialog.FileName);
			}
		}

		private void OtherWorkingDirButton_Click(object sender, EventArgs e)
		{
			using FolderBrowserDialog dialog = new FolderBrowserDialog();
			dialog.SelectedPath = OtherWorkingDirTextBox.Text;
			if (dialog.ShowDialog() == DialogResult.OK)
			{
				OtherWorkingDirTextBox.Text = dialog.SelectedPath;
			}
		}

		private string AddBaseDirectory(string fileName)
		{
			if (fileName.Contains("$(", StringComparison.Ordinal))
			{
				return "";
			}
			else if (Path.IsPathRooted(fileName))
			{
				return fileName;
			}
			else
			{
				return FileReference.Combine(_baseDirectory, fileName).FullName;
			}
		}

		private string RemoveBaseDirectory(string fileName)
		{
			FileReference fullFileName = new FileReference(fileName);
			if (fullFileName.IsUnderDirectory(_baseDirectory))
			{
				return fullFileName.MakeRelativeTo(_baseDirectory);
			}
			else
			{
				return fileName;
			}
		}

		private void OnClosedVariablesWindow(object? sender, EventArgs e)
		{
			_variablesWindow = null;
		}

		private void InsertVariable(string name)
		{
			IContainerControl container = this;
			for (; ; )
			{
				IContainerControl? nextContainer = container.ActiveControl as IContainerControl;
				if (nextContainer == null)
				{
					break;
				}
				container = nextContainer;
			}

			TextBox? focusTextBox = container.ActiveControl as TextBox;
			if (focusTextBox != null)
			{
				focusTextBox.SelectedText = name;
			}
		}

		private void VariablesButton_Click(object sender, EventArgs e)
		{
			if (_variablesWindow == null)
			{
				_variablesWindow = new VariablesWindow(_variables);
				_variablesWindow.OnInsertVariable += InsertVariable;
				_variablesWindow.Location = new Point(Bounds.Right + 20, Bounds.Top);
				_variablesWindow.Size = new Size(_variablesWindow.Size.Width, Size.Height);
				_variablesWindow.FormClosed += OnClosedVariablesWindow;
				_variablesWindow.Show(this);

				components ??= new System.ComponentModel.Container();
				components.Add(_variablesWindow);
			}
			else
			{
				_variablesWindow.Close();
				_variablesWindow = null;
			}
		}
	}
}
