// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Windows.Forms;

namespace UnrealGameSync
{
	public partial class ChangelistWindow : ThemedForm
	{
		public int ChangeNumber { get; private set; }

		public ChangelistWindow(int changeNumber)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			MaximumSize = new System.Drawing.Size(32768, Height);
			MinimumSize = new System.Drawing.Size(Width, Height);

			ChangeNumber = changeNumber;
			ChangeNumberTextBox.Text = (ChangeNumber > 0) ? ChangeNumber.ToString() : "";
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			int newChangeNumber;
			if (Int32.TryParse(ChangeNumberTextBox.Text.Trim(), out newChangeNumber))
			{
				int deltaChangeNumber = newChangeNumber - ChangeNumber;
				if ((deltaChangeNumber > -10000 && deltaChangeNumber < 1000) || ChangeNumber <= 0 || MessageBox.Show(String.Format("Changelist {0} is {1} changes away from your current sync. Was that intended?", newChangeNumber, Math.Abs(deltaChangeNumber)), "Changelist is far away", MessageBoxButtons.YesNo) == DialogResult.Yes)
				{
					ChangeNumber = newChangeNumber;
					DialogResult = DialogResult.OK;
					Close();
				}
			}
		}

		private void CancelBtn_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
			Close();
		}
	}
}
