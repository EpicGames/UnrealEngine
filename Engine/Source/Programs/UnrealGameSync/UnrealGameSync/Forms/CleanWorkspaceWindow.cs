// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.Diagnostics;
using System.Drawing;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Windows.Forms;
using System.Windows.Forms.VisualStyles;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	partial class CleanWorkspaceWindow : ThemedForm
	{
		enum TreeNodeAction
		{
			Sync,
			Delete,
		}

		class TreeNodeData
		{
			public TreeNodeAction _action;
			public FileInfo? _file;
			public FolderToClean? _folder;
			public int _numFiles;
			public int _numSelectedFiles;
			public int _numEmptySelectedFiles;
			public int _numMissingSelectedFiles;
			public int _numDefaultSelectedFiles;
		}

		enum SelectionType
		{
			All,
			SafeToDelete,
			Missing,
			Empty,
			None,
		}

		static readonly CheckBoxState[] s_checkBoxStates =
		{
			CheckBoxState.UncheckedNormal,
			CheckBoxState.MixedNormal,
			CheckBoxState.CheckedNormal
		};

		readonly IPerforceSettings _perforceSettings;
		readonly FolderToClean _rootFolderToClean;

		static readonly string[] s_safeToDeleteFolders =
		{
			"/binaries/",
			"/intermediate/",
			"/build/receipts/",
			"/.vs/",
			"/automationtool/saved/rules/",
			"/saved/logs/",

			"/bin/debug/",
			"/bin/development/",
			"/bin/release/",
			"/obj/debug/",
			"/obj/development/",
			"/obj/release/",

			"/bin/x86/debug/",
			"/bin/x86/development/",
			"/bin/x86/release/",
			"/obj/x86/debug/",
			"/obj/x86/development/",
			"/obj/x86/release/",

			"/bin/x64/debug/",
			"/bin/x64/development/",
			"/bin/x64/release/",
			"/obj/x64/debug/",
			"/obj/x64/development/",
			"/obj/x64/release/",
		};

		static readonly string[] s_safeToDeleteExtensions =
		{
			".pdb",
			".obj",
			".sdf",
			".suo",
			".sln",
			".csproj.user",
			".csproj.references",
		};

		readonly IReadOnlyList<string> _extraSafeToDeleteFolders;
		readonly IReadOnlyList<string> _extraSafeToDeleteExtensions;
		readonly ILogger _logger;

		[DllImport("Shell32.dll", EntryPoint = "ExtractIconExW", CharSet = CharSet.Unicode, ExactSpelling = true, CallingConvention = CallingConvention.StdCall)]
		private static extern int ExtractIconEx(string sFile, int iIndex, IntPtr piLargeVersion, out IntPtr piSmallVersion, int amountIcons);

		private CleanWorkspaceWindow(IPerforceSettings perforceSettings, FolderToClean rootFolderToClean, string[] extraSafeToDeleteFolders, string[] extraSafeToDeleteExtensions, ILogger<CleanWorkspaceWindow> logger)
		{
			_perforceSettings = perforceSettings;
			_rootFolderToClean = rootFolderToClean;
			_extraSafeToDeleteFolders = extraSafeToDeleteFolders.Select(x => x.Trim().Replace('\\', '/').Trim('/')).Where(x => x.Length > 0).Select(x => String.Format("/{0}/", x.ToLowerInvariant())).ToArray();
			_extraSafeToDeleteExtensions = extraSafeToDeleteExtensions.Select(x => x.Trim().ToLowerInvariant()).Where(x => x.Length > 0).ToArray();
			_logger = logger;

			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));
		}

		public static bool DoClean(IWin32Window owner, IPerforceSettings perforceSettings, DirectoryReference localRootPath, string clientRootPath, IReadOnlyList<string> syncPaths, string[] extraSafeToDeleteFolders, string[] extraSafeToDeleteExtensions, ILogger<CleanWorkspaceWindow> logger)
		{
			// Figure out which folders to clean
			FolderToClean rootFolderToClean = new FolderToClean(localRootPath.ToDirectoryInfo());
			using (FindFoldersToCleanTask queryWorkspace = new FindFoldersToCleanTask(perforceSettings, rootFolderToClean, clientRootPath, syncPaths, logger))
			{
				ModalTask? result = ModalTask.Execute(owner, "Clean Workspace", "Querying files in Perforce, please wait...", x => queryWorkspace.RunAsync(x), ModalTaskFlags.None);
				if (result == null || !result.Succeeded)
				{
					return false;
				}
			}

			// If there's nothing to delete, don't bother displaying the dialog at all
			if (rootFolderToClean._filesToDelete.Count == 0 && rootFolderToClean._nameToSubFolder.Count == 0)
			{
				MessageBox.Show("You have no local files which are not in Perforce.", "Clean Workspace", MessageBoxButtons.OK);
				return false;
			}

			// Populate the tree
			using CleanWorkspaceWindow cleanWorkspace = new CleanWorkspaceWindow(perforceSettings, rootFolderToClean, extraSafeToDeleteFolders, extraSafeToDeleteExtensions, logger);
			DialogResult dialogResult = cleanWorkspace.ShowDialog();

			return (dialogResult == DialogResult.OK);
		}

		private void CleanWorkspaceWindow_Load(object sender, EventArgs e)
		{
			IntPtr folderIconPtr;
			ExtractIconEx("imageres.dll", 3, IntPtr.Zero, out folderIconPtr, 1);

			IntPtr fileIconPtr;
			ExtractIconEx("imageres.dll", 2, IntPtr.Zero, out fileIconPtr, 1);

			Icon[] icons = new Icon[] { Icon.FromHandle(folderIconPtr), Icon.FromHandle(fileIconPtr) };

			Size largestIconSize = Size.Empty;
			foreach (Icon icon in icons)
			{
				largestIconSize = new Size(Math.Max(largestIconSize.Width, icon.Width), Math.Max(largestIconSize.Height, icon.Height));
			}

			Size largestCheckBoxSize = Size.Empty;
			using (Graphics graphics = Graphics.FromHwnd(IntPtr.Zero))
			{
				foreach (CheckBoxState state in s_checkBoxStates)
				{
					Size checkBoxSize = CheckBoxRenderer.GetGlyphSize(graphics, state);
					largestCheckBoxSize = new Size(Math.Max(largestCheckBoxSize.Width, checkBoxSize.Width), Math.Max(largestCheckBoxSize.Height, checkBoxSize.Height));
				}
			}

			Size imageSize = new Size(largestCheckBoxSize.Width + largestIconSize.Width, Math.Max(largestIconSize.Height, largestCheckBoxSize.Height));

			Bitmap typeImageListBitmap = new Bitmap(icons.Length * 3 * imageSize.Width, imageSize.Height, System.Drawing.Imaging.PixelFormat.Format32bppArgb);
			using (Graphics graphics = Graphics.FromImage(typeImageListBitmap))
			{
				int minX = 0;
				for (int iconIdx = 0; iconIdx < icons.Length; iconIdx++)
				{
					for (int stateIdx = 0; stateIdx < 3; stateIdx++)
					{
						Size checkBoxSize = CheckBoxRenderer.GetGlyphSize(graphics, s_checkBoxStates[stateIdx]);
						CheckBoxRenderer.DrawCheckBox(graphics, new Point(minX + (largestCheckBoxSize.Width - checkBoxSize.Width) / 2, (largestCheckBoxSize.Height - checkBoxSize.Height) / 2), s_checkBoxStates[stateIdx]);

						Size iconSize = icons[iconIdx].Size;
						graphics.DrawIcon(icons[iconIdx], minX + largestCheckBoxSize.Width + (largestIconSize.Width - iconSize.Width) / 2, (largestIconSize.Height - iconSize.Height) / 2);

						minX += imageSize.Width;
					}
				}
			}

			ImageList typeImageList = new ImageList();
			typeImageList.ImageSize = imageSize;
			typeImageList.ColorDepth = ColorDepth.Depth32Bit;
			typeImageList.Images.AddStrip(typeImageListBitmap);
			TreeView.ImageList = typeImageList;

			TreeNode node = BuildTreeViewStructure(_rootFolderToClean, "/", false, 0);
			node.Text = _rootFolderToClean._directory.FullName;
			TreeView.Nodes.Add(node);
		}

		private TreeNode BuildTreeViewStructure(FolderToClean folder, string folderPath, bool parentFolderSelected, int depth)
		{
			bool selectFolder = parentFolderSelected || IsSafeToDeleteFolder(folderPath) || folder._emptyLeaf;

			TreeNodeData folderNodeData = new TreeNodeData();
			folderNodeData._folder = folder;

			TreeNode folderNode = new TreeNode();
			folderNode.Text = folder.Name;
			folderNode.Tag = folderNodeData;

			foreach (FolderToClean subFolder in folder._nameToSubFolder.OrderBy(x => x.Key).Select(x => x.Value))
			{
				TreeNode childNode = BuildTreeViewStructure(subFolder, folderPath + subFolder.Name.ToLowerInvariant() + "/", selectFolder, depth + 1);
				folderNode.Nodes.Add(childNode);

				TreeNodeData childNodeData = childNode.Tag as TreeNodeData ?? throw new InvalidOperationException("Missing node data");
				folderNodeData._numFiles += childNodeData._numFiles;
				folderNodeData._numSelectedFiles += childNodeData._numSelectedFiles;
				folderNodeData._numEmptySelectedFiles += childNodeData._numEmptySelectedFiles;
				folderNodeData._numMissingSelectedFiles += childNodeData._numMissingSelectedFiles;
				folderNodeData._numDefaultSelectedFiles += childNodeData._numDefaultSelectedFiles;
			}

			foreach (FileInfo file in folder._filesToSync.OrderBy(x => x.Name))
			{
				TreeNodeData fileNodeData = new TreeNodeData();
				fileNodeData._action = TreeNodeAction.Sync;
				fileNodeData._file = file;
				fileNodeData._numFiles = 1;
				fileNodeData._numSelectedFiles = 1;
				fileNodeData._numEmptySelectedFiles = 0;
				fileNodeData._numMissingSelectedFiles = 1;
				fileNodeData._numDefaultSelectedFiles = fileNodeData._numSelectedFiles;

				TreeNode fileNode = new TreeNode();
				fileNode.Text = file.Name + " (sync)";
				fileNode.Tag = fileNodeData;
				folderNode.Nodes.Add(fileNode);

				UpdateImage(fileNode);

				folderNodeData._numFiles++;
				folderNodeData._numSelectedFiles += fileNodeData._numSelectedFiles;
				folderNodeData._numEmptySelectedFiles += fileNodeData._numEmptySelectedFiles;
				folderNodeData._numMissingSelectedFiles += fileNodeData._numMissingSelectedFiles;
				folderNodeData._numDefaultSelectedFiles += fileNodeData._numDefaultSelectedFiles;
			}

			foreach (FileInfo file in folder._filesToDelete.OrderBy(x => x.Name))
			{
				string name = file.Name.ToLowerInvariant();

				bool selectFile = selectFolder || IsSafeToDeleteFile(file.Name.ToLowerInvariant());

				TreeNodeData fileNodeData = new TreeNodeData();
				fileNodeData._action = TreeNodeAction.Delete;
				fileNodeData._file = file;
				fileNodeData._numFiles = 1;
				fileNodeData._numSelectedFiles = selectFile ? 1 : 0;
				fileNodeData._numEmptySelectedFiles = 0;
				fileNodeData._numMissingSelectedFiles = 0;
				fileNodeData._numDefaultSelectedFiles = fileNodeData._numSelectedFiles;

				TreeNode fileNode = new TreeNode();
				fileNode.Text = file.Name;
				fileNode.Tag = fileNodeData;
				folderNode.Nodes.Add(fileNode);

				UpdateImage(fileNode);

				folderNodeData._numFiles++;
				folderNodeData._numSelectedFiles += fileNodeData._numSelectedFiles;
				folderNodeData._numEmptySelectedFiles += fileNodeData._numEmptySelectedFiles;
				folderNodeData._numMissingSelectedFiles += fileNodeData._numMissingSelectedFiles;
				folderNodeData._numDefaultSelectedFiles += fileNodeData._numDefaultSelectedFiles;
			}

			if (folderNodeData._folder._emptyLeaf)
			{
				folderNodeData._numFiles++;
				folderNodeData._numSelectedFiles++;
				folderNodeData._numEmptySelectedFiles++;
				folderNodeData._numDefaultSelectedFiles++;
			}

			if (folderNodeData._numSelectedFiles > 0 && !folderNodeData._folder._emptyAfterClean && depth < 2)
			{
				folderNode.Expand();
			}
			else
			{
				folderNode.Collapse();
			}

			UpdateImage(folderNode);
			return folderNode;
		}

		private bool IsSafeToDeleteFolder(string folderPath)
		{
			return s_safeToDeleteFolders.Any(x => folderPath.EndsWith(x, StringComparison.OrdinalIgnoreCase)) || _extraSafeToDeleteFolders.Any(x => folderPath.EndsWith(x, StringComparison.OrdinalIgnoreCase));
		}

		private bool IsSafeToDeleteFile(string name)
		{
			return s_safeToDeleteExtensions.Any(x => name.EndsWith(x, StringComparison.OrdinalIgnoreCase)) || _extraSafeToDeleteExtensions.Any(x => name.EndsWith(x, StringComparison.OrdinalIgnoreCase));
		}

		private void TreeView_DrawNode(object sender, DrawTreeNodeEventArgs e)
		{
			e.Graphics.DrawLine(Pens.Black, new Point(e.Bounds.Left, e.Bounds.Top), new Point(e.Bounds.Right, e.Bounds.Bottom));
		}

		private void TreeView_NodeMouseClick(object sender, TreeNodeMouseClickEventArgs e)
		{
			if (e.Node is not TreeNode node)
			{
				return;
			}
			if (e.Button == System.Windows.Forms.MouseButtons.Right)
			{
				TreeView.SelectedNode = node;
				FolderContextMenu.Tag = node;
				FolderContextMenu.Show(TreeView.PointToScreen(e.Location));
			}
			else if (e.X >= node.Bounds.Left - 32 && e.X < node.Bounds.Left - 16)
			{
				TreeNodeData nodeData = node.Tag as TreeNodeData ?? throw new InvalidOperationException("Missing node data");
				SetSelected(node, (nodeData._numSelectedFiles == 0) ? SelectionType.All : SelectionType.None);
			}
		}

		private static void SetSelected(TreeNode parentNode, SelectionType type)
		{
			TreeNodeData parentNodeData = parentNode.Tag as TreeNodeData ?? throw new InvalidOperationException("Missing node data");

			int prevNumSelectedFiles = parentNodeData._numSelectedFiles;
			SetSelectedOnChildren(parentNode, type);

			int deltaNumSelectedFiles = parentNodeData._numSelectedFiles - prevNumSelectedFiles;
			if (deltaNumSelectedFiles != 0)
			{
				for (TreeNode? nextParentNode = parentNode.Parent; nextParentNode != null; nextParentNode = nextParentNode.Parent)
				{
					TreeNodeData nextParentNodeData = nextParentNode.Tag as TreeNodeData ?? throw new InvalidOperationException("Missing node data");
					nextParentNodeData._numSelectedFiles += deltaNumSelectedFiles;
					UpdateImage(nextParentNode);
				}
			}
		}

		private static void SetSelectedOnChildren(TreeNode parentNode, SelectionType type)
		{
			TreeNodeData parentNodeData = parentNode.Tag as TreeNodeData ?? throw new InvalidOperationException("Missing node data");

			int newNumSelectedFiles = 0;
			switch (type)
			{
				case SelectionType.All:
					newNumSelectedFiles = parentNodeData._numFiles;
					break;
				case SelectionType.Empty:
					newNumSelectedFiles = parentNodeData._numEmptySelectedFiles;
					break;
				case SelectionType.Missing:
					newNumSelectedFiles = parentNodeData._numMissingSelectedFiles;
					break;
				case SelectionType.SafeToDelete:
					newNumSelectedFiles = parentNodeData._numDefaultSelectedFiles;
					break;
				case SelectionType.None:
					newNumSelectedFiles = 0;
					break;
			}

			if (newNumSelectedFiles != parentNodeData._numSelectedFiles)
			{
				foreach (TreeNode? childNode in parentNode.Nodes)
				{
					if (childNode != null)
					{
						SetSelectedOnChildren(childNode, type);
					}
				}
				parentNodeData._numSelectedFiles = newNumSelectedFiles;
				UpdateImage(parentNode);
			}
		}

		private static void UpdateImage(TreeNode node)
		{
			TreeNodeData nodeData = node.Tag as TreeNodeData ?? throw new InvalidOperationException("Missing node data");
			int imageIndex = (nodeData._folder != null) ? 0 : 3;
			imageIndex += (nodeData._numSelectedFiles == 0) ? 0 : (nodeData._numSelectedFiles < nodeData._numFiles || (nodeData._folder != null && !nodeData._folder._emptyAfterClean)) ? 1 : 2;
			node.ImageIndex = imageIndex;
			node.SelectedImageIndex = imageIndex;
		}

		private void CleanBtn_Click(object sender, EventArgs e)
		{
			List<FileInfo> filesToSync = new List<FileInfo>();
			List<FileInfo> filesToDelete = new List<FileInfo>();
			List<DirectoryInfo> directoriesToDelete = new List<DirectoryInfo>();
			foreach (TreeNode? rootNode in TreeView.Nodes)
			{
				if (rootNode != null)
				{
					FindSelection(rootNode, filesToSync, filesToDelete, directoriesToDelete);
				}
			}

			ModalTask? result = ModalTask.Execute(this, "Clean Workspace", "Cleaning files, please wait...", x => DeleteFilesTask.RunAsync(_perforceSettings, filesToSync, filesToDelete, directoriesToDelete, _logger, x), ModalTaskFlags.Quiet);
			if (result != null && result.Failed)
			{
				using FailedToDeleteWindow failedToDelete = new FailedToDeleteWindow();
				failedToDelete.FileList.Text = result.Error;
				failedToDelete.FileList.SelectionStart = 0;
				failedToDelete.FileList.SelectionLength = 0;
				failedToDelete.ShowDialog();
			}
		}

		private static void FindSelection(TreeNode node, List<FileInfo> filesToSync, List<FileInfo> filesToDelete, List<DirectoryInfo> directoriesToDelete)
		{
			TreeNodeData nodeData = node.Tag as TreeNodeData ?? throw new InvalidOperationException("Missing node data");
			if (nodeData._file != null)
			{
				if (nodeData._numSelectedFiles > 0)
				{
					if (nodeData._action == TreeNodeAction.Delete)
					{
						filesToDelete.Add(nodeData._file);
					}
					else
					{
						filesToSync.Add(nodeData._file);
					}
				}
			}
			else
			{
				foreach (TreeNode? childNode in node.Nodes)
				{
					if (childNode != null)
					{
						FindSelection(childNode, filesToSync, filesToDelete, directoriesToDelete);
					}
				}
				if (nodeData._folder != null && nodeData._folder._emptyAfterClean && nodeData._numSelectedFiles == nodeData._numFiles)
				{
					directoriesToDelete.Add(nodeData._folder._directory);
				}
			}
		}

		private void SelectAllBtn_Click(object sender, EventArgs e)
		{
			foreach (TreeNode? node in TreeView.Nodes)
			{
				if (node != null)
				{
					SetSelected(node, SelectionType.All);
				}
			}
		}

		private void SelectMissingBtn_Click(object sender, EventArgs e)
		{
			foreach (TreeNode? node in TreeView.Nodes)
			{
				if (node != null)
				{
					SetSelected(node, SelectionType.Missing);
				}
			}
		}

		private void FolderContextMenu_SelectAll_Click(object sender, EventArgs e)
		{
			TreeNode node = (TreeNode)FolderContextMenu.Tag!;
			SetSelected(node, SelectionType.All);
		}

		private void FolderContextMenu_SelectSafeToDelete_Click(object sender, EventArgs e)
		{
			TreeNode node = (TreeNode)FolderContextMenu.Tag!;
			SetSelected(node, SelectionType.SafeToDelete);
		}

		private void FolderContextMenu_SelectEmptyFolder_Click(object sender, EventArgs e)
		{
			TreeNode node = (TreeNode)FolderContextMenu.Tag!;
			SetSelected(node, SelectionType.Empty);
		}

		private void FolderContextMenu_SelectNone_Click(object sender, EventArgs e)
		{
			TreeNode node = (TreeNode)FolderContextMenu.Tag!;
			SetSelected(node, SelectionType.None);
		}

		private void FolderContextMenu_OpenWithExplorer_Click(object sender, EventArgs e)
		{
			TreeNode node = (TreeNode)FolderContextMenu.Tag!;
			TreeNodeData? nodeData = node.Tag as TreeNodeData;

			if (nodeData?._folder != null)
			{
				Process.Start("explorer.exe", String.Format("\"{0}\"", nodeData._folder._directory.FullName));
			}
			else if (nodeData?._file != null)
			{
				Process.Start("explorer.exe", String.Format("\"{0}\"", nodeData._file.Directory!.FullName));
			}
		}
	}
}
