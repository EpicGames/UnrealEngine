// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Drawing;
using System.IO;
using System.Windows.Forms;

namespace UnrealGameSync
{
	public partial class ClobberWindow : ThemedForm
	{
		readonly Dictionary<string, bool> _filesToClobber;

		public ClobberWindow(Dictionary<string, bool> inFilesToClobber, HashSet<string> inUncontrolledFiles)
		{
			bool uncontrolledChangeFound = false;

			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_filesToClobber = inFilesToClobber;

			foreach (KeyValuePair<string, bool> fileToClobber in _filesToClobber)
			{
				ListViewItem item = new ListViewItem(Path.GetFileName(fileToClobber.Key));
				item.Tag = fileToClobber.Key;
				item.Checked = fileToClobber.Value;
				item.SubItems.Add(Path.GetDirectoryName(fileToClobber.Key));
				FileList.Items.Add(item);

				if (inUncontrolledFiles.Contains(fileToClobber.Key.Replace("\\", "/", StringComparison.Ordinal)))
				{
					uncontrolledChangeFound = true;
					item.ForeColor = Color.Red;
				}
			}

			if (uncontrolledChangeFound)
			{
				// Updates the string to inform the user to take special care with Uncontrolled Changes
				label1.Text = "The following files are writable in your workspace." + Environment.NewLine +
	"Red files are Uncontrolled Changes and may contain modifications you made on purpose." + Environment.NewLine +
	"Select which files you want to overwrite:";
			}
		}

		private void UncheckAll_Click(object sender, EventArgs e)
		{
			foreach (ListViewItem? item in FileList.Items)
			{
				if (item != null)
				{
					item.Checked = false;
				}
			}
		}

		private void ContinueButton_Click(object sender, EventArgs e)
		{
			foreach (ListViewItem? item in FileList.Items)
			{
				if (item != null)
				{
					_filesToClobber[(string)item.Tag!] = item.Checked;
				}
			}
		}
	}
}
