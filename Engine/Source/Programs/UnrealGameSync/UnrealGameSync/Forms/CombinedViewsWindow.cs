// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Drawing;
using System.Text;
using System.Windows.Forms;

namespace UnrealGameSync.Forms
{
	public partial class CombinedViewsWindow : ThemedForm
	{
		private readonly IList<string> _views;
		public CombinedViewsWindow(IList<string> views)
		{
			InitializeComponent();

			_views = views;
			FilterViews(String.Empty);
		}

		private void CopyToClipBoard_Click(object sender, EventArgs e)
		{
			StringBuilder sb = new StringBuilder();

			foreach (object? view in Views.Items)
			{
				if (view is string item)
				{
					sb.AppendLine(item);
				}
			}

			Clipboard.SetText(sb.ToString());
		}

		private void SearchBox_TextChanged(object sender, EventArgs e)
		{
			FilterViews(SearchBox.Text);
		}

		private void FilterViews(string search)
		{
			Views.Items.Clear();

			if (String.IsNullOrWhiteSpace(search))
			{
				foreach (string view in _views)
				{
					Views.Items.Add(view);
				}
			}
			else
			{
				foreach (string view in _views)
				{
					if (view.Contains(search, StringComparison.OrdinalIgnoreCase))
					{
						Views.Items.Add(view);
					}
				}
			}

			// Create a Graphics object to use when determining the size of the largest item in the ListBox.
			Graphics g = Views.CreateGraphics();

			// calculate the max size
			int maxHSize = 0;
			for (int i = 0; i < Views.Items.Count; i++)
			{
				int hzSize = (int)g.MeasureString(Views.Items[i].ToString(), Views.Font).Width;
				maxHSize = Math.Max(maxHSize, hzSize);
			}

			// Set the HorizontalExtent property.
			Views.HorizontalExtent = maxHSize;
		}

		private void ClearSearch_Click(object sender, EventArgs e)
		{
			SearchBox.Clear();
		}
	}
}
