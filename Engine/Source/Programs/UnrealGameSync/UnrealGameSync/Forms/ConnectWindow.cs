// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Drawing;
using System.Windows.Forms;
using EpicGames.Perforce;

namespace UnrealGameSync
{
	partial class ConnectWindow : ThemedForm
	{
		readonly IPerforceSettings _defaultPerforceSettings;
		string? _serverAndPortOverride;
		string? _userNameOverride;
		readonly IServiceProvider _serviceProvider;

		private ConnectWindow(IPerforceSettings defaultPerforceSettings, string? serverAndPortOverride, string? userNameOverride, IServiceProvider serviceProvider)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_defaultPerforceSettings = defaultPerforceSettings;
			_serviceProvider = serviceProvider;

			if (!String.IsNullOrWhiteSpace(serverAndPortOverride))
			{
				_serverAndPortOverride = serverAndPortOverride.Trim();
			}
			if (!String.IsNullOrEmpty(userNameOverride))
			{
				_userNameOverride = userNameOverride.Trim();
			}

			ServerAndPortTextBox.CueBanner = defaultPerforceSettings.ServerAndPort;
			ServerAndPortTextBox.Text = _serverAndPortOverride ?? defaultPerforceSettings.ServerAndPort;
			UserNameTextBox.CueBanner = defaultPerforceSettings.UserName;
			UserNameTextBox.Text = _userNameOverride ?? defaultPerforceSettings.UserName;
			UseDefaultConnectionSettings.Checked = _serverAndPortOverride == null && _userNameOverride == null;

			UpdateEnabledControls();
		}

		public static bool ShowModal(IWin32Window owner, IPerforceSettings defaultSettings, ref string? serverAndPortOverride, ref string? userNameOverride, IServiceProvider serviceProvider)
		{
			using ConnectWindow connect = new ConnectWindow(defaultSettings, serverAndPortOverride, userNameOverride, serviceProvider);
			if (connect.ShowDialog(owner) == DialogResult.OK)
			{
				serverAndPortOverride = connect._serverAndPortOverride;
				userNameOverride = connect._userNameOverride;
				return true;
			}
			return false;
		}

		private void CancelBtn_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
			Close();
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			if (UseDefaultConnectionSettings.Checked)
			{
				_serverAndPortOverride = null;
				_userNameOverride = null;
			}
			else
			{
				_serverAndPortOverride = ServerAndPortTextBox.Text.Trim();
				if (_serverAndPortOverride.Length == 0)
				{
					_serverAndPortOverride = _defaultPerforceSettings.ServerAndPort;
				}

				_userNameOverride = UserNameTextBox.Text.Trim();
				if (_userNameOverride.Length == 0)
				{
					_userNameOverride = _defaultPerforceSettings.UserName;
				}
			}

			DialogResult = DialogResult.OK;
			Close();
		}

		private void UseCustomSettings_CheckedChanged(object sender, EventArgs e)
		{
			UpdateEnabledControls();
		}

		private void UpdateEnabledControls()
		{
			bool useDefaultSettings = UseDefaultConnectionSettings.Checked;

			ServerAndPortLabel.Enabled = !useDefaultSettings;
			ServerAndPortTextBox.Enabled = !useDefaultSettings;

			UserNameLabel.Enabled = !useDefaultSettings;
			UserNameTextBox.Enabled = !useDefaultSettings;
			
			Color activeTextColor = Theme.TextColor;
			Color inactiveTextColor = Theme.DimmedText;
			ServerAndPortLabel.ForeColor = ServerAndPortLabel.Enabled ? activeTextColor : inactiveTextColor;
			ServerAndPortTextBox.ForeColor = ServerAndPortTextBox.Enabled ? activeTextColor : inactiveTextColor;
			UserNameLabel.ForeColor = UserNameLabel.Enabled ? activeTextColor : inactiveTextColor;
			UserNameTextBox.ForeColor = UserNameTextBox.Enabled ? activeTextColor : inactiveTextColor;
		}

		private void BrowseUserBtn_Click(object sender, EventArgs e)
		{
			string? newUserName;
			if (SelectUserWindow.ShowModal(this, new PerforceSettings(_defaultPerforceSettings) { UserName = UserNameTextBox.Text, ServerAndPort = ServerAndPortTextBox.Text }, _serviceProvider, out newUserName))
			{
				UserNameTextBox.Text = newUserName;
			}
		}
	}
}
