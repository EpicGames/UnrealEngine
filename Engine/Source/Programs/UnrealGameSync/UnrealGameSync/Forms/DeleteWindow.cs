// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Windows.Forms;

namespace UnrealGameSync
{
	public partial class DeleteWindow : ThemedForm
	{
		readonly Dictionary<string, bool> _filesToDelete;

		public DeleteWindow(Dictionary<string, bool> inFilesToDelete)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_filesToDelete = inFilesToDelete;

			foreach (KeyValuePair<string, bool> fileToDelete in _filesToDelete)
			{
				ListViewItem item = new ListViewItem(fileToDelete.Key);
				item.Tag = fileToDelete.Key;
				item.Checked = fileToDelete.Value;
				FileList.Items.Add(item);
			}
		}

		private void UncheckAll_Click(object sender, EventArgs e)
		{
			foreach (ListViewItem? item in FileList.Items)
			{
				item!.Checked = false;
			}
		}

		private void CheckAll_Click(object sender, EventArgs e)
		{
			foreach (ListViewItem? item in FileList.Items)
			{
				item!.Checked = true;
			}
		}

		private void ContinueButton_Click(object sender, EventArgs e)
		{
			foreach (ListViewItem? item in FileList.Items)
			{
				_filesToDelete[(string)item!.Tag!] = item.Checked;
			}
		}
	}
}
