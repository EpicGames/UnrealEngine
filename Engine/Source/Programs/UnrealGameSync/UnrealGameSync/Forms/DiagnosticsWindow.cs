// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Windows.Forms;
using EpicGames.Core;

namespace UnrealGameSync
{
	public partial class DiagnosticsWindow : ThemedForm
	{
		readonly DirectoryReference _appDataFolder;
		readonly DirectoryReference _workspaceDataFolder;
		readonly List<FileReference> _extraFiles;

		public DiagnosticsWindow(DirectoryReference inAppDataFolder, DirectoryReference inWorkspaceDataFolder, string inDiagnosticsText, IEnumerable<FileReference> inExtraFiles)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_appDataFolder = inAppDataFolder;
			_workspaceDataFolder = inWorkspaceDataFolder;

			DiagnosticsTextBox.Text = inDiagnosticsText.Replace("\n", "\r\n", StringComparison.Ordinal);
			_extraFiles = inExtraFiles.ToList();
		}

		private void ViewApplicationDataButton_Click(object sender, EventArgs e)
		{
			Process.Start("explorer.exe", _appDataFolder.FullName);
		}

		private void ViewWorkspaceDataButton_Click(object sender, EventArgs e)
		{
			Process.Start("explorer.exe", _workspaceDataFolder.FullName);
		}

		private void SaveButton_Click(object sender, EventArgs e)
		{
			using SaveFileDialog dialog = new SaveFileDialog();
			dialog.Filter = "Zip Files (*.zip)|*.zip|AllFiles (*.*)|*.*";
			dialog.InitialDirectory = Environment.GetFolderPath(Environment.SpecialFolder.Desktop);
			dialog.FileName = Path.Combine(dialog.InitialDirectory, "UGS-Diagnostics.zip");
			if (dialog.ShowDialog() == DialogResult.OK)
			{
				FileReference diagnosticsFileName = FileReference.Combine(_appDataFolder, "Diagnostics.txt");
				try
				{
					FileReference.WriteAllLines(diagnosticsFileName, DiagnosticsTextBox.Lines);
				}
				catch (Exception ex)
				{
					MessageBox.Show(String.Format("Couldn't write to '{0}'\n\n{1}", diagnosticsFileName, ex.ToString()));
					return;
				}

				string zipFileName = dialog.FileName;
				try
				{
					using (ZipArchive zip = new ZipArchive(File.OpenWrite(zipFileName), ZipArchiveMode.Create))
					{
						AddFilesToZip(zip, _appDataFolder, "App/");
						AddFilesToZip(zip, _workspaceDataFolder, "Workspace/");

						foreach (FileReference extraFile in _extraFiles)
						{
							if (FileReference.Exists(extraFile))
							{
								using (FileStream inputStream = FileReference.Open(extraFile, FileMode.Open, FileAccess.Read, FileShare.ReadWrite))
								{
									ZipArchiveEntry entry = zip.CreateEntry(extraFile.FullName.Replace(":", "", StringComparison.Ordinal).Replace("\\", "/", StringComparison.Ordinal));
									using (Stream outputStream = entry.Open())
									{
										inputStream.CopyTo(outputStream);
									}
								}
							}
						}
					}
				}
				catch (Exception ex)
				{
					MessageBox.Show(String.Format("Couldn't save '{0}'\n\n{1}", zipFileName, ex.ToString()));
					return;
				}
			}
		}

		private static void AddFilesToZip(ZipArchive zip, DirectoryReference dataFolder, string relativeDir)
		{
			foreach (FileReference fileName in DirectoryReference.EnumerateFiles(dataFolder))
			{
				if (!fileName.HasExtension(".exe") && !fileName.HasExtension(".dll"))
				{
					using (FileStream inputStream = FileReference.Open(fileName, FileMode.Open, FileAccess.Read, FileShare.ReadWrite))
					{
						ZipArchiveEntry entry = zip.CreateEntry(relativeDir + fileName.MakeRelativeTo(dataFolder).Replace('\\', '/'));
						using (Stream outputStream = entry.Open())
						{
							inputStream.CopyTo(outputStream);
						}
					}
				}
			}
		}
	}
}
