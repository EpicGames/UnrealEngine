// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.ExceptionServices;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;

namespace UnrealGameSync.Forms
{
	public partial class DownloadProgressWindow : ThemedForm
	{
		class ProgressReporter : IProgress<string>
		{
			readonly DownloadProgressWindow _owner;
			readonly SynchronizationContext _syncContext;

			public ProgressReporter(DownloadProgressWindow owner, SynchronizationContext syncContext)
			{
				_owner = owner;
				_syncContext = syncContext;
			}

			public void Report(string value)
			{
				_syncContext.Post(_ => _owner.UpdateStatus(value), null);
			}
		}

		readonly Func<IProgress<string>, CancellationToken, Task> _taskFunc;
		readonly CancellationTokenSource _cancellationSource;
		ExceptionDispatchInfo? _taskExceptionDispatchInfo;
		Task _task = Task.CompletedTask;

		public Task Task => _task;

		private DownloadProgressWindow(Func<IProgress<string>, CancellationToken, Task> taskFunc, CancellationToken cancellationToken)
		{
			_taskFunc = taskFunc;
			_cancellationSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);

			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));
		}

		protected override void Dispose(bool disposing)
		{
			if (disposing)
			{
				components?.Dispose();
				_cancellationSource.Dispose();
			}
			base.Dispose(disposing);
		}

		public static void Execute(Func<IProgress<string>, CancellationToken, Task> taskFunc, CancellationToken cancellationToken)
		{
			using DownloadProgressWindow window = new DownloadProgressWindow(taskFunc, cancellationToken);
			window.ShowDialog();

			cancellationToken.ThrowIfCancellationRequested();

			window._taskExceptionDispatchInfo?.Throw();
		}

		private void UpdateStatus(string text)
		{
			if (!IsDisposed)
			{
				StatusMessage.Text = text;
			}
		}

		private async Task RunTaskAsync(SynchronizationContext syncContext)
		{
			try
			{
				await _taskFunc(new ProgressReporter(this, syncContext), _cancellationSource.Token);
			}
			catch (Exception ex)
			{
				_taskExceptionDispatchInfo = ExceptionDispatchInfo.Capture(ex);
			}
		}

		private void DownloadProcessWindow_Load(object sender, EventArgs e)
		{
			SynchronizationContext syncContext = SynchronizationContext.Current!;
			_task = Task.Run(() => RunTaskAsync(syncContext), _cancellationSource.Token);
			_task.ContinueWith(x => syncContext.Post(y => Close(), null), TaskScheduler.Default);
		}

		private void DownloadProgressWindow_FormClosing(object sender, FormClosingEventArgs e)
		{
			_cancellationSource.Cancel();
			if (_task != null && !_task.IsCompleted)
			{
				e.Cancel = true;
			}
		}
	}
}
