// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Windows.Forms;
using EpicGames.Core;

namespace UnrealGameSync.Forms
{
	public partial class DownloadSettingsWindow : ThemedForm
	{
		public DownloadSettingsWindow()
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));
		}

		public static bool Show(string source, string defaultOutputDir, [NotNullWhen(true)] out DirectoryReference? outputDir)
		{
			using DownloadSettingsWindow window = new DownloadSettingsWindow();
			window.SourceText.Text = source;
			window.OutputFolderText.Text = defaultOutputDir;

			if (window.ShowDialog() != DialogResult.OK)
			{
				outputDir = null;
				return false;
			}

			outputDir = new DirectoryReference(window.OutputFolderText.Text);
			return true;
		}

		private void OutputFolderBrowseBtn_Click(object sender, EventArgs e)
		{
			using FolderBrowserDialog dialog = new FolderBrowserDialog();
			dialog.InitialDirectory = OutputFolderText.Text;

			if (dialog.ShowDialog() == DialogResult.OK)
			{
				OutputFolderText.Text = dialog.SelectedPath;
			}
		}

		private void DownloadSettingsWindow_Load(object sender, EventArgs e)
		{
			ActiveControl = DownloadBtn;
		}
	}
}
