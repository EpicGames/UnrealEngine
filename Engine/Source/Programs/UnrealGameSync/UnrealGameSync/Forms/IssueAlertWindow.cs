// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Drawing;
using System.Runtime.InteropServices;
using System.Text;
using System.Windows.Forms;

namespace UnrealGameSync
{
	partial class IssueAlertWindow : ThemedForm
	{
		static readonly IntPtr HwndTopmost = new IntPtr(-1);

		const uint SwpNomove = 0x0002;
		const uint SwpNosize = 0x0001;
		const uint SwpNoactivate = 0x0010;

		[DllImport("user32.dll", SetLastError = true)]
		static extern bool SetWindowPos(IntPtr hWnd, IntPtr hWndInsertAfter, int x, int y, int width, int height, uint flags);

		public IssueMonitor IssueMonitor { get; }

		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public bool IsWarning { get; set; }

		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public bool? StrongAlert { get; set; }

		public IssueAlertWindow(IssueMonitor issueMonitor, IssueData issue, IssueAlertReason reason)
		{
			IssueMonitor = issueMonitor;
			Issue = issue;

			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			SetIssue(issue, reason);
		}

		public IssueData Issue
		{
			get;
			private set;
		}

		public IssueAlertReason Reason
		{
			get;
			private set;
		}

		protected override void OnPaintBackground(PaintEventArgs e)
		{
			base.OnPaintBackground(e);

			if (StrongAlert ?? false)
			{
				Color stripeColor = IsWarning ? Color.FromArgb(216, 167, 64) : Color.FromArgb(200, 74, 49);//214, 69, 64);
				using (Brush stripeBrush = new SolidBrush(stripeColor))
				{
					e.Graphics.FillRectangle(stripeBrush, 0, 0, Bounds.Width, Bounds.Height);
				}
				using (Pen pen = new Pen(Color.FromArgb(255, 255, 255)))
				{
					e.Graphics.DrawRectangle(pen, 0, 0, Bounds.Width - 1, Bounds.Height - 1);
				}
			}
			else
			{
				Color stripeColor = IsWarning ? Color.FromArgb(216, 167, 64) : Color.FromArgb(214, 69, 64);

				Color backgroundColor = Color.FromArgb(241, 236, 236);
				using (Brush solidBrush = new SolidBrush(backgroundColor))
				{
					//				e.Graphics.FillRectangle(SolidBrush, 0, 0, Bounds.Width - 1, Bounds.Height - 1);
				}

				using (Pen pen = new Pen(Color.FromArgb(128, 128, 128)))
				{
					e.Graphics.DrawRectangle(pen, 0, 0, Bounds.Width - 1, Bounds.Height - 1);
				}

				using (Brush stripeBrush = new SolidBrush(stripeColor))
				{
					e.Graphics.FillRectangle(stripeBrush, 0, 0, /*6*/10 * e.Graphics.DpiX / 96.0f, Bounds.Height);
				}
			}
		}

		public void SetIssue(IssueData newIssue, IssueAlertReason newReason)
		{
			bool newStrongAlert = false;

			StringBuilder ownerTextBuilder = new StringBuilder();
			if (newIssue.Owner == null)
			{
				ownerTextBuilder.Append("Currently unassigned.");
			}
			else
			{
				if (String.Equals(newIssue.Owner, IssueMonitor.UserName, StringComparison.OrdinalIgnoreCase))
				{
					if (newIssue.NominatedBy != null)
					{
						ownerTextBuilder.AppendFormat("You have been nominated to fix this issue by {0}.", Utility.FormatUserName(newIssue.NominatedBy));
					}
					else
					{
						ownerTextBuilder.AppendFormat("Assigned to {0}.", Utility.FormatUserName(newIssue.Owner));
					}
					newStrongAlert = true;
				}
				else
				{
					ownerTextBuilder.AppendFormat("Assigned to {0}", Utility.FormatUserName(newIssue.Owner));
					if (newIssue.NominatedBy != null)
					{
						ownerTextBuilder.AppendFormat(" by {0}", Utility.FormatUserName(newIssue.NominatedBy));
					}
					if (!newIssue.AcknowledgedAt.HasValue && (newReason & IssueAlertReason.UnacknowledgedTimer) != 0)
					{
						ownerTextBuilder.Append(" (not acknowledged)");
					}
					ownerTextBuilder.Append('.');
				}
			}
			ownerTextBuilder.AppendFormat(" Open for {0}.", Utility.FormatDurationMinutes((int)(newIssue.RetrievedAt - newIssue.CreatedAt).TotalMinutes));
			string ownerText = ownerTextBuilder.ToString();

			bool newIsWarning = Issue.IsWarning;

			Issue = newIssue;

			string summary = newIssue.Summary;

			int maxLength = 128;
			if (summary.Length > maxLength)
			{
				summary = summary.Substring(0, maxLength).TrimEnd() + "...";
			}

			if (summary != SummaryLabel.Text || ownerText != OwnerLabel.Text || Reason != newReason || IsWarning != newIsWarning || StrongAlert != newStrongAlert)
			{
				Rectangle prevBounds = Bounds;
				SuspendLayout();

				SummaryLabel.Text = summary;
				OwnerLabel.Text = ownerText;

				bool forceUpdateButtons = false;
				if (StrongAlert != newStrongAlert)
				{
					StrongAlert = newStrongAlert;

					if (newStrongAlert)
					{
						SummaryLabel.ForeColor = Color.FromArgb(255, 255, 255);
						SummaryLabel.LinkColor = Color.FromArgb(255, 255, 255);
						OwnerLabel.ForeColor = Color.FromArgb(255, 255, 255);
						DetailsBtn.Theme = AlertButtonControl.AlertButtonTheme.Strong;
						AcceptBtn.Theme = AlertButtonControl.AlertButtonTheme.Strong;
						LatestBuildLinkLabel.LinkColor = Color.FromArgb(255, 255, 255);
					}
					else
					{
						SummaryLabel.ForeColor = Color.FromArgb(32, 32, 64);
						SummaryLabel.LinkColor = Color.FromArgb(32, 32, 64);
						OwnerLabel.ForeColor = Color.FromArgb(32, 32, 64);
						DetailsBtn.Theme = AlertButtonControl.AlertButtonTheme.Normal;
						AcceptBtn.Theme = AlertButtonControl.AlertButtonTheme.Green;
						LatestBuildLinkLabel.LinkColor = Color.FromArgb(16, 102, 192);
					}

					forceUpdateButtons = true;
				}

				if (IsWarning != newIsWarning)
				{
					IsWarning = newIsWarning;
				}

				if (Reason != newReason || forceUpdateButtons)
				{
					Reason = newReason;

					List<Button> buttons = new List<Button>();
					buttons.Add(DetailsBtn);
					if ((newReason & IssueAlertReason.Owner) != 0)
					{
						AcceptBtn.Text = "Acknowledge";
						buttons.Add(AcceptBtn);
					}
					else if ((newReason & IssueAlertReason.Normal) != 0)
					{
						AcceptBtn.Text = "Will Fix";
						buttons.Add(AcceptBtn);
						DeclineBtn.Text = "Not Me";
						buttons.Add(DeclineBtn);
					}
					else
					{
						DeclineBtn.Text = "Dismiss";
						buttons.Add(DeclineBtn);
					}

					tableLayoutPanel3.ColumnCount = buttons.Count;
					tableLayoutPanel3.Controls.Clear();
					for (int idx = 0; idx < buttons.Count; idx++)
					{
						tableLayoutPanel3.Controls.Add(buttons[idx], idx, 0);
					}
				}

				ResumeLayout(true);
				Location = new Point(prevBounds.Right - Bounds.Width, prevBounds.Y);
				Invalidate();
			}
		}

		protected override void OnHandleCreated(EventArgs e)
		{
			base.OnHandleCreated(e);

			// Manually make the window topmost, so that we can pass SWP_NOACTIVATE
			SetWindowPos(Handle, HwndTopmost, 0, 0, 0, 0, SwpNomove | SwpNosize | SwpNoactivate);
		}

		protected override bool ShowWithoutActivation => true;

		private void SummaryLabel_LinkClicked(object sender, System.Windows.Forms.LinkLabelLinkClickedEventArgs e)
		{
			LaunchUrl();
		}

		private void IssueAlertWindow_Click(object sender, EventArgs e)
		{
			LaunchUrl();
		}

		public void LaunchUrl()
		{
			string url = Issue.BuildUrl;
			if (String.IsNullOrEmpty(url))
			{
				MessageBox.Show("No additional information is available");
			}
			else
			{
				try
				{
					Utility.OpenUrl(url);
				}
				catch (Exception ex)
				{
					MessageBox.Show(String.Format("Unable to launch '{0}' (Error: {1})", url, ex.Message));
				}
			}
		}
	}
}
