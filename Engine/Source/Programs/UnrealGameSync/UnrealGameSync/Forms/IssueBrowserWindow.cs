// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.Drawing;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Perforce;

namespace UnrealGameSync
{
	partial class IssueBrowserWindow : ThemedForm
	{
		readonly IssueMonitor _issueMonitor;
		readonly IPerforceSettings _perforceSettings;
		readonly TimeSpan? _serverTimeOffset;
		readonly IServiceProvider _serviceProvider;
		readonly string? _currentStream;
		int _maxResults = 0;
		int _pendingMaxResults = 0;
		string? _filterName;
		readonly Dictionary<string, Func<IssueData, bool>> _customFilters = new Dictionary<string, Func<IssueData, bool>>();
		List<IssueData> _issues = new List<IssueData>();
		readonly SynchronizationContext _mainThreadSynchronizationContext;
		Task? _backgroundTask;
		bool _disposed;

		public IssueBrowserWindow(IssueMonitor issueMonitor, IPerforceSettings perforceSettings, TimeSpan? serverTimeOffset, IServiceProvider serviceProvider, string? currentStream, Dictionary<string, Func<IssueData, bool>> customFilters, string? filterName)
		{
			_issueMonitor = issueMonitor;
			_perforceSettings = perforceSettings;
			_serverTimeOffset = serverTimeOffset;
			_serviceProvider = serviceProvider;
			_currentStream = currentStream;
			_filterName = filterName;
			_customFilters = customFilters;
			_mainThreadSynchronizationContext = SynchronizationContext.Current!;

			issueMonitor.AddRef();

			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			float dpiScaleX = DeviceDpi / 96.0f;
			foreach (ColumnHeader? column in IssueListView.Columns)
			{
				if (column != null)
				{
					column.Width = (int)(column.Width * dpiScaleX);
				}
			}

			System.Reflection.PropertyInfo doubleBufferedProperty = typeof(Control).GetProperty("DoubleBuffered", System.Reflection.BindingFlags.NonPublic | System.Reflection.BindingFlags.Instance)!;
			doubleBufferedProperty.SetValue(IssueListView, true, null);
		}

		protected override void Dispose(bool disposing)
		{
			if (disposing && (components != null))
			{
				components.Dispose();
			}

			_issueMonitor.Release();

			_disposed = true;

			base.Dispose(disposing);
		}

		private void IssueBrowserWindow_Load(object sender, EventArgs e)
		{
			FetchMoreResults();
		}

		private void FetchMoreResults()
		{
			_pendingMaxResults = _maxResults + 100;
			CheckToStartBackgroundThread();
		}

		async Task FetchIssues(int newMaxResults, CancellationToken cancellationToken)
		{
			try
			{
				SortedDictionary<int, IssueData> newSortedIssues = new SortedDictionary<int, IssueData>();

				List<IssueData> openIssues = await RestApi.GetAsync<List<IssueData>>($"{_issueMonitor.ApiUrl}/api/issues", cancellationToken);
				foreach (IssueData openIssue in openIssues)
				{
					newSortedIssues[(int)openIssue.Id] = openIssue;
				}

				List<IssueData> resolvedIssues = await RestApi.GetAsync<List<IssueData>>($"{_issueMonitor.ApiUrl}/api/issues?includeresolved=true&maxresults={newMaxResults}", cancellationToken);
				foreach (IssueData resolvedIssue in resolvedIssues)
				{
					newSortedIssues[(int)resolvedIssue.Id] = resolvedIssue;
				}

				List<IssueData> newIssues = newSortedIssues.Values.Reverse().ToList();
				_mainThreadSynchronizationContext.Post((o) =>
				{
					if (!_disposed)
					{
						FetchIssuesSuccess(newMaxResults, newIssues);
					}
				}, null);
			}
			catch (Exception ex)
			{
				_mainThreadSynchronizationContext.Post((o) =>
				{
					if (!_disposed)
					{
						FetchIssuesFailure(ex);
					}
				}, null);
			}
		}

		void FetchIssuesSuccess(int newMaxResults, List<IssueData> newIssues)
		{
			_maxResults = newMaxResults;

			// Update the list of project names
			_issues = newIssues;
			UpdateIssueList();

			_backgroundTask = null;
			CheckToStartBackgroundThread();
		}

		void FetchIssuesFailure(Exception ex)
		{
			StatusLabel.Text = String.Format("Unable to fetch issues ({0})", ex.Message);
			_backgroundTask = null;
			CheckToStartBackgroundThread();
		}

		void CheckToStartBackgroundThread()
		{
			if (_pendingMaxResults != _maxResults)
			{
				StartBackgroundThread();
			}
		}

		void StartBackgroundThread()
		{
			if (_backgroundTask == null)
			{
				int newMaxResultsCopy = _pendingMaxResults;
				_backgroundTask = Task.Run(() => FetchIssues(newMaxResultsCopy, CancellationToken.None));
			}
		}

		void UpdateIssueList()
		{
			// Get the time at midnight
			DateTime now = DateTime.Now;
			DateTime midnight = (now - now.TimeOfDay).ToUniversalTime();

			// Get the regex for the selected filter
			Func<IssueData, bool>? filter;
			if (String.IsNullOrEmpty(_filterName))
			{
				filter = x => true;
			}
			else if (!_customFilters.TryGetValue(_filterName, out filter))
			{
				filter = x => x.Streams == null || x.Streams.Any(y => String.Equals(y, _filterName, StringComparison.OrdinalIgnoreCase));
			}

			// Update the table
			int itemIdx = 0;
			IssueListView.BeginUpdate();
			foreach (IssueData issue in _issues)
			{
				if (filter(issue))
				{
					for (; ; )
					{
						if (itemIdx == IssueListView.Items.Count)
						{
							IssueList_InsertItem(itemIdx, issue, midnight);
							break;
						}

						ListViewItem existingItem = IssueListView.Items[itemIdx];
						IssueData existingIssue = (IssueData)existingItem.Tag!;
						if (existingIssue == null || existingIssue.Id < issue.Id)
						{
							IssueList_InsertItem(itemIdx, issue, midnight);
							break;
						}
						else if (existingIssue.Id == issue.Id)
						{
							IssueList_UpdateItem(existingItem, issue, midnight);
							break;
						}
						else
						{
							IssueListView.Items.RemoveAt(itemIdx);
							continue;
						}
					}
					itemIdx++;
				}
			}
			while (itemIdx < IssueListView.Items.Count)
			{
				IssueListView.Items.RemoveAt(itemIdx);
			}
			IssueListView.EndUpdate();

			// Update the maximum number of results
			string filterText = "";
			if (!String.IsNullOrEmpty(_filterName))
			{
				filterText = String.Format(" matching filter '{0}'", _filterName);
			}
			StatusLabel.Text = (IssueListView.Items.Count == _issues.Count) ? String.Format("Showing {0} results{1}.", _issues.Count, filterText) : String.Format("Showing {0}/{1} results{2}.", IssueListView.Items.Count, _issues.Count, filterText);
		}

		void IssueList_InsertItem(int itemIdx, IssueData issue, DateTime midnight)
		{
			ListViewItem item = new ListViewItem("");
			for (int idx = 0; idx < IssueListView.Columns.Count - 1; idx++)
			{
				item.SubItems.Add("");
			}
			item.Tag = issue;
			IssueList_UpdateItem(item, issue, midnight);
			IssueListView.Items.Insert(itemIdx, item);
		}

		void IssueList_UpdateItem(ListViewItem item, IssueData issue, DateTime midnight)
		{
			item.SubItems[IdHeader.Index].Text = issue.Id.ToString();
			item.SubItems[CreatedHeader.Index].Text = FormatIssueDateTime(issue.CreatedAt.ToLocalTime(), midnight);
			item.SubItems[ResolvedHeader.Index].Text = issue.ResolvedAt.HasValue ? FormatIssueDateTime(issue.ResolvedAt.Value.ToLocalTime(), midnight) : "Unresolved";
			item.SubItems[TimeToFixHeader.Index].Text = issue.ResolvedAt.HasValue ? Utility.FormatDurationMinutes(issue.ResolvedAt.Value - issue.CreatedAt) : "-";
			item.SubItems[OwnerHeader.Index].Text = (issue.Owner == null) ? "-" : Utility.FormatUserName(issue.Owner);
			item.SubItems[DescriptionHeader.Index].Text = issue.Summary;
		}

		static string FormatIssueDateTime(DateTime dateTime, DateTime midnight)
		{
			if (dateTime > midnight)
			{
				return dateTime.ToShortTimeString();
			}
			else
			{
				return dateTime.ToShortDateString();
			}
		}

		static readonly List<IssueBrowserWindow> s_existingWindows = new List<IssueBrowserWindow>();

		public static void Show(Form owner, IssueMonitor issueMonitor, IPerforceSettings perforceSettings, TimeSpan? serverTimeOffset, IServiceProvider serviceProvider, string? currentStream, Dictionary<string, Func<IssueData, bool>> customFilters, string? defaultFilter)
		{
			IssueBrowserWindow? window = s_existingWindows.FirstOrDefault(x => x._issueMonitor == issueMonitor);
			if (window == null)
			{
				window = new IssueBrowserWindow(issueMonitor, perforceSettings, serverTimeOffset, serviceProvider, currentStream, customFilters, defaultFilter);
				window.Owner = owner;
				window.StartPosition = FormStartPosition.Manual;
				window.Location = new Point(owner.Location.X + (owner.Width - window.Width) / 2, owner.Location.Y + (owner.Height - window.Height) / 2);
				window.Show(owner);

				s_existingWindows.Add(window);
				window.FormClosed += (e, s) => s_existingWindows.Remove(window);
			}
			else
			{
				window.Activate();
			}
		}

		private void FetchMoreResultsLinkLabel_LinkClicked(object sender, LinkLabelLinkClickedEventArgs e)
		{
			FetchMoreResults();
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.OK;
			Close();
		}

		private void IssueListView_MouseDoubleClick(object sender, MouseEventArgs e)
		{
			ListViewHitTestInfo hitTest = IssueListView.HitTest(e.Location);
			if (hitTest.Item != null)
			{
				IssueData issue = (IssueData)hitTest.Item.Tag!;
				ShowIssue(issue);
			}
		}

		private void FilterBtn_Click(object sender, EventArgs e)
		{
			int separatorIdx = FilterMenu.Items.IndexOf(FilterMenu_Separator);
			while (FilterMenu.Items.Count > separatorIdx + 1)
			{
				FilterMenu.Items.RemoveAt(separatorIdx + 1);
			}

			FilterMenu_ShowAll.Checked = (_filterName == null);

			if (_customFilters.Count > 0)
			{
				foreach (KeyValuePair<string, Func<IssueData, bool>> customFilter in _customFilters.OrderBy(x => x.Key))
				{
					ToolStripMenuItem item = new ToolStripMenuItem(customFilter.Key);
					item.Checked = (_filterName == customFilter.Key);
					item.Click += (s, e) =>
					{
						_filterName = customFilter.Key;
						UpdateIssueList();
					};
					FilterMenu.Items.Add(item);
				}
			}

			HashSet<string> streams = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			foreach (IssueData issue in _issues)
			{
				if (issue.Streams != null)
				{
					streams.UnionWith(issue.Streams);
				}
			}

			if (streams.Count > 0)
			{
				FilterMenu.Items.Add(new ToolStripSeparator());
				foreach (string stream in streams.OrderBy(x => x))
				{
					ToolStripMenuItem item = new ToolStripMenuItem(stream);
					item.Checked = (_filterName == stream);
					item.Click += (s, e) =>
					{
						_filterName = stream;
						UpdateIssueList();
					};
					FilterMenu.Items.Add(item);
				}
			}

			FilterMenu.Show(FilterBtn, new Point(FilterBtn.Left, FilterBtn.Bottom));
			ThemeManager?.ApplyThemeRecursively(FilterMenu);
		}

		private void FilterMenu_ShowAll_Click(object sender, EventArgs e)
		{
			_filterName = null;
			UpdateIssueList();
		}

		private void ShowIssue(IssueData issue)
		{
			IssueDetailsWindow.Show(Owner!, _issueMonitor, _perforceSettings, _serverTimeOffset, issue, _serviceProvider, _currentStream);
		}

		private void IssueListView_DrawItem(object sender, DrawListViewItemEventArgs e)
		{
			if (e.Item.Selected)
			{
				IssueListView.DrawSelectedBackground(e.Graphics, e.Bounds);
			}
			else if (e.ItemIndex == IssueListView.HoverItem)
			{
				IssueListView.DrawTrackedBackground(e.Graphics, e.Bounds);
			}
			else if (e.Item.Tag is IssueData issue)
			{
				Color backgroundColor;
				if (issue.ResolvedAt.HasValue)
				{
					backgroundColor = Theme.InputBackgroundColor;
				}
				else if (issue.FixChange > 0)
				{
					backgroundColor = Theme.BackgroundColor;
				}
				else
				{
					backgroundColor = Theme.ErrorNotificationBGTint;
				}

				using (SolidBrush brush = new SolidBrush(backgroundColor))
				{
					e.Graphics.FillRectangle(brush, e.Bounds);
				}
			}
			else
			{
				IssueListView.DrawDefaultBackground(e.Graphics, e.Bounds);
			}
		}

		private void IssueListView_DrawSubItem(object sender, DrawListViewSubItemEventArgs e)
		{
			if (e.Item == null || e.SubItem == null)
			{
				return;
			}

			IssueData issue = (IssueData)e.Item.Tag!;
			if (e.ColumnIndex == IconHeader.Index)
			{
				if (!issue.ResolvedAt.HasValue && issue.FixChange == 0)
				{
					IssueListView.DrawIcon(e.Graphics, e.Bounds, WorkspaceControl.BadBuildIcon);
				}
				else
				{
					IssueListView.DrawIcon(e.Graphics, e.Bounds, WorkspaceControl.GoodBuildIcon);
				}
			}
			else
			{
				IssueListView.DrawNormalSubItem(e);
			}
		}

		private void RefreshIssuesTimer_Tick(object sender, EventArgs e)
		{
			StartBackgroundThread();
		}
	}
}
