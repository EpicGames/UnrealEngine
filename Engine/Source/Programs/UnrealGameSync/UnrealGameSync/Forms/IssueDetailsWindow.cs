// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.Diagnostics.CodeAnalysis;
using System.Drawing;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	partial class IssueDetailsWindow : ThemedForm
	{
		class BuildGroup
		{
			public string JobName { get; }
			public string JobUrl { get; }
			public int Change { get; }
			public IssueBuildOutcome Outcome { get; }
			public IReadOnlyList<IssueBuildData> Builds { get; }

			public BuildGroup(string jobName, string jobUrl, int change, IssueBuildOutcome outcome, IReadOnlyList<IssueBuildData> builds)
			{
				JobName = jobName;
				JobUrl = jobUrl;
				Change = change;
				Outcome = outcome;
				Builds = builds;
			}
		}

		class PerforceChangeRange
		{
			public bool Expanded { get; set; }
			public int MinChange { get; }
			public int MaxChange { get; }
			public List<ChangesRecord>? Changes { get; set; }
			public string? ErrorMessage { get; set; }
			public BuildGroup BuildGroup { get; }

			public PerforceChangeRange(int minChange, int maxChange, BuildGroup buildGroup)
			{
				MinChange = minChange;
				MaxChange = maxChange;
				BuildGroup = buildGroup;
			}
		}

		class PerforceChangeDetailsWithDescribeRecord : PerforceChangeDetails
		{
			public DescribeRecord DescribeRecord;

			public PerforceChangeDetailsWithDescribeRecord(DescribeRecord describeRecord)
				: base(describeRecord)
			{
				DescribeRecord = describeRecord;
			}
		}

		class PerforceWorkerTask : IDisposable
		{
			readonly IPerforceSettings _perforceSettings;
			readonly string _filter;
			Action<PerforceChangeRange>? _onUpdateChanges;
			Action<ChangesRecord>? _onUpdateChangeMetadata;
			readonly ILogger _logger;
			readonly object _lockObject = new object();
			readonly SynchronizationContext _synchronizationContext;
			Task? _workerTask;
#pragma warning disable CA2213 // warning CA2213: 'PerforceWorkerTask' contains field '_cancellationSource' that is of IDisposable type 'CancellationTokenSource', but it is never disposed. Change the Dispose method on 'PerforceWorkerTask' to call Close or Dispose on this field.
			readonly CancellationTokenSource _cancellationSource;
#pragma warning restore CA2213
			readonly AsyncEvent _refreshEvent;
			readonly List<PerforceChangeRange> _requests = new List<PerforceChangeRange>();
			readonly Dictionary<int, PerforceChangeDetailsWithDescribeRecord> _changeNumberToDetails = new Dictionary<int, PerforceChangeDetailsWithDescribeRecord>();

			public PerforceWorkerTask(IPerforceSettings perforceSettings, string filter, Action<PerforceChangeRange>? onUpdateChanges, Action<ChangesRecord>? onUpdateChangeMetadata, ILogger logger)
			{
				_perforceSettings = perforceSettings;
				_filter = filter;
				_onUpdateChanges = onUpdateChanges;
				_onUpdateChangeMetadata = onUpdateChangeMetadata;
				_logger = logger;
				_synchronizationContext = SynchronizationContext.Current!;

				_refreshEvent = new AsyncEvent();
				_cancellationSource = new CancellationTokenSource();
				_workerTask = Task.Run(() => RunAsync(_cancellationSource.Token));
			}

			public void AddRequest(PerforceChangeRange range)
			{
				lock (_lockObject)
				{
					_requests.Add(range);
				}
				_refreshEvent.Set();
			}

			public void Dispose()
			{
				if (_workerTask != null)
				{
					_onUpdateChanges = null;
					_onUpdateChangeMetadata = null;

					_cancellationSource.Cancel();

					_workerTask.ContinueWith(x => _cancellationSource.Dispose(), TaskScheduler.Default);
					_workerTask = null;
				}
			}

			async Task RunAsync(CancellationToken cancellationToken)
			{
				using IPerforceConnection perforce = await PerforceConnection.CreateAsync(_perforceSettings, _logger);

				List<PerforceChangeRange> completedRequests = new List<PerforceChangeRange>();
				while (!cancellationToken.IsCancellationRequested)
				{
					Task refreshTask = _refreshEvent.Task;

					// Check if there's a request in the queue
					PerforceChangeRange? nextRequest = null;
					lock (_lockObject)
					{
						if (_requests.Count > 0)
						{
							nextRequest = _requests[0];
							_requests.RemoveAt(0);
						}
					}

					// Process the request
					if (nextRequest != null)
					{
						string rangeFilter = String.Format("{0}@{1},{2}", _filter, nextRequest.MinChange, (nextRequest.MaxChange == -1) ? "now" : nextRequest.MaxChange.ToString());

						PerforceResponseList<ChangesRecord> newChanges = await perforce.TryGetChangesAsync(ChangesOptions.None, -1, ChangeStatus.Submitted, rangeFilter, cancellationToken);
						if (newChanges.Succeeded)
						{
							nextRequest.Changes = newChanges.Data;
							completedRequests.Add(nextRequest);
						}
						else
						{
							nextRequest.ErrorMessage = "Unable to fetch changes. Check your connection settings and try again.";
						}

						_synchronizationContext.Post(x => _onUpdateChanges?.Invoke(nextRequest), null);
						continue;
					}

					// Figure out which changes to fetch
					List<ChangesRecord> describeChanges;
					lock (_lockObject)
					{
						describeChanges = completedRequests.SelectMany(x => x.Changes ?? new List<ChangesRecord>()).Where(x => !_changeNumberToDetails.ContainsKey(x.Number)).ToList();
					}

					// Fetch info on each individual change
					foreach (ChangesRecord describeChange in describeChanges)
					{
						lock (_lockObject)
						{
							if (cancellationToken.IsCancellationRequested || _requests.Count > 0)
							{
								break;
							}
						}

						PerforceResponse<DescribeRecord> response = await perforce.TryDescribeAsync(describeChange.Number, cancellationToken);
						if (response.Succeeded)
						{
							DescribeRecord record = response.Data;
							lock (_lockObject)
							{
								_changeNumberToDetails[record.Number] = new PerforceChangeDetailsWithDescribeRecord(record);
							}
						}

						_synchronizationContext.Post(x => _onUpdateChangeMetadata?.Invoke(describeChange), null);
					}

					// Wait for something to change
					using (IDisposable _ = cancellationToken.Register(() => _refreshEvent.Set()))
					{
						await refreshTask;
					}
				}
			}

			public bool TryGetChangeDetails(int changeNumber, [NotNullWhen(true)] out PerforceChangeDetailsWithDescribeRecord? details)
			{
				lock (_lockObject)
				{
					return _changeNumberToDetails.TryGetValue(changeNumber, out details);
				}
			}
		}

		class BadgeInfo
		{
			public string _label;
			public Color _color;

			public BadgeInfo(string label, Color color)
			{
				_label = label;
				_color = color;
			}
		}

		class ExpandRangeStatusElement : StatusElement
		{
			readonly string _text;
			readonly Action _linkAction;

			public ExpandRangeStatusElement(string text, Action inLinkAction)
			{
				_text = text;
				_linkAction = inLinkAction;
				Cursor = NativeCursors.Hand;
			}

			public override void OnClick(Point location)
			{
				_linkAction();
			}

			public override Size Measure(Graphics graphics, StatusElementResources resources)
			{
				return TextRenderer.MeasureText(graphics, _text, resources.FindOrAddFont(FontStyle.Regular), new Size(Int32.MaxValue, Int32.MaxValue), TextFormatFlags.NoPadding);
			}

			public override void Draw(Graphics graphics, StatusElementResources resources)
			{
				Color textColor = Theme.TextColor;
				FontStyle style = FontStyle.Italic;
				if (MouseDown)
				{
					textColor = Color.FromArgb(textColor.B / 2, textColor.G / 2, textColor.R);
					style |= FontStyle.Underline;
				}
				else if (MouseOver)
				{
					textColor = Color.FromArgb(textColor.B, textColor.G, textColor.R);
					style |= FontStyle.Underline;
				}
				TextRenderer.DrawText(graphics, _text, resources.FindOrAddFont(style), Bounds.Location, textColor, TextFormatFlags.NoPadding);
			}
		}

		IssueMonitor _issueMonitor;
		IssueData _issue;
		readonly List<IssueBuildData> _issueBuilds;
		readonly List<IssueDiagnosticData> _diagnostics;
		readonly IPerforceSettings _perforceSettings;
		readonly TimeSpan? _serverTimeOffset;
		PerforceWorkerTask? _perforceWorker;
		readonly IServiceProvider _serviceProvider;
		readonly SynchronizationContext _mainThreadSynchronizationContext;
		string? _selectedStream;
		List<PerforceChangeRange> _selectedStreamRanges = new List<PerforceChangeRange>();
		bool _isDisposing;
		Font? _boldFont;
		ChangesRecord? _contextMenuChange;
		string? _lastOwner;
		string? _lastDetailsText;
#pragma warning disable CA2213 // warning CA2213: 'IssueDetailsWindow' contains field '_updateTimer' that is of IDisposable type 'Timer?', but it is never disposed. Change the Dispose method on 'IssueDetailsWindow' to call Close or Dispose on this field.
		System.Windows.Forms.Timer? _updateTimer;
#pragma warning restore CA2213
		StatusElementResources _statusElementResources;

		IssueDetailsWindow(IssueMonitor issueMonitor, IssueData issue, List<IssueBuildData> issueBuilds, List<IssueDiagnosticData> diagnostics, IPerforceSettings perforceSettings, TimeSpan? serverTimeOffset, IServiceProvider serviceProvider, string? currentStream)
		{
			_issueMonitor = issueMonitor;
			_issue = issue;
			_issueBuilds = issueBuilds;
			_diagnostics = diagnostics;
			_perforceSettings = perforceSettings;
			_serverTimeOffset = serverTimeOffset;
			_serviceProvider = serviceProvider;

			issueMonitor.AddRef();

			_mainThreadSynchronizationContext = SynchronizationContext.Current!;

			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			Text = String.Format("Issue {0}", issue.Id);
			_statusElementResources = new StatusElementResources(BuildListView.Font);
			Disposed += IssueDetailsWindow_Disposed;

			issueMonitor.OnIssuesChanged += OnUpdateIssuesAsync;
			issueMonitor.StartTracking(issue.Id);

			BuildListView.SmallImageList = new ImageList() { ImageSize = new Size(1, 20) };

			System.Reflection.PropertyInfo doubleBufferedProperty = typeof(Control).GetProperty("DoubleBuffered", System.Reflection.BindingFlags.NonPublic | System.Reflection.BindingFlags.Instance)!;
			doubleBufferedProperty.SetValue(BuildListView, true, null);

			using (Graphics graphics = Graphics.FromHwnd(IntPtr.Zero))
			{
				float dpiScaleX = graphics.DpiX / 96.0f;
				foreach (ColumnHeader? column in BuildListView.Columns)
				{
					if (column != null)
					{
						column.Width = (int)(column.Width * dpiScaleX);
					}
				}
			}

			int selectIdx = 0;
			foreach (string stream in issueBuilds.Select(x => x.Stream).Distinct().OrderBy(x => x))
			{
				StreamComboBox.Items.Add(stream);
				if (stream == currentStream)
				{
					selectIdx = StreamComboBox.Items.Count - 1;
				}
			}
			if (StreamComboBox.Items.Count > 0)
			{
				StreamComboBox.SelectedIndex = selectIdx;
			}

			FilterTypeComboBox.SelectedIndex = 0;

			CreateWorker();

			UpdateCurrentIssue();
		}

		protected override void Dispose(bool disposing)
		{
			if (disposing && (components != null))
			{
				components.Dispose();
			}

			if (_issueMonitor != null)
			{
				_issueMonitor.StopTracking(_issue.Id);
				_issueMonitor.OnIssuesChanged -= OnUpdateIssuesAsync;
				_issueMonitor.Release();
				_issueMonitor = null!;
			}

			_statusElementResources.Dispose();
			_boldFont?.Dispose();
			_boldFont = null;
			_perforceWorker?.Dispose();
			_perforceWorker = null;

			base.Dispose(disposing);
		}

		private void StartUpdateTimer()
		{
			if (_updateTimer == null)
			{
				_updateTimer = new System.Windows.Forms.Timer();
				_updateTimer.Interval = 100;
				_updateTimer.Tick += UpdateTimer_Tick;
				_updateTimer.Start();

				components.Add(_updateTimer);
			}
		}

		private void StopUpdateTimer()
		{
			if (_updateTimer != null)
			{
				components.Remove(_updateTimer);

				_updateTimer.Dispose();
				_updateTimer = null;
			}
		}

		private void UpdateTimer_Tick(object? sender, EventArgs e)
		{
			UpdateBuildList();
			StopUpdateTimer();
		}

		static void UpdateSummaryTextIfChanged(Label label, string newText)
		{
			if (label.Text != newText)
			{
				label.Text = newText;
			}
		}

		static void AppendEscapedRtf(StringBuilder result, string text)
		{
			for (int idx = 0; idx < text.Length; idx++)
			{
				char character = text[idx];
				if (character == '\n')
				{
					result.Append(@"\line");
				}
				else if (character >= 0x20 && character <= 0x7f)
				{
					if (character == '\\' || character == '{' || character == '}')
					{
						result.Append('\\');
					}
					result.Append(character);
				}
				else
				{
					result.AppendFormat("\\u{0}?", (int)character);
				}
			}
		}

		static void AppendHyperlink(StringBuilder richText, string label, string url)
		{
			richText.Append(@"{\field");
			richText.Append(@"{\*\fldinst");
			richText.AppendFormat("{{ HYPERLINK \"{0}\" }}", url);
			richText.Append('}');
			richText.Append(@"{\fldrslt ");
			AppendEscapedRtf(richText, label);
			richText.Append('}');
			richText.Append('}');
		}

		static string CreateRichTextErrors(List<IssueBuildData> issueBuilds, List<IssueDiagnosticData> diagnostics)
		{
			StringBuilder richText = new StringBuilder();

			richText.AppendLine(@"{\rtf1\ansi");
			richText.AppendLine(@"{\fonttbl{\f0\fnil\fcharset0 Arial;}{\f1\fnil\fcharset0 Courier New;}{\f2\fnil\fcharset0 Calibri;}}");
			richText.AppendLine(@"{\colortbl;\red192\green80\blue77;\red0\green0\blue0;}");

			bool first = true;
			foreach (IGrouping<long, IssueDiagnosticData> group in diagnostics.GroupBy(x => x.BuildId ?? -1))
			{
				// Step 'Foo'
				IssueBuildData? build = issueBuilds.FirstOrDefault(x => x.Id == group.Key);
				if (build != null)
				{
					richText.Append(@"\pard");   // Paragraph default
					richText.Append(@"\cf2");    // Foreground color
					richText.Append(@"\b1");     // Bold
					richText.Append(@"\f0");     // Font
					richText.Append(@"\fs16");   // Font size
					if (first)
					{
						richText.Append(@"\sb100"); // Space before
					}
					else
					{
						richText.Append(@"\sb300"); // Space before
					}
					richText.Append(@" In step '\ul1");
					AppendHyperlink(richText, build.JobStepName, build.JobStepUrl);
					richText.Append(@"\ul0':");

					richText.AppendLine(@"\par");
				}

				IssueDiagnosticData[] diagnosticsArray = group.ToArray();
				for (int idx = 0; idx < diagnosticsArray.Length; idx++)
				{
					IssueDiagnosticData diagnostic = diagnosticsArray[idx];

					// Error X/Y:
					richText.Append(@"\pard");   // Paragraph default
					richText.Append(@"\cf1");    // Foreground color
					richText.Append(@"\b1");     // Bold
					richText.Append(@"\f0");     // Font
					richText.Append(@"\fs16");   // Font size
					richText.Append(@"\fi50");   // First line indent
					richText.Append(@"\li50");   // Other line indent
					richText.Append(@"\sb100");  // Space before
					richText.Append(@"\sa50");   // Space after
					richText.Append(' ');

					richText.Append(@"\ul1");
					AppendHyperlink(richText, String.Format("Error {0}/{1}", idx + 1, diagnosticsArray.Length), diagnostic.Url);
					richText.Append(@"\ul0");

					richText.AppendLine(@"\par");

					// Error text
					foreach (string line in diagnostic.Message.TrimEnd().Split('\n'))
					{
						richText.Append(@"\pard");   // Paragraph default
						richText.Append(@"\cf0");    // Foreground color
						richText.Append(@"\b0");     // Bold
						richText.Append(@"\f1");     // Font
						richText.Append(@"\fs16");   // Font size 16
						richText.Append(@"\fi150");  // First line indent
						richText.Append(@"\li150");  // Other line indent
						AppendEscapedRtf(richText, line);
						richText.Append(@"\par");
					}
				}

				first = false;
			}

			richText.AppendLine("}");
			return richText.ToString();
		}

		void UpdateCurrentIssue()
		{
			UpdateSummaryTextIfChanged(SummaryTextBox, _issue.Summary.ToString());

			IssueBuildData? firstFailingBuild = _issueBuilds.FirstOrDefault(x => x.ErrorUrl != null);
			BuildLinkLabel.Text = (firstFailingBuild != null) ? firstFailingBuild.JobName : "Unknown";

			StringBuilder status = new StringBuilder();
			if (_issueMonitor.HasPendingUpdate())
			{
				status.Append("Updating...");
			}
			else if (_issue.FixChange != 0)
			{
				if (_issue.FixChange < 0)
				{
					if (_issue.ResolvedAt.HasValue)
					{
						status.AppendFormat("Closed as systemic issue.");
					}
					else
					{
						status.AppendFormat("Fixed as systemic issue (pending verification).");
					}
				}
				else
				{
					if (_issue.ResolvedAt.HasValue)
					{
						status.AppendFormat("Closed. Fixed in CL {0}.", _issue.FixChange);
					}
					else
					{
						status.AppendFormat("Fixed in CL {0} (pending verification)", _issue.FixChange);
					}
				}
			}
			else if (_issue.ResolvedAt.HasValue)
			{
				status.Append("Resolved");
			}
			else if (_issue.Owner == null)
			{
				status.Append("Currently unassigned");
			}
			else
			{
				status.Append(Utility.FormatUserName(_issue.Owner));
				if (_issue.NominatedBy != null)
				{
					status.AppendFormat(" nominated by {0}", Utility.FormatUserName(_issue.NominatedBy));
				}
				if (_issue.AcknowledgedAt.HasValue)
				{
					status.AppendFormat(" (acknowledged {0})", Utility.FormatRecentDateTime(_issue.AcknowledgedAt.Value.ToLocalTime()));
				}
				else
				{
					status.Append(" (not acknowledged)");
				}
			}
			UpdateSummaryTextIfChanged(StatusTextBox, status.ToString());

			StringBuilder openSince = new StringBuilder();
			openSince.Append(Utility.FormatRecentDateTime(_issue.CreatedAt.ToLocalTime()));
			if (openSince.Length > 0)
			{
				openSince[0] = Char.ToUpper(openSince[0]);
			}
			openSince.AppendFormat(" ({0})", Utility.FormatDurationMinutes(_issue.RetrievedAt - _issue.CreatedAt));
			UpdateSummaryTextIfChanged(OpenSinceTextBox, openSince.ToString());

			if (_lastOwner != _issue.Owner)
			{
				_lastOwner = _issue.Owner;
				BuildListView.Invalidate();
			}

			UpdateSummaryTextIfChanged(StepNamesTextBox, String.Join(", ", _issueBuilds.Select(x => x.JobStepName).Distinct().OrderBy(x => x)));
			UpdateSummaryTextIfChanged(StreamNamesTextBox, String.Join(", ", _issueBuilds.Select(x => x.Stream).Distinct().OrderBy(x => x)));

			string rtfText = CreateRichTextErrors(_issueBuilds, _diagnostics);
			if (_lastDetailsText != rtfText)
			{
				using (MemoryStream stream = new MemoryStream(Encoding.UTF8.GetBytes(rtfText), false))
				{
					DetailsTextBox.LoadFile(stream, RichTextBoxStreamType.RichText);
					DetailsTextBox.Select(0, 0);
				}
				_lastDetailsText = rtfText;
			}

			if (_issue.FixChange == 0)
			{
				MarkFixedBtn.Text = "Mark Fixed...";
			}
			else
			{
				MarkFixedBtn.Text = "Reopen";
			}
		}

		private void IssueDetailsWindow_Disposed(object? sender, EventArgs e)
		{
			_isDisposing = true;
			DestroyWorker();
		}

		void FetchBuildChanges(PerforceChangeRange range)
		{
			if (!range.Expanded)
			{
				range.Expanded = true;
				_perforceWorker!.AddRequest(range);
				UpdateBuildList();
			}
		}

		static bool FilterMatch(Regex filterRegex, ChangesRecord summary)
		{
			if (filterRegex.IsMatch(summary.User))
			{
				return true;
			}
			if (filterRegex.IsMatch(summary.Description))
			{
				return true;
			}
			return false;
		}

		static bool FilterMatch(Regex filterRegex, DescribeRecord describeRecord)
		{
			if (filterRegex.IsMatch(describeRecord.User))
			{
				return true;
			}
			if (filterRegex.IsMatch(describeRecord.Description))
			{
				return true;
			}
			if (describeRecord.Files.Any(x => filterRegex.IsMatch(x.DepotFile)))
			{
				return true;
			}
			return false;
		}

		void UpdateBuildList()
		{
			int numNewItems = 0;
			BuildListView.BeginUpdate();

			// Capture the initial selection
			object? prevSelection = null;
			if (BuildListView.SelectedItems.Count > 0)
			{
				prevSelection = BuildListView.SelectedItems[0].Tag;
			}

			// Get all the search terms
			string[] filterTerms = FilterTextBox.Text.Split(new char[] { ' ' }, StringSplitOptions.RemoveEmptyEntries);

			// Build a regex from each filter term
			List<Regex> filterRegexes = new List<Regex>();
			foreach (string filterTerm in filterTerms)
			{
				string regexText = Regex.Escape(filterTerm);
				regexText = regexText.Replace("\\?", ".", StringComparison.Ordinal);
				regexText = regexText.Replace("\\*", "[^\\\\/]*", StringComparison.Ordinal);
				regexText = regexText.Replace("\\.\\.\\.", ".*", StringComparison.Ordinal);
				filterRegexes.Add(new Regex(regexText, RegexOptions.IgnoreCase));
			}

			// Get the filter type
			bool onlyShowCodeChanges = (FilterTypeComboBox.SelectedIndex == 1);
			bool onlyShowContentChanges = (FilterTypeComboBox.SelectedIndex == 2);
			
			Color textColor = Theme.TextColor;

			// Create rows for all the ranges
			foreach (PerforceChangeRange range in _selectedStreamRanges)
			{
				if (range.Expanded)
				{
					if (range.Changes == null)
					{
						ListViewItem fetchingItem = new ListViewItem("");
						fetchingItem.Tag = range;
						fetchingItem.SubItems.Add("");
						fetchingItem.SubItems.Add("");

						StatusLineListViewWidget fetchingWidget = new StatusLineListViewWidget(fetchingItem, _statusElementResources);
						fetchingWidget.HorizontalAlignment = HorizontalAlignment.Left;
						if (range.ErrorMessage != null)
						{
							fetchingWidget.Line.AddText(range.ErrorMessage, textColor, FontStyle.Italic);
						}
						else
						{
							fetchingWidget.Line.AddText("Fetching changes, please wait...", textColor, FontStyle.Italic);
						}

						fetchingItem.SubItems.Add(new ListViewItem.ListViewSubItem(fetchingItem, "") { Tag = fetchingWidget });
						fetchingItem.SubItems.Add(new ListViewItem.ListViewSubItem(fetchingItem, "") { Tag = fetchingWidget });

						BuildListView.Items.Insert(numNewItems++, fetchingItem);
					}
					else
					{
						foreach (ChangesRecord change in range.Changes)
						{
							PerforceChangeDetailsWithDescribeRecord? details;
							_perforceWorker!.TryGetChangeDetails(change.Number, out details);

							if (filterRegexes.Count > 0 || onlyShowCodeChanges || onlyShowContentChanges)
							{
								if (details == null)
								{
									if (onlyShowCodeChanges || onlyShowCodeChanges)
									{
										continue;
									}
									if (filterRegexes.Any(x => !FilterMatch(x, change)))
									{
										continue;
									}
								}
								else
								{
									if (onlyShowCodeChanges && !details.ContainsCode)
									{
										continue;
									}
									if (onlyShowContentChanges && !details.ContainsContent)
									{
										continue;
									}
									if (filterRegexes.Any(x => !FilterMatch(x, details.DescribeRecord)))
									{
										continue;
									}
								}
							}

							ListViewItem item = new ListViewItem("");
							item.Tag = change;

							StatusLineListViewWidget typeWidget = new StatusLineListViewWidget(item, _statusElementResources);
							UpdateChangeTypeWidget(typeWidget, details);
							item.SubItems.Add(new ListViewItem.ListViewSubItem(item, "") { Tag = typeWidget });

							item.SubItems.Add(change.Number.ToString());

							DateTime displayTime;
							if (!_serverTimeOffset.HasValue)
							{
								displayTime = change.Time.ToLocalTime();
							}
							else
							{
								displayTime = new DateTime(change.Time.Ticks + _serverTimeOffset.Value.Ticks, DateTimeKind.Local);
							}

							item.SubItems.Add(displayTime.ToString("h\\.mmtt"));

							item.SubItems.Add(WorkspaceControl.FormatUserName(change.User));

							item.SubItems.Add(change.Description);
							BuildListView.Items.Insert(numNewItems++, item);
						}
					}
				}
				/*
				else
				{
					ListViewItem RangeItem = new ListViewItem("");
					RangeItem.Tag = Range;

					StatusLineListViewWidget RangeWidget = new StatusLineListViewWidget(RangeItem, StatusElementResources);
					RangeWidget.Line.AddText("-", Color.Gray);//.AddBadge("...", Color.LightGray, () => FetchBuildChanges(RangeItem));
					RangeItem.SubItems.Add(new ListViewItem.ListViewSubItem(RangeItem, ""){ Tag = RangeWidget });

					StatusLineListViewWidget RangeWidget2 = new StatusLineListViewWidget(RangeItem, StatusElementResources);
					RangeWidget2.Line.AddText("-", Color.Gray);//.AddBadge("...", Color.LightGray, () => FetchBuildChanges(RangeItem));
					RangeItem.SubItems.Add(new ListViewItem.ListViewSubItem(RangeItem, "-"){ Tag = RangeWidget2 });

					StatusLineListViewWidget TipWidget = new StatusLineListViewWidget(RangeItem, StatusElementResources);
					TipWidget.HorizontalAlignment = HorizontalAlignment.Left;
					TipWidget.Line.Add(new ExpandRangeStatusElement("Click to show changes...", () => FetchAllBuildChanges()));

					RangeItem.SubItems.Add(new ListViewItem.ListViewSubItem(RangeItem, ""){ Tag = TipWidget });
					RangeItem.SubItems.Add(new ListViewItem.ListViewSubItem(RangeItem, ""){ Tag = TipWidget });

					BuildListView.Items.Insert(NumNewItems++, RangeItem);
				}
				*/
				ListViewItem buildItem = new ListViewItem("");
				buildItem.Tag = range.BuildGroup;
				buildItem.SubItems.Add("");
				buildItem.SubItems.Add(range.BuildGroup.Change.ToString());
				buildItem.SubItems.Add("");

				StatusLineListViewWidget buildWidget = new StatusLineListViewWidget(buildItem, _statusElementResources);
				buildWidget.HorizontalAlignment = HorizontalAlignment.Left;

				buildWidget.Line.AddLink(range.BuildGroup.JobName, FontStyle.Underline, () => Utility.OpenUrl(range.BuildGroup.JobUrl));
				buildItem.SubItems.Add(new ListViewItem.ListViewSubItem(buildItem, "") { Tag = buildWidget });
				buildItem.SubItems.Add(new ListViewItem.ListViewSubItem(buildItem, "") { Tag = buildWidget });

				BuildListView.Items.Insert(numNewItems++, buildItem);
			}

			// Re-select the original item
			for (int idx = 0; idx < BuildListView.Items.Count; idx++)
			{
				if (BuildListView.Items[idx].Tag == prevSelection)
				{
					BuildListView.Items[idx].Selected = true;
				}
			}

			// Remove all the items we no longer need
			while (BuildListView.Items.Count > numNewItems)
			{
				BuildListView.Items.RemoveAt(BuildListView.Items.Count - 1);
			}

			// Redraw everything
			BuildListView.EndUpdate();
			BuildListView.Invalidate();
		}

		BuildGroup? _selectedBuildGroup;

		void ShowJobContextMenu(Point point, BuildGroup buildGroup)
		{
			_selectedBuildGroup = buildGroup;

			int minIndex = JobContextMenu.Items.IndexOf(JobContextMenu_StepSeparatorMin) + 1;
			while (JobContextMenu.Items.Count > minIndex)
			{
				JobContextMenu.Items.RemoveAt(minIndex);
			}

			JobContextMenu_ViewJob.Text = String.Format("View Job: {0}", buildGroup.JobName);

			foreach (IssueBuildData build in buildGroup.Builds.OrderBy(x => x.JobStepName))
			{
#pragma warning disable CA2000 // warning CA2000: Call System.IDisposable.Dispose on object created by 'new ToolStripMenuItem(String.Format("View Step: {0}", build.JobStepName))' before all references to it are out of scope
				ToolStripMenuItem menuItem = new ToolStripMenuItem(String.Format("View Step: {0}", build.JobStepName));
				menuItem.Click += (s, e) => Utility.OpenUrl(build.JobStepUrl);
				JobContextMenu.Items.Insert(minIndex++, menuItem);
#pragma warning restore CA2000
			}

			JobContextMenu.Show(BuildListView, point, ToolStripDropDownDirection.BelowRight);
		}

		static void UpdateChangeTypeWidget(StatusLineListViewWidget typeWidget, PerforceChangeDetails? details)
		{
			typeWidget.Line.Clear();
			if (details == null)
			{
				typeWidget.Line.AddBadge("Unknown", Color.FromArgb(192, 192, 192), null);
			}
			else
			{
				if (details.ContainsCode)
				{
					typeWidget.Line.AddBadge("Code", Theme.CodeBadgeColor, null);
				}
				if (details.ContainsContent)
				{
					typeWidget.Line.AddBadge("Content", Theme.ContentBadgeColor, null);
				}
			}
		}

		void CreateWorker()
		{
			string? newSelectedStream = StreamComboBox.SelectedItem as string;
			if (_selectedStream != newSelectedStream)
			{
				DestroyWorker();

				_selectedStream = newSelectedStream;

				BuildListView.BeginUpdate();
				BuildListView.Items.Clear();

				if (_selectedStream != null)
				{
					_selectedStreamRanges = new List<PerforceChangeRange>();

					int maxChange = -1;
					foreach (IGrouping<string, IssueBuildData> group in _issueBuilds.Where(x => x.Stream == _selectedStream).OrderByDescending(x => x.Change).ThenByDescending(x => x.JobUrl).GroupBy(x => x.JobUrl))
					{
						IssueBuildOutcome outcome = group.Any(x => x.Outcome == IssueBuildOutcome.Error) ? IssueBuildOutcome.Error : group.Any(x => x.Outcome == IssueBuildOutcome.Warning) ? IssueBuildOutcome.Warning : IssueBuildOutcome.Success;
						BuildGroup buildGroup = new BuildGroup(group.First().JobName, group.Key, group.First().Change, outcome, group.ToList());

						PerforceChangeRange range = new PerforceChangeRange(buildGroup.Change + 1, maxChange, buildGroup);
						_selectedStreamRanges.Add(range);

						maxChange = buildGroup.Change;
					}

					_perforceWorker = new PerforceWorkerTask(_perforceSettings, String.Format("{0}/...", _selectedStream), OnRequestComplete, UpdateChangeMetadata, _serviceProvider.GetRequiredService<ILogger<PerforceWorkerTask>>());

					UpdateBuildList();

					for (int idx = 0; idx + 2 < _selectedStreamRanges.Count; idx++)
					{
						if (_selectedStreamRanges[idx].BuildGroup.Outcome != _selectedStreamRanges[idx + 1].BuildGroup.Outcome)
						{
							FetchBuildChanges(_selectedStreamRanges[idx + 1]);
						}
					}
					FetchBuildChanges(_selectedStreamRanges[^1]);
				}

				BuildListView.EndUpdate();
				BuildListView.Invalidate();
			}
		}

		void OnUpdateIssues()
		{
			List<IssueData> newIssues = _issueMonitor.GetIssues();
			foreach (IssueData newIssue in newIssues)
			{
				if (newIssue.Id == _issue.Id)
				{
					_issue = newIssue;
					UpdateCurrentIssue();
					break;
				}
			}
		}

		private void OnUpdateIssuesAsync()
		{
			_mainThreadSynchronizationContext.Post((o) =>
			{
				if (!_isDisposing)
				{
					OnUpdateIssues();
				}
			}, null);
		}

		void DestroyWorker()
		{
			if (_perforceWorker != null)
			{
				_perforceWorker.Dispose();
				_perforceWorker = null!;
			}
		}

		void OnRequestComplete(PerforceChangeRange range)
		{
			UpdateBuildList();
		}

		void UpdateChangeMetadata(ChangesRecord change)
		{
			if (FilterTextBox.Text.Length > 0 || FilterTypeComboBox.SelectedIndex != 0)
			{
				StartUpdateTimer();
			}
			else
			{
				foreach (ListViewItem? item in BuildListView.Items)
				{
					if (item != null && item.Tag == change)
					{
						PerforceChangeDetailsWithDescribeRecord? details;
						_perforceWorker!.TryGetChangeDetails(change.Number, out details);

						StatusLineListViewWidget typeWidget = (StatusLineListViewWidget)item.SubItems[TypeHeader.Index].Tag!;
						UpdateChangeTypeWidget(typeWidget, details);

						BuildListView.RedrawItems(item.Index, item.Index, true);
						break;
					}
				}
			}
		}

		static readonly List<IssueDetailsWindow> s_existingWindows = new List<IssueDetailsWindow>();

		class UpdateIssueDetailsTask
		{
			readonly string _apiUrl;
			readonly long _issueId;
			readonly List<IssueDiagnosticData> _diagnostics;

			public UpdateIssueDetailsTask(string apiUrl, long issueId, List<IssueDiagnosticData> diagnostics)
			{
				_apiUrl = apiUrl;
				_issueId = issueId;
				_diagnostics = diagnostics;
			}

			public async Task RunAsync(CancellationToken cancellationToken)
			{
				_diagnostics.AddRange(await RestApi.GetAsync<List<IssueDiagnosticData>>($"{_apiUrl}/api/issues/{_issueId}/diagnostics", cancellationToken));
			}
		}

		public static void Show(Form owner, IssueMonitor issueMonitor, IPerforceSettings perforceSettings, TimeSpan? serverTimeOffset, IssueData issue, IServiceProvider serviceProvider, string? currentStream)
		{
			Task<List<IssueBuildData>> Func(CancellationToken cancellationToken) => RestApi.GetAsync<List<IssueBuildData>>($"{issueMonitor.ApiUrl}/api/issues/{issue.Id}/builds", cancellationToken);

			ModalTask<List<IssueBuildData>>? issueBuilds = ModalTask.Execute(owner, "Querying issue", "Querying issue, please wait...", Func);
			if (issueBuilds != null && issueBuilds.Succeeded)
			{
				Show(owner, issueMonitor, perforceSettings, serverTimeOffset, issue, issueBuilds.Result, serviceProvider, currentStream);
			}
		}

		public static void Show(Form owner, IssueMonitor issueMonitor, IPerforceSettings perforceSettings, TimeSpan? serverTimeOffset, IssueData issue, List<IssueBuildData> issueBuilds, IServiceProvider serviceProvider, string? currentStream)
		{
			IssueDetailsWindow? window = s_existingWindows.FirstOrDefault(x => x._issueMonitor == issueMonitor && x._issue.Id == issue.Id);
			if (window == null)
			{
				List<IssueDiagnosticData> diagnostics = new List<IssueDiagnosticData>();

				UpdateIssueDetailsTask task = new UpdateIssueDetailsTask(issueMonitor.ApiUrl!, issue.Id, diagnostics);
				ModalTask? taskResult = ModalTask.Execute(owner, "Fetching data", "Fetching data, please wait...", task.RunAsync);
				if (taskResult == null || !taskResult.Succeeded)
				{
					return;
				}

				window = new IssueDetailsWindow(issueMonitor, issue, issueBuilds, diagnostics, perforceSettings, serverTimeOffset, serviceProvider, currentStream);
				window.Owner = owner;
				if (owner.Visible && owner.WindowState != FormWindowState.Minimized)
				{
					window.StartPosition = FormStartPosition.Manual;
					window.Location = new Point(owner.Location.X + (owner.Width - window.Width) / 2, owner.Location.Y + (owner.Height - window.Height) / 2);
				}
				else
				{
					window.StartPosition = FormStartPosition.CenterScreen;
				}
				window.Show();

				s_existingWindows.Add(window);
				window.FormClosed += (s, e) => s_existingWindows.Remove(window);
			}
			else
			{
				window.Location = new Point(owner.Location.X + (owner.Width - window.Width) / 2, owner.Location.Y + (owner.Height - window.Height) / 2);
				window.Activate();
			}
		}

		private void CloseBtn_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.OK;
			Close();
		}

		private void StreamComboBox_SelectedIndexChanged(object sender, EventArgs e)
		{
			DestroyWorker();
			CreateWorker();
		}

		private void BuildListView_DrawColumnHeader(object sender, DrawListViewColumnHeaderEventArgs e)
		{
			e.DrawDefault = true;
		}

		private void BuildListView_DrawItem(object sender, DrawListViewItemEventArgs e)
		{
			if (e.Item.Selected)
			{
				BuildListView.DrawSelectedBackground(e.Graphics, e.Bounds);
			}
			else if (e.ItemIndex == BuildListView.HoverItem)
			{
				BuildListView.DrawTrackedBackground(e.Graphics, e.Bounds);
			}
			else if (e.Item.Tag is BuildGroup buildGroup)
			{
				Color backgroundColor;
				if (buildGroup.Outcome == IssueBuildOutcome.Error)
				{
					backgroundColor = Theme.ErrorNotificationBGTint;
				}
				else if (buildGroup.Outcome == IssueBuildOutcome.Warning)
				{
					backgroundColor = Theme.WarningNotificationBGTint;
				}
				else if (buildGroup.Outcome == IssueBuildOutcome.Success)
				{
					backgroundColor = Theme.SuccessNotificationBGTint;
				}
				else
				{
					backgroundColor = Theme.StandardNotificationBGTint;
				}

				using (SolidBrush brush = new SolidBrush(backgroundColor))
				{
					e.Graphics.FillRectangle(brush, e.Bounds);
				}
			}
			else
			{
				BuildListView.DrawDefaultBackground(e.Graphics, e.Bounds);
			}
		}

		private void BuildListView_DrawSubItem(object sender, DrawListViewSubItemEventArgs e)
		{
			if (e.Item == null || e.SubItem == null)
			{
				return;
			}

			Font currentFont = Font;//BuildFont;(Change.Number == Workspace.PendingChangeNumber || Change.Number == Workspace.CurrentChangeNumber)? SelectedBuildFont : BuildFont;

			Color textColor = Theme.TextColor;

			if (e.SubItem.Tag is CustomListViewWidget)
			{
				BuildListView.DrawCustomSubItem(e.Graphics, e.SubItem);
			}
			else if (e.Item.Tag is ChangesRecord change)
			{
				Font changeFont = BuildListView.Font;
				if (_issue.Owner != null && String.Equals(change.User, _issue.Owner, StringComparison.OrdinalIgnoreCase))
				{
					changeFont = _boldFont!;
				}

				if (e.ColumnIndex == ChangeHeader.Index)
				{
					TextRenderer.DrawText(e.Graphics, e.SubItem.Text, changeFont, e.Bounds, textColor, TextFormatFlags.EndEllipsis | TextFormatFlags.SingleLine | TextFormatFlags.HorizontalCenter | TextFormatFlags.VerticalCenter | TextFormatFlags.NoPrefix);
				}
				else if (e.ColumnIndex == AuthorHeader.Index)
				{
					TextRenderer.DrawText(e.Graphics, e.SubItem.Text, changeFont, e.Bounds, textColor, TextFormatFlags.EndEllipsis | TextFormatFlags.SingleLine | TextFormatFlags.Left | TextFormatFlags.VerticalCenter | TextFormatFlags.NoPrefix);
				}
				else if (e.ColumnIndex == DescriptionHeader.Index)
				{
					TextRenderer.DrawText(e.Graphics, e.SubItem.Text, changeFont, e.Bounds, textColor, TextFormatFlags.EndEllipsis | TextFormatFlags.SingleLine | TextFormatFlags.Left | TextFormatFlags.VerticalCenter | TextFormatFlags.NoPrefix);
				}
				else if (e.ColumnIndex == TimeHeader.Index)
				{
					TextRenderer.DrawText(e.Graphics, e.SubItem.Text, changeFont, e.Bounds, textColor, TextFormatFlags.EndEllipsis | TextFormatFlags.SingleLine | TextFormatFlags.HorizontalCenter | TextFormatFlags.VerticalCenter | TextFormatFlags.NoPrefix);
				}
			}
			else if (e.Item.Tag is IssueBuildData buildData)
			{
				Font boldFont = BuildListView.Font;

				//				TextColor = SystemColors.Window;
				if (e.ColumnIndex == IconHeader.Index)
				{
					if (buildData.Outcome == IssueBuildOutcome.Success)
					{
						BuildListView.DrawIcon(e.Graphics, e.Bounds, WorkspaceControl.GoodBuildIcon);
					}
					else if (buildData.Outcome == IssueBuildOutcome.Warning)
					{
						BuildListView.DrawIcon(e.Graphics, e.Bounds, WorkspaceControl.MixedBuildIcon);
					}
					else if (buildData.Outcome == IssueBuildOutcome.Error)
					{
						BuildListView.DrawIcon(e.Graphics, e.Bounds, WorkspaceControl.BadBuildIcon);
					}
					else
					{
						BuildListView.DrawIcon(e.Graphics, e.Bounds, WorkspaceControl.DefaultBuildIcon);
					}
				}
				else if (e.ColumnIndex == ChangeHeader.Index)
				{
					TextRenderer.DrawText(e.Graphics, buildData.Change.ToString(), boldFont, e.Bounds, textColor, TextFormatFlags.EndEllipsis | TextFormatFlags.SingleLine | TextFormatFlags.HorizontalCenter | TextFormatFlags.VerticalCenter | TextFormatFlags.NoPrefix);
				}
				else if (e.ColumnIndex == TypeHeader.Index)
				{
					string status;
					if (buildData.Outcome == IssueBuildOutcome.Success)
					{
						status = "Succeeded";
					}
					else if (buildData.Outcome == IssueBuildOutcome.Warning)
					{
						status = "Warning";
					}
					else if (buildData.Outcome == IssueBuildOutcome.Error)
					{
						status = "Failed";
					}
					else
					{
						status = "Pending";
					}
					TextRenderer.DrawText(e.Graphics, status, boldFont, e.Bounds, textColor, TextFormatFlags.EndEllipsis | TextFormatFlags.SingleLine | TextFormatFlags.HorizontalCenter | TextFormatFlags.VerticalCenter | TextFormatFlags.NoPrefix);
				}
				else if (e.ColumnIndex == AuthorHeader.Index)
				{
					Rectangle bounds = new Rectangle(e.Bounds.X, e.Bounds.Y, e.Bounds.Width + e.Item.SubItems[e.ColumnIndex].Bounds.Width, e.Bounds.Height);
					TextRenderer.DrawText(e.Graphics, buildData.JobName, Font, bounds, textColor, TextFormatFlags.EndEllipsis | TextFormatFlags.SingleLine | TextFormatFlags.Left | TextFormatFlags.VerticalCenter | TextFormatFlags.NoPrefix);
				}
			}
			else
			{
				TextFormatFlags flags = TextFormatFlags.EndEllipsis | TextFormatFlags.SingleLine | TextFormatFlags.VerticalCenter | TextFormatFlags.NoPrefix;
				if (BuildListView.Columns[e.ColumnIndex].TextAlign == HorizontalAlignment.Left)
				{
					flags |= TextFormatFlags.Left;
				}
				else if (BuildListView.Columns[e.ColumnIndex].TextAlign == HorizontalAlignment.Center)
				{
					flags |= TextFormatFlags.HorizontalCenter;
				}
				else
				{
					flags |= TextFormatFlags.Right;
				}
				TextRenderer.DrawText(e.Graphics, e.SubItem.Text, currentFont, e.Bounds, textColor, flags);
			}
		}

		private void BuildListView_FontChanged(object sender, EventArgs e)
		{
			_boldFont?.Dispose();
			_boldFont = new Font(BuildListView.Font.FontFamily, BuildListView.Font.SizeInPoints, FontStyle.Bold);

			_statusElementResources?.Dispose();
			_statusElementResources = new StatusElementResources(BuildListView.Font);
		}

		private void BuildListView_MouseClick(object sender, MouseEventArgs args)
		{
			if (args.Button == MouseButtons.Right)
			{
				ListViewHitTestInfo hitTest = BuildListView.HitTest(args.Location);
				if (hitTest.Item != null && hitTest.Item.Tag != null)
				{
					_contextMenuChange = hitTest.Item.Tag as ChangesRecord;
					if (_contextMenuChange != null)
					{
						BuildListContextMenu_Assign.Text = String.Format("Assign to {0}", _contextMenuChange.User);
						BuildListContextMenu.Show(BuildListView, args.Location);
						ThemeManager?.ApplyThemeRecursively(BuildListContextMenu);
					}
				}
			}
		}

		private void BuildListContextMenu_MoreInfo_Click(object sender, EventArgs e)
		{
			if (_contextMenuChange != null)
			{
				Program.SpawnP4Vc(String.Format("{0} change {1}", _perforceSettings.GetArgumentsForExternalProgram(true), _contextMenuChange.Number));
			}
		}

		private void BuildListContextMenu_Blame_Click(object sender, EventArgs e)
		{
			if (_contextMenuChange != null && _contextMenuChange.User != null)
			{
				AssignToUser(_contextMenuChange.User);
			}
		}

		private void AssignToUser(string user)
		{
			IssueUpdateData update = new IssueUpdateData();
			update.Id = _issue.Id;
			update.Owner = user;
			update.FixChange = 0;
			if (String.Equals(user, _perforceSettings.UserName, StringComparison.OrdinalIgnoreCase))
			{
				update.NominatedBy = "";
				update.Acknowledged = true;
			}
			else
			{
				update.NominatedBy = _perforceSettings.UserName;
				update.Acknowledged = false;
			}
			_issueMonitor.PostUpdate(update);

			UpdateCurrentIssue();
		}

		private void AssignToMeBtn_Click(object sender, EventArgs e)
		{
			AssignToUser(_issueMonitor.UserName);
		}

		private void AssignToOtherBtn_Click(object sender, EventArgs e)
		{
			string? selectedUserName;
			if (SelectUserWindow.ShowModal(this, _perforceSettings, _serviceProvider, out selectedUserName))
			{
				AssignToUser(selectedUserName);
			}
		}

		private void FilterTextBox_TextChanged(object sender, EventArgs e)
		{
			StopUpdateTimer();
			StartUpdateTimer();
		}

		private void FilterTypeComboBox_SelectedIndexChanged(object sender, EventArgs e)
		{
			UpdateBuildList();
		}

		private void MarkFixedBtn_Click(object sender, EventArgs e)
		{
			int fixChangeNumber = _issue.FixChange;
			if (fixChangeNumber == 0)
			{
				if (IssueFixedWindow.ShowModal(this, _perforceSettings, _serviceProvider, ref fixChangeNumber))
				{
					IssueUpdateData update = new IssueUpdateData();
					update.Id = _issue.Id;
					update.FixChange = fixChangeNumber;
					_issueMonitor.PostUpdate(update);
				}
			}
			else
			{
				IssueUpdateData update = new IssueUpdateData();
				update.Id = _issue.Id;
				update.FixChange = 0;
				_issueMonitor.PostUpdate(update);
			}
		}

		private void DescriptionLabel_LinkClicked(object sender, LinkLabelLinkClickedEventArgs e)
		{
			IssueBuildData? lastBuild = _issueBuilds.Where(x => x.Stream == _selectedStream).OrderByDescending(x => x.Change).ThenByDescending(x => x.ErrorUrl).FirstOrDefault();
			if (lastBuild != null)
			{
				Utility.OpenUrl(lastBuild.ErrorUrl);
			}
		}

		private void JobContextMenu_ShowError_Click(object sender, EventArgs e)
		{
			if (_selectedBuildGroup != null)
			{
				foreach (IssueBuildData build in _selectedBuildGroup.Builds)
				{
					if (build.ErrorUrl != null)
					{
						Utility.OpenUrl(build.ErrorUrl);
						break;
					}
				}
			}
		}

		private void JobContextMenu_ViewJob_Click(object sender, EventArgs e)
		{
			if (_selectedBuildGroup != null)
			{
				Utility.OpenUrl(_selectedBuildGroup.JobUrl);
			}
		}

		private void BuildListView_MouseUp(object sender, MouseEventArgs e)
		{
			if ((e.Button & MouseButtons.Right) != 0)
			{
				ListViewHitTestInfo hitTest = BuildListView.HitTest(e.Location);
				if (hitTest.Item != null)
				{
					BuildGroup? group = hitTest.Item.Tag as BuildGroup;
					if (group != null)
					{
						ShowJobContextMenu(e.Location, group);
					}
				}
			}
		}

		private void BuildLinkLabel_LinkClicked(object sender, LinkLabelLinkClickedEventArgs e)
		{
			IssueBuildData? build = _issueBuilds.FirstOrDefault(x => x.ErrorUrl != null);
			if (build != null)
			{
				Utility.OpenUrl(build.ErrorUrl);
			}
		}

		private void DetailsTextBox_LinkClicked(object sender, LinkClickedEventArgs e)
		{
			if (e.LinkText != null)
			{
				Utility.OpenUrl(e.LinkText);
			}
		}
	}
}
