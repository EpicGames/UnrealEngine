// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Data;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace UnrealGameSync
{
	partial class IssueFixedWindow : ThemedForm
	{
		class FindChangesWorker : Component
		{
			public delegate void OnCompleteDelegate(string userName, List<DescribeRecord> changes);

			readonly IPerforceSettings _perforceSettings;
			readonly SynchronizationContext _mainThreadSyncContext;
			readonly AsyncEvent _wakeEvent;
			string? _requestedUserName;
			readonly CancellationTokenSource _cancellationSource;
			Task _backgroundTask;
			OnCompleteDelegate? _onComplete;
			readonly ILogger _logger;

			public FindChangesWorker(IPerforceSettings perforceSettings, OnCompleteDelegate? onComplete, ILogger logger)
			{
				_perforceSettings = perforceSettings;
				_mainThreadSyncContext = SynchronizationContext.Current!;
				_wakeEvent = new AsyncEvent();
				_onComplete = onComplete;
				_logger = logger;

				_cancellationSource = new CancellationTokenSource();
				_backgroundTask = Task.Run(() => DoWork(_cancellationSource.Token));
			}

			public void Stop()
			{
				StopAsync().Wait();
			}

			async Task StopAsync()
			{
				if (_backgroundTask != null)
				{
					_onComplete = null;

					await _cancellationSource.CancelAsync();
					await _backgroundTask;

					_backgroundTask = null!;
				}
			}

			public void FetchChanges(string userName)
			{
				_requestedUserName = userName;
				_wakeEvent.Set();
			}

			protected override void Dispose(bool disposing)
			{
				base.Dispose(disposing);

				Stop();

				_cancellationSource.Dispose();
			}

			public async Task DoWork(CancellationToken cancellationToken)
			{
				using IPerforceConnection perforce = await PerforceConnection.CreateAsync(_perforceSettings, _logger);

				Task wakeTask = _wakeEvent.Task;
				Task cancelTask = Task.Delay(-1, cancellationToken);
				for (; ; )
				{
					await Task.WhenAny(wakeTask, cancelTask);

					if (cancellationToken.IsCancellationRequested)
					{
						break;
					}

					wakeTask = _wakeEvent.Task;

					string? userName = _requestedUserName;
					if (userName != null)
					{
						List<ChangesRecord> changes = await perforce.GetChangesAsync(ChangesOptions.IncludeTimes, 100, ChangeStatus.Submitted, FileSpecList.Any, cancellationToken);
						List<DescribeRecord> descriptions = await perforce.DescribeAsync(changes.Select(x => x.Number).ToArray(), cancellationToken);

						_mainThreadSyncContext.Post(_ => _onComplete?.Invoke(userName, descriptions), null);
					}
				}
			}
		}

		readonly IPerforceSettings _perforceSettings;
		int _changeNumber;
#pragma warning disable CA2213 // warning CA2213: 'IssueFixedWindow' contains field '_worker' that is of IDisposable type 'FindChangesWorker', but it is never disposed. Change the Dispose method on 'IssueFixedWindow' to call Close or Dispose on this field.
		readonly FindChangesWorker _worker;
#pragma warning restore CA2213
		readonly IServiceProvider _serviceProvider;

		public IssueFixedWindow(IPerforceSettings perforceSettings, int initialChangeNumber, IServiceProvider serviceProvider)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_perforceSettings = perforceSettings;
			_worker = new FindChangesWorker(perforceSettings, PopulateChanges, serviceProvider.GetRequiredService<ILogger<FindChangesWorker>>());
			_serviceProvider = serviceProvider;
			components!.Add(_worker);

			UserNameTextBox.Text = perforceSettings.UserName;
			UserNameTextBox.SelectionStart = UserNameTextBox.Text.Length;

			if (initialChangeNumber != 0)
			{
				if (initialChangeNumber < 0)
				{
					SpecifyChangeRadioButton.Checked = false;
					SystemicFixRadioButton.Checked = true;
				}
				else
				{
					SpecifyChangeRadioButton.Checked = true;
					ChangeNumberTextBox.Text = initialChangeNumber.ToString();
				}
			}

			UpdateOkButton();
		}

		protected override void OnLoad(EventArgs e)
		{
			base.OnLoad(e);

			if (SystemicFixRadioButton.Checked)
			{
				SystemicFixRadioButton.Select();
			}
			else if (SpecifyChangeRadioButton.Checked)
			{
				SpecifyChangeRadioButton.Select();
			}
			else if (RecentChangeRadioButton.Checked)
			{
				ChangesListView.Select();
			}

			_worker.FetchChanges(UserNameTextBox.Text);
		}

		public static bool ShowModal(IWin32Window owner, IPerforceSettings perforce, IServiceProvider serviceProvider, ref int fixChangeNumber)
		{
			using (IssueFixedWindow fixedWindow = new IssueFixedWindow(perforce, fixChangeNumber, serviceProvider))
			{
				if (fixedWindow.ShowDialog(owner) == DialogResult.OK)
				{
					fixChangeNumber = fixedWindow._changeNumber;
					return true;
				}
				else
				{
					fixChangeNumber = 0;
					return false;
				}
			}
		}

		private void UpdateSelectedChangeAndClose()
		{
			if (TryGetSelectedChange(out _changeNumber))
			{
				DialogResult = DialogResult.OK;
				Close();
			}
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			UpdateSelectedChangeAndClose();
		}

		private void PopulateChanges(string userName, List<DescribeRecord> changes)
		{
			if (!IsDisposed)
			{
				ChangesListView.BeginUpdate();
				ChangesListView.Items.Clear();
				if (changes != null)
				{
					foreach (DescribeRecord change in changes)
					{
						if (change.Description != null && !change.Description.Contains("#ROBOMERGE-SOURCE", StringComparison.Ordinal))
						{
							string stream = "";
							if (change.Files.Count > 0)
							{
								string depotFile = change.Files[0].DepotFile;

								int idx = 0;
								for (int count = 0; idx < depotFile.Length; idx++)
								{
									if (depotFile[idx] == '/' && ++count >= 4)
									{
										break;
									}
								}

								stream = depotFile.Substring(0, idx);
							}

							ListViewItem item = new ListViewItem("");
							item.Tag = change;
							item.SubItems.Add(change.Number.ToString());
							item.SubItems.Add(stream);
							item.SubItems.Add(change.Description.Replace('\n', ' '));
							ChangesListView.Items.Add(item);
						}
					}
				}
				ChangesListView.EndUpdate();
			}
		}

		private void ChangesListView_MouseClick(object sender, MouseEventArgs args)
		{
			if (args.Button == MouseButtons.Right)
			{
				ListViewHitTestInfo hitTest = ChangesListView.HitTest(args.Location);
				if (hitTest.Item != null && hitTest.Item.Tag != null)
				{
					DescribeRecord? record = hitTest.Item.Tag as DescribeRecord;
					if (record != null)
					{
						ChangesListContextMenu.Tag = record;
						ChangesListContextMenu.Show(ChangesListView, args.Location);
						ThemeManager?.ApplyThemeRecursively(ChangesListContextMenu);
					}
				}
			}
		}

		private void ChangesListView_MouseDoubleClick(object sender, MouseEventArgs e)
		{
			UpdateSelectedChangeAndClose();
		}

		private void ChangesContextMenu_MoreInfo_Click(object sender, EventArgs e)
		{
			DescribeRecord record = (DescribeRecord)ChangesListContextMenu.Tag!;
			Program.SpawnP4Vc(String.Format("{0} change {1}", _perforceSettings.GetArgumentsForExternalProgram(true), record.Number));
		}

		private void ChangeNumberTextBox_Enter(object sender, EventArgs e)
		{
			SpecifyChangeRadioButton.Checked = true;
			RecentChangeRadioButton.Checked = false;
		}

		private void ChangesListView_Enter(object sender, EventArgs e)
		{
			SpecifyChangeRadioButton.Checked = false;
			RecentChangeRadioButton.Checked = true;
		}

		private void UserNameTextBox_Enter(object sender, EventArgs e)
		{
			SpecifyChangeRadioButton.Checked = false;
			RecentChangeRadioButton.Checked = true;
		}

		private void UserBrowseBtn_Enter(object sender, EventArgs e)
		{
			SpecifyChangeRadioButton.Checked = false;
			RecentChangeRadioButton.Checked = true;
		}

		private void ChangeNumberTextBox_TextChanged(object sender, EventArgs e)
		{
			UpdateOkButton();
		}

		private void ChangesListView_ItemSelectionChanged(object sender, ListViewItemSelectionChangedEventArgs e)
		{
			UpdateOkButton();
		}

		private bool TryGetSelectedChange(out int changeNumber)
		{
			if (SpecifyChangeRadioButton.Checked)
			{
				return Int32.TryParse(ChangeNumberTextBox.Text, out changeNumber);
			}
			else if (SystemicFixRadioButton.Checked)
			{
				changeNumber = -1;
				return true;
			}
			else
			{
				DescribeRecord? change = (ChangesListView.SelectedItems.Count > 0) ? ChangesListView.SelectedItems[0].Tag as DescribeRecord : null;
				if (change == null)
				{
					changeNumber = 0;
					return false;
				}
				else
				{
					changeNumber = change.Number;
					return true;
				}
			}
		}

		private void UpdateOkButton()
		{
			OkBtn.Enabled = TryGetSelectedChange(out _);
		}

		private void SpecifyChangeRadioButton_CheckedChanged(object sender, EventArgs e)
		{
			if (SpecifyChangeRadioButton.Checked)
			{
				RecentChangeRadioButton.Checked = false;
				SystemicFixRadioButton.Checked = false;
			}
			UpdateOkButton();
		}

		private void RecentChangeRadioButton_CheckedChanged(object sender, EventArgs e)
		{
			if (RecentChangeRadioButton.Checked)
			{
				SpecifyChangeRadioButton.Checked = false;
				SystemicFixRadioButton.Checked = false;
			}
			UpdateOkButton();
		}

		private void SystemicFixRadioButton_CheckedChanged(object sender, EventArgs e)
		{
			if (SystemicFixRadioButton.Checked)
			{
				RecentChangeRadioButton.Checked = false;
				SpecifyChangeRadioButton.Checked = false;
			}
			UpdateOkButton();
		}

		private void UserBrowseBtn_Click(object sender, EventArgs e)
		{
			string? selectedUserName;
			if (SelectUserWindow.ShowModal(this, _perforceSettings, _serviceProvider, out selectedUserName))
			{
				UserNameTextBox.Text = selectedUserName;
			}
		}

		private void UserNameTextBox_TextChanged(object sender, EventArgs e)
		{
			_worker.FetchChanges(UserNameTextBox.Text);
		}
	}
}
