// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Windows.Forms;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	partial class IssueSettingsWindow : ThemedForm
	{
		readonly UserSettings _settings;
		readonly ILogger _logger;

		public IssueSettingsWindow(UserSettings settings, string currentProject, ILogger logger)
		{
			_settings = settings;
			_logger = logger;

			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			if (settings.NotifyProjects.Count == 0)
			{
				NotifyProjectsCheckBox.Checked = false;
				NotifyProjectsTextBox.Text = currentProject;
			}
			else
			{
				NotifyProjectsCheckBox.Checked = true;
				NotifyProjectsTextBox.Text = String.Join(" ", settings.NotifyProjects);
			}

			if (settings.NotifyUnassignedMinutes < 0)
			{
				NotifyUnassignedCheckBox.Checked = false;
				NotifyUnassignedTextBox.Text = "5";
			}
			else
			{
				NotifyUnassignedCheckBox.Checked = true;
				NotifyUnassignedTextBox.Text = settings.NotifyUnassignedMinutes.ToString();
			}

			if (settings.NotifyUnacknowledgedMinutes < 0)
			{
				NotifyUnacknowledgedCheckBox.Checked = false;
				NotifyUnacknowledgedTextBox.Text = "5";
			}
			else
			{
				NotifyUnacknowledgedCheckBox.Checked = true;
				NotifyUnacknowledgedTextBox.Text = settings.NotifyUnacknowledgedMinutes.ToString();
			}

			if (settings.NotifyUnresolvedMinutes < 0)
			{
				NotifyUnresolvedCheckBox.Checked = false;
				NotifyUnresolvedTextBox.Text = "20";
			}
			else
			{
				NotifyUnresolvedCheckBox.Checked = true;
				NotifyUnresolvedTextBox.Text = settings.NotifyUnresolvedMinutes.ToString();
			}

			UpdateEnabledTextBoxes();
		}

		private void UpdateEnabledTextBoxes()
		{
			NotifyProjectsTextBox.Enabled = NotifyProjectsCheckBox.Checked;
			NotifyUnassignedTextBox.Enabled = NotifyUnassignedCheckBox.Checked;
			NotifyUnacknowledgedTextBox.Enabled = NotifyUnacknowledgedCheckBox.Checked;
			NotifyUnresolvedTextBox.Enabled = NotifyUnresolvedCheckBox.Checked;
		}

		private void NotifyProjectsCheckBox_CheckedChanged(object sender, EventArgs e)
		{
			UpdateEnabledTextBoxes();
		}

		private void NotifyUnassignedCheckBox_CheckedChanged(object sender, EventArgs e)
		{
			UpdateEnabledTextBoxes();
		}

		private void NotifyUnacknowledgedCheckBox_CheckedChanged(object sender, EventArgs e)
		{
			UpdateEnabledTextBoxes();
		}

		private void NotifyUnresolvedCheckBox_CheckedChanged(object sender, EventArgs e)
		{
			UpdateEnabledTextBoxes();
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			List<string> newNotifyProjects = new List<string>();
			if (NotifyProjectsCheckBox.Checked)
			{
				newNotifyProjects.AddRange(NotifyProjectsTextBox.Text.Split(new char[] { ' ' }, StringSplitOptions.RemoveEmptyEntries));
			}

			int newNotifyUnresolvedMinutes = -1;
			if (NotifyUnresolvedCheckBox.Checked)
			{
				ushort newNotifyUnresolvedMinutesValue;
				if (!UInt16.TryParse(NotifyUnresolvedTextBox.Text, out newNotifyUnresolvedMinutesValue))
				{
					MessageBox.Show("Invalid time");
					return;
				}
				newNotifyUnresolvedMinutes = newNotifyUnresolvedMinutesValue;
			}

			int newNotifyUnacknowledgedMinutes = -1;
			if (NotifyUnacknowledgedCheckBox.Checked)
			{
				ushort newNotifyUnacknowledgedMinutesValue;
				if (!UInt16.TryParse(NotifyUnacknowledgedTextBox.Text, out newNotifyUnacknowledgedMinutesValue))
				{
					MessageBox.Show("Invalid time");
					return;
				}
				newNotifyUnacknowledgedMinutes = newNotifyUnacknowledgedMinutesValue;
			}

			int newNotifyUnassignedMinutes = -1;
			if (NotifyUnassignedCheckBox.Checked)
			{
				ushort newNotifyUnassignedMinutesValue;
				if (!UInt16.TryParse(NotifyUnassignedTextBox.Text, out newNotifyUnassignedMinutesValue))
				{
					MessageBox.Show("Invalid time");
					return;
				}
				newNotifyUnassignedMinutes = newNotifyUnassignedMinutesValue;
			}

			_settings.NotifyProjects.Clear();
			_settings.NotifyProjects.AddRange(newNotifyProjects);
			_settings.NotifyUnresolvedMinutes = newNotifyUnresolvedMinutes;
			_settings.NotifyUnacknowledgedMinutes = newNotifyUnacknowledgedMinutes;
			_settings.NotifyUnassignedMinutes = newNotifyUnassignedMinutes;
			_settings.Save(_logger);

			DialogResult = DialogResult.OK;
			Close();
		}

		private void CancelBtn_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
			Close();
		}
	}
}
