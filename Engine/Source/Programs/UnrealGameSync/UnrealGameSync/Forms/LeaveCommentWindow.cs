// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealGameSync
{
	public partial class LeaveCommentWindow : ThemedForm
	{
		public LeaveCommentWindow()
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));
		}
	}
}
