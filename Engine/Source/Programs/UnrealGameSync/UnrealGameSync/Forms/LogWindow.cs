// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealGameSync
{
	public partial class LogWindow : ThemedForm
	{
		public LogWindow(string text)
		{
			InitializeComponent();

			LogTextBox.Text = text;
			LogTextBox.Select(text.Length, 0);
		}
	}
}
