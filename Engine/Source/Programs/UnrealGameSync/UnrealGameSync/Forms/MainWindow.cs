// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Drawing;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using UnrealGameSync.Forms;

namespace UnrealGameSync
{
	interface IMainWindowTabPanel : IDisposable
	{
		void Activate();
		void Deactivate();
		void Hide();
		void Show();
		bool IsBusy();
		bool CanClose();
		bool CanSyncNow();
		void SyncLatestChange();
		bool CanLaunchEditor();
		void LaunchEditor();
		void UpdateSettings();

		Color? TintColor
		{
			get;
		}

		Tuple<TaskbarState, float> DesiredTaskbarState
		{
			get;
		}

		UserSelectedProjectSettings SelectedProject
		{
			get;
		}
	}

	partial class MainWindow : ThemedForm, IWorkspaceControlOwner
	{
		class WorkspaceIssueMonitor
		{
			public IssueMonitor _issueMonitor;
			public int _refCount;

			public WorkspaceIssueMonitor(IssueMonitor issueMonitor)
			{
				_issueMonitor = issueMonitor;
			}
		}

		[Flags]
		enum OpenProjectOptions
		{
			None,
			Quiet,
		}

		/// <summary>
		/// Minimum version of the launcher that should be installed. Users will be prompted to upgrade it if the installed version is older than this.
		/// </summary>
		static readonly int s_minLauncherVersion = MakeLauncherVersion(1, 19);

		[DllImport("uxtheme.dll", CharSet = CharSet.Unicode)]
		static extern int SetWindowTheme(IntPtr hWnd, string pszSubAppName, string pszSubIdList);

		[DllImport("user32.dll")]
		public static extern int SendMessage(IntPtr hWnd, int wMsg, int wParam, int lParam);

		private const int WmSetredraw = 11;

		readonly UpdateMonitor _updateMonitor;
		readonly SynchronizationContext _mainThreadSynchronizationContext;
		readonly List<IssueMonitor> _defaultIssueMonitors = new List<IssueMonitor>();
		readonly List<WorkspaceIssueMonitor> _workspaceIssueMonitors = new List<WorkspaceIssueMonitor>();

		readonly string? _apiUrl;
		readonly DirectoryReference _dataFolder;
		readonly IPerforceSettings _defaultPerforceSettings;
		readonly IServiceProvider _serviceProvider;
		readonly ILogger _logger;
		readonly UserSettings _settings;
		int _tabMenuTabIdx = -1;
		int _changingWorkspacesRefCount;

		bool _allowClose = false;

		System.Threading.Timer? _scheduleTimer;
		System.Threading.Timer? _scheduleSettledTimer;

		readonly string _originalExecutableFileName;
		readonly bool _preview;

		IMainWindowTabPanel? _currentTabPanel;

		AutomationServer _automationServer;

		readonly bool _allowCreatingHandle;

		Rectangle _primaryWorkArea;
		readonly List<IssueAlertWindow> _alertWindows = new List<IssueAlertWindow>();

		public ToolUpdateMonitor ToolUpdateMonitor { get; private set; }

		public MainWindow(UpdateMonitor inUpdateMonitor, string? apiUrl, DirectoryReference dataFolder, string originalExecutableFileName, bool preview, List<(UserSelectedProjectSettings, ModalTask<OpenProjectInfo>)> startupTasks, IPerforceSettings defaultPerforceSettings, IServiceProvider serviceProvider, UserSettings settings, string? uri)
		{
			_serviceProvider = serviceProvider;
			_logger = _serviceProvider.GetRequiredService<ILogger<MainWindow>>();

			_logger.LogInformation("Opening Main Window for {NumProject} projects. Last Project {LastProject}", startupTasks.Count, settings.LastProject);

			InitializeComponent();
			Font = new Font("Segoe UI", 8.25F, FontStyle.Regular, GraphicsUnit.Point, ((byte)(0)));

			_updateMonitor = inUpdateMonitor;
			_mainThreadSynchronizationContext = SynchronizationContext.Current!;
			_apiUrl = apiUrl;
			_dataFolder = dataFolder;
			_originalExecutableFileName = originalExecutableFileName;
			_preview = preview;
			_defaultPerforceSettings = defaultPerforceSettings;
			ToolUpdateMonitor = new ToolUpdateMonitor(_defaultPerforceSettings, _dataFolder, settings, _serviceProvider.GetRequiredService<ILogger<ToolUpdateMonitor>>(), _serviceProvider);

			_settings = settings;

			// While creating tab controls during startup, we need to prevent layout calls resulting in the window handle being created too early. Disable layout calls here.
			SuspendLayout();
			TabPanel.SuspendLayout();

			TabControl.OnTabChanged += TabControl_OnTabChanged;
			TabControl.OnNewTabClick += TabControl_OnNewTabClick;
			TabControl.OnTabClicked += TabControl_OnTabClicked;
			TabControl.OnTabClosing += TabControl_OnTabClosing;
			TabControl.OnTabClosed += TabControl_OnTabClosed;
			TabControl.OnTabReorder += TabControl_OnTabReorder;
			TabControl.OnButtonClick += TabControl_OnButtonClick;

			SetupDefaultControl();

			int selectTabIdx = -1;
			foreach ((UserSelectedProjectSettings project, ModalTask<OpenProjectInfo> startupTask) in startupTasks)
			{
				int tabIdx = -1;
				if (startupTask.Succeeded)
				{
					tabIdx = TryOpenProject(startupTask.Result, -1, OpenProjectOptions.Quiet);
				}
				else if (startupTask.Failed)
				{
					_logger.LogError("StartupProject Error: {Message}", startupTask.Error);
					CreateErrorPanel(-1, project, startupTask.Error);
				}

				if (tabIdx != -1 && _settings.LastProject != null && project.Equals(_settings.LastProject))
				{
					selectTabIdx = tabIdx;
				}
			}

			if (selectTabIdx != -1)
			{
				TabControl.SelectTab(selectTabIdx);
			}
			else if (TabControl.GetTabCount() > 0)
			{
				TabControl.SelectTab(0);
			}

			StartScheduleTimer();

			if (_preview)
			{
				Text += $" {Program.GetVersionString()} (UNSTABLE)";
			}

			ILogger<AutomationServer> automationLogger = _serviceProvider.GetRequiredService<ILogger<AutomationServer>>();
			_automationServer = new AutomationServer(request => { _mainThreadSynchronizationContext.Post(obj => PostAutomationRequest(request), null); }, uri, automationLogger);

			// Allow creating controls from now on
			TabPanel.ResumeLayout(false);
			ResumeLayout(false);

			foreach (string defaultIssueApiUrl in DeploymentSettings.Instance.DefaultIssueApiUrls)
			{
				_defaultIssueMonitors.Add(CreateIssueMonitor(defaultIssueApiUrl, defaultPerforceSettings.UserName));
			}

			_allowCreatingHandle = true;

			foreach (WorkspaceIssueMonitor workspaceIssueMonitor in _workspaceIssueMonitors)
			{
				workspaceIssueMonitor._issueMonitor.Start();
			}

			ToolUpdateMonitor.Start();
		}

		void PostAutomationRequest(AutomationRequest request)
		{
			try
			{
				if (!CanFocus)
				{
					request.SetOutput(new AutomationRequestOutput(AutomationRequestResult.Busy));
				}
				else if (request.Input.Type == AutomationRequestType.SyncProject)
				{
					AutomationRequestOutput? output = StartAutomatedSync(request, true);
					if (output != null)
					{
						request.SetOutput(output);
					}
				}
				else if (request.Input.Type == AutomationRequestType.FindProject)
				{
					AutomationRequestOutput output = FindProject(request);
					request.SetOutput(output);
				}
				else if (request.Input.Type == AutomationRequestType.OpenProject)
				{
					AutomationRequestOutput? output = StartAutomatedSync(request, false);
					if (output != null)
					{
						request.SetOutput(output);
					}
				}
				else if (request.Input.Type == AutomationRequestType.ExecCommand)
				{
					AutomationRequestOutput output = StartExecCommand(request);
					request.SetOutput(output);
				}
				else if (request.Input.Type == AutomationRequestType.OpenIssue)
				{
					AutomationRequestOutput output = OpenIssue(request);
					request.SetOutput(new AutomationRequestOutput(AutomationRequestResult.Ok));
				}
				else if (request.Input.Type == AutomationRequestType.RunCustomTool)
				{
					AutomationRequestOutput output = RunCustomTool(request);
					request.SetOutput(new AutomationRequestOutput(AutomationRequestResult.Ok));
				}
				else
				{
					request.SetOutput(new AutomationRequestOutput(AutomationRequestResult.Invalid));
				}
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception running automation request");
				request.SetOutput(new AutomationRequestOutput(AutomationRequestResult.Invalid));
			}
		}

		AutomationRequestOutput StartExecCommand(AutomationRequest request)
		{
			using BinaryReader reader = new BinaryReader(new MemoryStream(request.Input.Data));
			string streamName = reader.ReadString();
			int changelist = reader.ReadInt32();
			string command = reader.ReadString();
			string projectPath = reader.ReadString();

			AutomatedBuildWindow.BuildInfo? buildInfo;
			if (!AutomatedBuildWindow.ShowModal(this, _defaultPerforceSettings, streamName, projectPath, changelist, command.ToString(), _settings, _serviceProvider, out buildInfo))
			{
				return new AutomationRequestOutput(AutomationRequestResult.Canceled);
			}

			WorkspaceControl? workspace;
			if (!OpenWorkspaceForAutomation(buildInfo.SelectedWorkspaceInfo, streamName, buildInfo.ProjectPath, out workspace))
			{
				return new AutomationRequestOutput(AutomationRequestResult.Error);
			}

			workspace.AddStartupCallback((control, cancel) => StartExecCommandAfterStartup(control, cancel, buildInfo.Sync ? changelist : -1, buildInfo.ExecCommand));
			return new AutomationRequestOutput(AutomationRequestResult.Ok);
		}

		private static void StartExecCommandAfterStartup(WorkspaceControl workspace, bool cancel, int changelist, string command)
		{
			if (!cancel)
			{
				if (changelist == -1)
				{
					StartExecCommandAfterSync(workspace, WorkspaceUpdateResult.Success, command);
				}
				else
				{
					// I believe we want to bypass the known CL here
					workspace.SyncChange(changelist, true, result => StartExecCommandAfterSync(workspace, result, command), true);
				}
			}
		}

		private static void StartExecCommandAfterSync(WorkspaceControl workspace, WorkspaceUpdateResult result, string command)
		{
			if (result == WorkspaceUpdateResult.Success && command != null)
			{
				string cmdExe = Environment.GetEnvironmentVariable("COMSPEC") ?? "C:\\Windows\\System32\\cmd.exe";
				workspace.ExecCommand("Run build command", "Running build command", cmdExe, String.Format("/c {0}", command), workspace.BranchDirectoryName.FullName, true);
			}
		}

		AutomationRequestOutput? StartAutomatedSync(AutomationRequest request, bool forceSync)
		{
			ShowAndActivate();

			using BinaryReader reader = new BinaryReader(new MemoryStream(request.Input.Data));
			string streamName = reader.ReadString();
			string projectPath = reader.ReadString();

			AutomatedSyncWindow.WorkspaceInfo? workspaceInfo;
			if (!AutomatedSyncWindow.ShowModal(this, _defaultPerforceSettings, streamName, projectPath, _settings, out workspaceInfo, _serviceProvider))
			{
				return new AutomationRequestOutput(AutomationRequestResult.Canceled);
			}

			WorkspaceControl? workspace;
			if (!OpenWorkspaceForAutomation(workspaceInfo, streamName, projectPath, out workspace))
			{
				return new AutomationRequestOutput(AutomationRequestResult.Error);
			}

			if (!forceSync && workspace.CanLaunchEditor())
			{
				return new AutomationRequestOutput(AutomationRequestResult.Ok, Encoding.UTF8.GetBytes(workspace.SelectedFileName.FullName));
			}

			workspace.AddStartupCallback((control, cancel) => StartAutomatedSyncAfterStartup(control, cancel, request));
			return null;
		}

		private bool OpenWorkspaceForAutomation(AutomatedSyncWindow.WorkspaceInfo workspaceInfo, string streamName, string projectPath, [NotNullWhen(true)] out WorkspaceControl? outWorkspace)
		{
			if (workspaceInfo.RequiresStreamSwitch)
			{
				// Close any tab containing this window
				for (int existingTabIdx = 0; existingTabIdx < TabControl.GetTabCount(); existingTabIdx++)
				{
					WorkspaceControl? existingWorkspace = TabControl.GetTabData(existingTabIdx) as WorkspaceControl;
					if (existingWorkspace != null && existingWorkspace.ClientName.Equals(workspaceInfo.WorkspaceName, StringComparison.OrdinalIgnoreCase))
					{
						TabControl.RemoveTab(existingTabIdx);
						break;
					}
				}

				// Switch the stream
				async Task SwitchTask(IPerforceConnection perforce, CancellationToken cancellationToken)
				{
					await perforce.SwitchClientToStreamAsync(streamName, SwitchClientOptions.None, cancellationToken);
				}

				PerforceSettings settings = new PerforceSettings(workspaceInfo.ServerAndPort, workspaceInfo.UserName);
				settings.ClientName = workspaceInfo.WorkspaceName;

				ModalTask? result = PerforceModalTask.Execute(Owner, "Please wait", "Switching streams, please wait...", settings, SwitchTask, _logger, ModalTaskFlags.Quiet);
				if (result == null || !result.Succeeded)
				{
					_logger.LogError("Unable to switch stream ({Message})", result?.Error ?? "Operation cancelled");
					outWorkspace = null;
					return false;
				}
			}

			UserSelectedProjectSettings selectedProject = new UserSelectedProjectSettings(workspaceInfo.ServerAndPort, workspaceInfo.UserName, UserSelectedProjectType.Client, String.Format("//{0}{1}", workspaceInfo.WorkspaceName, projectPath), null);

			int tabIdx = TryOpenProject(selectedProject, -1, OpenProjectOptions.None);
			if (tabIdx == -1)
			{
				_logger.LogError("Unable to open project");
				outWorkspace = null;
				return false;
			}

			WorkspaceControl? workspace = TabControl.GetTabData(tabIdx) as WorkspaceControl;
			if (workspace == null)
			{
				_logger.LogError("Workspace was unable to open");
				outWorkspace = null;
				return false;
			}

			TabControl.SelectTab(tabIdx);
			outWorkspace = workspace;
			return true;
		}

		private static void StartAutomatedSyncAfterStartup(WorkspaceControl workspace, bool cancel, AutomationRequest request)
		{
			if (cancel)
			{
				request.SetOutput(new AutomationRequestOutput(AutomationRequestResult.Canceled));
			}
			else
			{
				workspace.SyncLatestChange(result => CompleteAutomatedSync(result, workspace.SelectedFileName, request));
			}
		}

		static void CompleteAutomatedSync(WorkspaceUpdateResult result, FileReference selectedFileName, AutomationRequest request)
		{
			if (result == WorkspaceUpdateResult.Success)
			{
				request.SetOutput(new AutomationRequestOutput(AutomationRequestResult.Ok, Encoding.UTF8.GetBytes(selectedFileName.FullName)));
			}
			else if (result == WorkspaceUpdateResult.Canceled)
			{
				request.SetOutput(new AutomationRequestOutput(AutomationRequestResult.Canceled));
			}
			else
			{
				request.SetOutput(new AutomationRequestOutput(AutomationRequestResult.Error));
			}
		}

		AutomationRequestOutput FindProject(AutomationRequest request)
		{
			using BinaryReader reader = new BinaryReader(new MemoryStream(request.Input.Data));
			string streamName = reader.ReadString();
			string projectPath = reader.ReadString();

			for (int existingTabIdx = 0; existingTabIdx < TabControl.GetTabCount(); existingTabIdx++)
			{
				WorkspaceControl? existingWorkspace = TabControl.GetTabData(existingTabIdx) as WorkspaceControl;
				if (existingWorkspace != null && String.Equals(existingWorkspace.StreamName, streamName, StringComparison.OrdinalIgnoreCase) && existingWorkspace.SelectedProject != null)
				{
					string? clientPath = existingWorkspace.SelectedProject.ClientPath;
					if (clientPath != null && clientPath.StartsWith("//", StringComparison.Ordinal))
					{
						int slashIdx = clientPath.IndexOf('/', 2);
						if (slashIdx != -1)
						{
							string existingProjectPath = clientPath.Substring(slashIdx);
							if (String.Equals(existingProjectPath, projectPath, StringComparison.OrdinalIgnoreCase))
							{
								return new AutomationRequestOutput(AutomationRequestResult.Ok, Encoding.UTF8.GetBytes(existingWorkspace.SelectedFileName.FullName));
							}
						}
					}
				}
			}

			return new AutomationRequestOutput(AutomationRequestResult.NotFound);
		}

		AutomationRequestOutput OpenIssue(AutomationRequest request)
		{
			using BinaryReader reader = new BinaryReader(new MemoryStream(request.Input.Data));
			int issueId = reader.ReadInt32();

			for (int existingTabIdx = 0; existingTabIdx < TabControl.GetTabCount(); existingTabIdx++)
			{
				WorkspaceControl? existingWorkspace = TabControl.GetTabData(existingTabIdx) as WorkspaceControl;
				if (existingWorkspace != null)
				{
					IssueMonitor issueMonitor = existingWorkspace.GetIssueMonitor();
					if (issueMonitor != null && (DeploymentSettings.Instance.UrlHandleIssueApi == null || String.Equals(issueMonitor.ApiUrl, DeploymentSettings.Instance.UrlHandleIssueApi, StringComparison.OrdinalIgnoreCase)))
					{
						issueMonitor.AddRef();
						try
						{
							Task<IssueData> Func(CancellationToken cancellationToken) => RestApi.GetAsync<IssueData>($"{issueMonitor.ApiUrl}/api/issues/{issueId}", cancellationToken);

							ModalTask<IssueData>? issueTask = ModalTask.Execute(this, "Finding Issue", "Querying issue data, please wait...", Func);
							if (issueTask == null)
							{
								_logger.LogInformation("Operation cancelled");
								return new AutomationRequestOutput(AutomationRequestResult.Canceled);
							}
							else if (issueTask.Succeeded)
							{
								existingWorkspace.ShowIssueDetails(issueTask.Result);
								return new AutomationRequestOutput(AutomationRequestResult.Ok);
							}
							else
							{
								_logger.LogError(issueTask.Exception, "Unable to query issue {IssueId} from {ApiUrl}: {Error}", issueId, issueMonitor.ApiUrl, issueTask.Error);
								return new AutomationRequestOutput(AutomationRequestResult.Error);
							}
						}
						finally
						{
							issueMonitor.Release();
						}
					}
				}
			}

			return new AutomationRequestOutput(AutomationRequestResult.NotFound);
		}

		AutomationRequestOutput RunCustomTool(AutomationRequest request)
		{
			using BinaryReader reader = new BinaryReader(new MemoryStream(request.Input.Data));
			string toolName = reader.ReadString();
			string command = reader.ReadString();

			for (int existingTabIdx = 0; existingTabIdx < TabControl.GetTabCount(); existingTabIdx++)
			{
				WorkspaceControl? existingWorkspace = TabControl.GetTabData(existingTabIdx) as WorkspaceControl;
				if (existingWorkspace != null)
				{
					// make sure the tool monitor has updated its state
					ToolUpdateMonitor.GetDataFromBackendAsync().Wait();

					if (ToolUpdateMonitor.GetTools().FirstOrDefault(t => String.Equals(t.Name, toolName, StringComparison.OrdinalIgnoreCase)) == null)
					{
						_logger.LogError("Unable to find tool {ToolName} when attempting to process uri request to it", toolName);
						return new AutomationRequestOutput(AutomationRequestResult.Error);
					}

					DirectoryReference? toolPath = ToolUpdateMonitor.GetToolPath(toolName);
					if (toolPath == null)
					{
						_logger.LogError("Unable to find path for {ToolName}", toolName);
						return new AutomationRequestOutput(AutomationRequestResult.Error);
					}

					// this is a async method but we do not wait for it as that blocks UGS
					_ = ToolUpdateMonitor.RunCommandAsync(toolName, command, CancellationToken.None);
					// we have started the tool, no need to check other workspaces for this command
					break; 
				}
			}

			return new AutomationRequestOutput(AutomationRequestResult.NotFound);
		}

		protected override void OnHandleCreated(EventArgs e)
		{
			base.OnHandleCreated(e);

			Debug.Assert(_allowCreatingHandle, "Window handle should not be created before constructor has run.");
		}

		void TabControl_OnButtonClick(int buttonIdx, Point location, MouseButtons buttons)
		{
			if (buttonIdx == 0)
			{
				EditSelectedProject(TabControl.GetSelectedTabIndex());
			}
		}

		void TabControl_OnTabClicked(object? tabData, Point location, MouseButtons buttons)
		{
			if (buttons == System.Windows.Forms.MouseButtons.Right)
			{
				Activate();

				int insertIdx = 0;

				while (TabMenu_RecentProjects.DropDownItems[insertIdx] != TabMenu_Recent_Separator)
				{
					TabMenu_RecentProjects.DropDownItems.RemoveAt(insertIdx);
				}

				_tabMenuTabIdx = -1;
				for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
				{
					if (TabControl.GetTabData(idx) == tabData)
					{
						_tabMenuTabIdx = idx;
						break;
					}
				}

				TabMenu_OpenProject.Enabled = _tabMenuTabIdx != -1;

				foreach (UserSelectedProjectSettings recentProject in _settings.RecentProjects)
				{
#pragma warning disable CA2000 // warning CA2000: Call System.IDisposable.Dispose on object created by 'new ToolStripMenuItem(recentProject.ToString(), null, new EventHandler((o, e) => TabMenu_OpenRecentProject_Click(recentProject, _tabMenuTabIdx)))' before all references to it are out of scope
					ToolStripMenuItem item = new ToolStripMenuItem(recentProject.ToString(), null, new EventHandler((o, e) => TabMenu_OpenRecentProject_Click(recentProject, _tabMenuTabIdx)));
					TabMenu_RecentProjects.DropDownItems.Insert(insertIdx, item);
#pragma warning restore CA2000
					insertIdx++;
				}

				TabMenu_RecentProjects.Visible = (_settings.RecentProjects.Count > 0);

				TabMenu_TabNames_Stream.Checked = _settings.TabLabels == TabLabels.Stream;
				TabMenu_TabNames_WorkspaceName.Checked = _settings.TabLabels == TabLabels.WorkspaceName;
				TabMenu_TabNames_WorkspaceRoot.Checked = _settings.TabLabels == TabLabels.WorkspaceRoot;
				TabMenu_TabNames_ProjectFile.Checked = _settings.TabLabels == TabLabels.ProjectFile;
				TabMenu.Show(TabControl, location);
				ThemeManager?.ApplyThemeRecursively(TabMenu);

				TabControl.LockHover();
			}
		}

		void TabControl_OnTabReorder()
		{
			SaveTabSettings();
		}

		void TabControl_OnTabClosed(object data)
		{
			IMainWindowTabPanel tabPanel = (IMainWindowTabPanel)data;
			if (_currentTabPanel == tabPanel)
			{
				_currentTabPanel = null;
			}
			tabPanel.Dispose();

			SaveTabSettings();
		}

		bool TabControl_OnTabClosing(object tabData)
		{
			IMainWindowTabPanel tabPanel = (IMainWindowTabPanel)tabData;
			return tabPanel.CanClose();
		}

		/// <summary>
		/// Clean up any resources being used.
		/// </summary>
		/// <param name="disposing">true if managed resources should be disposed; otherwise, false.</param>
		protected override void Dispose(bool disposing)
		{
			if (disposing && (components != null))
			{
				components.Dispose();
			}

			for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
			{
				((IMainWindowTabPanel)TabControl.GetTabData(idx)).Dispose();
			}

			StopScheduleTimer();

			foreach (IssueMonitor defaultIssueMonitor in _defaultIssueMonitors)
			{
				ReleaseIssueMonitor(defaultIssueMonitor);
			}
			_defaultIssueMonitors.Clear();
			Debug.Assert(_workspaceIssueMonitors.Count == 0);

			if (_automationServer != null)
			{
				_automationServer.Dispose();
				_automationServer = null!;
			}

			if (ToolUpdateMonitor != null)
			{
				ToolUpdateMonitor.Dispose();
				ToolUpdateMonitor = null!;
			}

			base.Dispose(disposing);
		}

		public bool ConfirmClose()
		{
			for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
			{
				IMainWindowTabPanel tabPanel = (IMainWindowTabPanel)TabControl.GetTabData(idx);
				if (!tabPanel.CanClose())
				{
					return false;
				}
			}
			return true;
		}

		private void MainWindow_FormClosing(object sender, FormClosingEventArgs eventArgs)
		{
			if (!_allowClose && _settings.KeepInTray)
			{
				Hide();
				eventArgs.Cancel = true;
			}
			else
			{
				if (!_allowClose)
				{
					for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
					{
						IMainWindowTabPanel tabPanel = (IMainWindowTabPanel)TabControl.GetTabData(idx)!;
						if (!tabPanel.CanClose())
						{
							eventArgs.Cancel = true;
							return;
						}
					}
				}

				StopScheduleTimer();
			}

			_settings.WindowVisible = Visible;
			_settings.WindowState = WindowState.ToString();
			if (WindowState == FormWindowState.Normal)
			{
				_settings.WindowBounds = new Rectangle(Location, Size);
			}
			else
			{
				_settings.WindowBounds = RestoreBounds;
			}

			_settings.Save(_logger);
		}

		private void SetupDefaultControl()
		{
			List<StatusLine> lines = new List<StatusLine>();

			StatusLine summaryLine = new StatusLine();
			summaryLine.AddText("To get started, open an existing Unreal project file on your hard drive.");
			lines.Add(summaryLine);

			StatusLine openLine = new StatusLine();
			openLine.AddLink("Open project...", FontStyle.Bold | FontStyle.Underline, () => { OpenNewProject(); });
			openLine.AddText("  |  ");
			openLine.AddLink("Application settings...", FontStyle.Bold | FontStyle.Underline, () => { ModifyApplicationSettings(); });
			lines.Add(openLine);

			DefaultControl.Set(lines, Enumerable.Empty<StatusLine>(), null, null, null);
		}

		private void CreateErrorPanel(int replaceTabIdx, UserSelectedProjectSettings project, string message)
		{
			_logger.LogError("{Error}", message ?? "Unknown error");

			ErrorPanel errorPanel = new ErrorPanel(project);
			errorPanel.Parent = TabPanel;
			errorPanel.BorderStyle = BorderStyle.FixedSingle;
			errorPanel.BackColor = Theme.BackgroundColor;
			errorPanel.Location = new Point(0, 0);
			errorPanel.Dock = DockStyle.Fill;
			errorPanel.Hide();

			string summaryText = String.Format("Unable to open '{0}'.", project.ToString());

			int newContentWidth = Math.Max(TextRenderer.MeasureText(summaryText, errorPanel.Font).Width, 400);
			if (!String.IsNullOrEmpty(message))
			{
				newContentWidth = Math.Max(newContentWidth, TextRenderer.MeasureText(message, errorPanel.Font).Width);
			}

			errorPanel.SetContentWidth(newContentWidth);

			List<StatusLine> lines = new List<StatusLine>();

			StatusLine summaryLine = new StatusLine();
			summaryLine.AddText(summaryText);
			lines.Add(summaryLine);

			if (!String.IsNullOrEmpty(message))
			{
				lines.Add(new StatusLine() { LineHeight = 0.5f });

				foreach (string messageLine in message.Split('\n'))
				{
					StatusLine errorLine = new StatusLine();
					errorLine.AddText(messageLine);
					errorLine.LineHeight = 0.8f;
					lines.Add(errorLine);
				}
			}

			lines.Add(new StatusLine() { LineHeight = 0.5f });

			StatusLine actionLine = new StatusLine();
			actionLine.AddLink("Retry", FontStyle.Bold | FontStyle.Underline, () =>
			{
				BeginInvoke(new System.Windows.Forms.MethodInvoker(() => TryOpenProject(project, TabControl.FindTabIndex(errorPanel))));
			});
			actionLine.AddText(" | ");
			actionLine.AddLink("Settings", FontStyle.Bold | FontStyle.Underline, () =>
			{
				BeginInvoke(new System.Windows.Forms.MethodInvoker(() => EditSelectedProject(errorPanel)));
			});
			actionLine.AddText(" | ");
			actionLine.AddLink("Close", FontStyle.Bold | FontStyle.Underline, () =>
			{
				BeginInvoke(new System.Windows.Forms.MethodInvoker(() => TabControl.RemoveTab(TabControl.FindTabIndex(errorPanel))));
			});
			lines.Add(actionLine);

			errorPanel.Set(lines, Enumerable.Empty<StatusLine>(), null, null, null);

			string newProjectName = "Unknown";
			if (project.Type == UserSelectedProjectType.Client && project.ClientPath != null)
			{
				newProjectName = project.ClientPath.Substring(project.ClientPath.LastIndexOf('/') + 1);
			}
			if (project.Type == UserSelectedProjectType.Local && project.LocalPath != null)
			{
				newProjectName = project.LocalPath.Substring(project.LocalPath.LastIndexOfAny(new char[] { '/', '\\' }) + 1);
			}

			string newTabName = String.Format("Error: {0}", newProjectName);
			if (replaceTabIdx == -1)
			{
				int tabIdx = TabControl.InsertTab(-1, newTabName, errorPanel, errorPanel.TintColor);
				TabControl.SelectTab(tabIdx);
			}
			else
			{
				TabControl.InsertTab(replaceTabIdx + 1, newTabName, errorPanel, errorPanel.TintColor);
				TabControl.RemoveTab(replaceTabIdx);
				TabControl.SelectTab(replaceTabIdx);
			}

			UpdateProgress();
		}

		[DllImport("user32.dll")]
		private static extern int ShowWindow(IntPtr hWnd, uint msg);

		private const uint SwRestore = 0x09;

		public void ShowAndActivate()
		{
			if (!IsDisposed)
			{
				Show();
				if (WindowState == FormWindowState.Minimized)
				{
					ShowWindow(Handle, SwRestore);
				}
				Activate();

				_settings.WindowVisible = Visible;
				_settings.Save(_logger);
			}
		}

		public bool CanPerformUpdate()
		{
			if (ContainsFocus || Form.ActiveForm == this || !CanFocus)
			{
				return false;
			}

			for (int tabIdx = 0; tabIdx < TabControl.GetTabCount(); tabIdx++)
			{
				IMainWindowTabPanel tabPanel = (IMainWindowTabPanel)TabControl.GetTabData(tabIdx)!;
				if (tabPanel.IsBusy())
				{
					return false;
				}
			}

			return true;
		}

		public bool CanSyncNow()
		{
			return _currentTabPanel != null && _currentTabPanel.CanSyncNow();
		}

		public bool CanLaunchEditor()
		{
			return _currentTabPanel != null && _currentTabPanel.CanLaunchEditor();
		}

		public void SyncLatestChange()
		{
			_currentTabPanel?.SyncLatestChange();
		}

		public void LaunchEditor()
		{
			_currentTabPanel?.LaunchEditor();
		}

		public void ForceClose()
		{
			_allowClose = true;
			Close();
		}

		private void MainWindow_Activated(object sender, EventArgs e)
		{
			_currentTabPanel?.Activate();
		}

		private void MainWindow_Deactivate(object sender, EventArgs e)
		{
			_currentTabPanel?.Deactivate();
		}

		public void SetupScheduledSync()
		{
			StopScheduleTimer();

			List<UserSelectedProjectSettings> openProjects = new List<UserSelectedProjectSettings>();
			for (int tabIdx = 0; tabIdx < TabControl.GetTabCount(); tabIdx++)
			{
				IMainWindowTabPanel tabPanel = (IMainWindowTabPanel)TabControl.GetTabData(tabIdx);
				openProjects.Add(tabPanel.SelectedProject);
			}

			Dictionary<UserSelectedProjectSettings, List<LatestChangeType>> projectToLatestChangeTypes = new Dictionary<UserSelectedProjectSettings, List<LatestChangeType>>();
			for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
			{
				WorkspaceControl? workspace = TabControl.GetTabData(idx) as WorkspaceControl;
				if (workspace != null)
				{
					projectToLatestChangeTypes.Add(workspace.SelectedProject, workspace.GetCustomLatestChangeTypes());
				}
			}

			using ScheduleWindow schedule = new ScheduleWindow(_settings.ScheduleEnabled, _settings.ScheduleTime, _settings.ScheduleAnyOpenProject, _settings.ScheduleProjects, openProjects, projectToLatestChangeTypes);
			if (schedule.ShowDialog() == System.Windows.Forms.DialogResult.OK)
			{
				schedule.CopySettings(_settings);
				_settings.Save(_logger);
			}

			StartScheduleTimer();
		}

		private void StartScheduleTimer()
		{
			StopScheduleTimer();

			if (_settings.ScheduleEnabled)
			{
				DateTime currentTime = DateTime.Now;
				Random rnd = new Random();

				// add or subtract from the schedule time to distribute scheduled syncs over a little bit more time
				// this avoids everyone hitting the p4 server at exactly the same time.
				const int FudgeMinutes = 10;
				TimeSpan fudgeTime = TimeSpan.FromMinutes(rnd.Next(FudgeMinutes * -100, FudgeMinutes * 100) / 100.0);
				DateTime nextScheduleTime = new DateTime(currentTime.Year, currentTime.Month, currentTime.Day, _settings.ScheduleTime.Hours, _settings.ScheduleTime.Minutes, _settings.ScheduleTime.Seconds);
				nextScheduleTime += fudgeTime;

				if (nextScheduleTime < currentTime)
				{
					nextScheduleTime = nextScheduleTime.AddDays(1.0);
				}

				TimeSpan intervalToFirstTick = nextScheduleTime - currentTime;
				_scheduleTimer = new System.Threading.Timer(x => _mainThreadSynchronizationContext.Post((o) =>
				{
					if (!IsDisposed)
					{
						ScheduleTimerElapsed();
					}
				}, null), null, intervalToFirstTick, TimeSpan.FromDays(1));

				_logger.LogInformation("Schedule: Started ScheduleTimer for {Time} ({Time} remaining)", nextScheduleTime, intervalToFirstTick);
			}
		}

		private void CheckLauncherVersionTimer_Tick(object sender, EventArgs e)
		{
			if (!CanPerformUpdate())
			{
				return;
			}

			CheckLauncherVersionTimer.Enabled = false;

			FileReference executableFile = new FileReference(Assembly.GetExecutingAssembly()!.Location);
			FileReference installerFile = FileReference.Combine(executableFile.Directory, "UnrealGameSync.msi");
			if (!FileReference.Exists(installerFile))
			{
				return;
			}

			DateTime now = DateTime.UtcNow;
			if (now.Ticks < _settings.NextLauncherVersionCheck)
			{
				return;
			}

			int version = GetLauncherVersion();
			if (version >= s_minLauncherVersion)
			{
				return;
			}

			using (LauncherUpdateWindow update = new LauncherUpdateWindow())
			{
				if (update.ShowDialog(this) == DialogResult.Ignore)
				{
					_settings.NextLauncherVersionCheck = (now + TimeSpan.FromDays(1.0)).Ticks;
					_settings.Save(_logger);

					CheckLauncherVersionTimer.Enabled = true;
					return;
				}
			}

			using (Process childProcess = new Process())
			{
				childProcess.StartInfo.FileName = installerFile.FullName;
				childProcess.StartInfo.UseShellExecute = true;
				childProcess.Start();
			}

			ForceClose();
		}

		static int GetLauncherVersion()
		{
			try
			{
				DirectoryReference? programFiles = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.ProgramFilesX86);
				if (programFiles == null)
				{
					return 0;
				}

				FileReference launcherFile = FileReference.Combine(programFiles, "UnrealGameSync", "UnrealGameSyncLauncher.exe");
				if (!FileReference.Exists(launcherFile))
				{
					return 0;
				}

				FileVersionInfo versionInfo = FileVersionInfo.GetVersionInfo(launcherFile.FullName);
				return MakeLauncherVersion(versionInfo.FileMajorPart, versionInfo.FileMinorPart);
			}
			catch (Exception ex)
			{
				Program.CaptureException(ex);
				return 0;
			}
		}

		static int MakeLauncherVersion(int major, int minor) => (major << 16) + minor;

		private void StopScheduleTimer()
		{
			if (_scheduleTimer != null)
			{
				_scheduleTimer.Dispose();
				_scheduleTimer = null;
				_logger.LogInformation("Schedule: Stopped ScheduleTimer");
			}
			StopScheduleSettledTimer();
		}

		private void ScheduleTimerElapsed()
		{
			_logger.LogInformation("Schedule: Timer Elapsed");

			// Try to open any missing tabs. 
			int numInitialTabs = TabControl.GetTabCount();
			foreach (UserSelectedProjectSettings scheduledProject in _settings.ScheduleProjects)
			{
				_logger.LogInformation("Schedule: Attempting to open {Project}", scheduledProject);
				TryOpenProject(scheduledProject, -1, OpenProjectOptions.Quiet);
			}

			// If we did open something, leave it for a while to populate with data before trying to start the sync.
			if (TabControl.GetTabCount() > numInitialTabs)
			{
				StartScheduleSettledTimer();
			}
			else
			{
				ScheduleSettledTimerElapsed();
			}
		}

		private void StartScheduleSettledTimer()
		{
			StopScheduleSettledTimer();
			_scheduleSettledTimer = new System.Threading.Timer(x => _mainThreadSynchronizationContext.Post((o) =>
			{
				if (!IsDisposed)
				{
					ScheduleSettledTimerElapsed();
				}
			}, null), null, TimeSpan.FromSeconds(20.0), TimeSpan.FromMilliseconds(-1.0));
			_logger.LogInformation("Schedule: Started ScheduleSettledTimer");
		}

		private void StopScheduleSettledTimer()
		{
			if (_scheduleSettledTimer != null)
			{
				_scheduleSettledTimer.Dispose();
				_scheduleSettledTimer = null;

				_logger.LogInformation("Schedule: Stopped ScheduleSettledTimer");
			}
		}

		private void ScheduleSettledTimerElapsed()
		{
			_logger.LogInformation("Schedule: Starting Sync");
			for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
			{
				WorkspaceControl? workspace = TabControl.GetTabData(idx) as WorkspaceControl;
				if (workspace != null)
				{
					_logger.LogInformation("Schedule: Considering {File}", workspace.SelectedFileName);
					if (_settings.ScheduleAnyOpenProject || _settings.ScheduleProjects.Any(x => x.LocalPath != null && x.LocalPath.Equals(workspace.SelectedProject.LocalPath, StringComparison.OrdinalIgnoreCase)))
					{
						_logger.LogInformation("Schedule: Starting Sync");
						workspace.ScheduleTimerElapsed();
					}
				}
			}
		}

		void TabControl_OnTabChanged(object? newTabData)
		{
			if (IsHandleCreated)
			{
				SendMessage(Handle, WmSetredraw, 0, 0);
			}

			SuspendLayout();

			if (_currentTabPanel != null)
			{
				_currentTabPanel.Deactivate();
				_currentTabPanel.Hide();
			}

			if (newTabData == null)
			{
				_currentTabPanel = null;
				_settings.LastProject = null;
				DefaultControl.Show();
			}
			else
			{
				_currentTabPanel = (IMainWindowTabPanel)newTabData;
				_settings.LastProject = _currentTabPanel.SelectedProject;
				DefaultControl.Hide();
			}

			_settings.Save(_logger);

			if (_currentTabPanel != null)
			{
				_currentTabPanel.Activate();
				_currentTabPanel.Show();
			}

			ResumeLayout();

			if (IsHandleCreated)
			{
				SendMessage(Handle, WmSetredraw, 1, 0);
			}

			Refresh();
		}

		public void RequestProjectChange(WorkspaceControl workspace, UserSelectedProjectSettings project, bool modal)
		{
			int tabIdx = TabControl.FindTabIndex(workspace);
			if (tabIdx != -1 && !workspace.IsBusy() && CanFocus)
			{
				if (modal)
				{
					TryOpenProject(project, tabIdx);
				}
				else
				{
					TryOpenProject(project, tabIdx, OpenProjectOptions.Quiet);
				}
			}
		}

		public void OpenNewProject()
		{
			OpenProjectInfo? openProjectInfo = OpenProjectWindow.ShowModal(this, null, _settings, _defaultPerforceSettings, _serviceProvider, _logger);
			if (openProjectInfo != null)
			{
				int newTabIdx = TryOpenProject(openProjectInfo, -1, OpenProjectOptions.None);
				if (newTabIdx != -1)
				{
					TabControl.SelectTab(newTabIdx);
					SaveTabSettings();
					UpdateRecentProjectsList(openProjectInfo.SelectedProject);
					ThemeManager?.ApplyThemeRecursively(this);
				}
			}
		}

		void UpdateRecentProjectsList(UserSelectedProjectSettings detectedProjectSettings)
		{
			_settings.RecentProjects.RemoveAll(x => x.LocalPath != null && x.LocalPath.Equals(detectedProjectSettings.LocalPath, StringComparison.OrdinalIgnoreCase));
			_settings.RecentProjects.Insert(0, detectedProjectSettings);

			const int MaxRecentProjects = 10;
			if (_settings.RecentProjects.Count > MaxRecentProjects)
			{
				_settings.RecentProjects.RemoveRange(MaxRecentProjects, _settings.RecentProjects.Count - MaxRecentProjects);
			}

			_settings.Save(_logger);
		}

		public void EditSelectedProject(int tabIdx)
		{
			object tabData = TabControl.GetTabData(tabIdx);
			if (tabData is WorkspaceControl workspace)
			{
				EditSelectedProject(tabIdx, workspace.SelectedProject);
			}
			else if (tabData is ErrorPanel error)
			{
				EditSelectedProject(tabIdx, error.SelectedProject);
			}
		}

		public void EditSelectedProject(WorkspaceControl workspace)
		{
			int tabIdx = TabControl.FindTabIndex(workspace);
			if (tabIdx != -1)
			{
				EditSelectedProject(tabIdx, workspace.SelectedProject);
			}
		}

		public void EditSelectedProject(ErrorPanel panel)
		{
			int tabIdx = TabControl.FindTabIndex(panel);
			if (tabIdx != -1)
			{
				EditSelectedProject(tabIdx, panel.SelectedProject);
			}
		}

		public void EditSelectedProject(int tabIdx, UserSelectedProjectSettings selectedProject)
		{
			OpenProjectInfo? openProjectInfo = OpenProjectWindow.ShowModal(this, selectedProject, _settings, _defaultPerforceSettings, _serviceProvider, _logger);
			if (openProjectInfo != null)
			{
				int newTabIdx = TryOpenProject(openProjectInfo, tabIdx, OpenProjectOptions.None);
				if (newTabIdx != -1)
				{
					TabControl.SelectTab(newTabIdx);
					SaveTabSettings();
					UpdateRecentProjectsList(openProjectInfo.SelectedProject);
				}
			}
		}

		int TryOpenProject(UserSelectedProjectSettings project, int replaceTabIdx, OpenProjectOptions options = OpenProjectOptions.None)
		{
			ILogger<OpenProjectInfo> projectLogger = _serviceProvider.GetRequiredService<ILogger<OpenProjectInfo>>();

			ModalTaskFlags taskFlags = ModalTaskFlags.None;
			if ((options & OpenProjectOptions.Quiet) != 0)
			{
				taskFlags |= ModalTaskFlags.Quiet;
			}

			PerforceSettings perforceSettings = Utility.OverridePerforceSettings(_defaultPerforceSettings, project.ServerAndPort, project.UserName);

			ModalTask<OpenProjectInfo>? settingsTask = PerforceModalTask.Execute(this, "Opening Project", "Opening project, please wait...", perforceSettings, (p, c) => OpenProjectWindow.DetectSettingsAsync(p, project, _settings, false, projectLogger, c), projectLogger, taskFlags);
			if (settingsTask == null || settingsTask.Failed)
			{
				if (settingsTask != null)
				{
					CreateErrorPanel(replaceTabIdx, project, settingsTask.Error);
				}
				return -1;
			}

			return TryOpenProject(settingsTask.Result, replaceTabIdx, options);
		}

		int TryOpenProject(OpenProjectInfo openProjectInfo, int replaceTabIdx, OpenProjectOptions options)
		{
			_logger.LogInformation("Trying to open project {Project}", openProjectInfo.ProjectInfo.ClientFileName);

			// Check that none of the other tabs already have it open
			for (int tabIdx = 0; tabIdx < TabControl.GetTabCount(); tabIdx++)
			{
				if (replaceTabIdx != tabIdx)
				{
					WorkspaceControl? workspace = TabControl.GetTabData(tabIdx) as WorkspaceControl;
					if (workspace != null)
					{
						if (workspace.SelectedFileName == openProjectInfo.ProjectInfo.LocalFileName)
						{
							_logger.LogInformation("  Already open in tab {TabIdx}", tabIdx);
							if ((options & OpenProjectOptions.Quiet) == 0)
							{
								TabControl.SelectTab(tabIdx);
							}
							return tabIdx;
						}
						else if (openProjectInfo.ProjectInfo.LocalFileName.IsUnderDirectory(workspace.BranchDirectoryName))
						{
							if ((options & OpenProjectOptions.Quiet) == 0 && MessageBox.Show($"{workspace.SelectedFileName.GetFileNameWithoutExtension()} is already open under {workspace.BranchDirectoryName}.\n\nWould you like to close it?", "Branch already open", MessageBoxButtons.YesNo) == System.Windows.Forms.DialogResult.Yes)
							{
								_logger.LogInformation("  Another project already open in this workspace, tab {TabIdx}. Replacing.", tabIdx);
								TabControl.RemoveTab(tabIdx);
							}
							else
							{
								_logger.LogInformation("  Another project already open in this workspace, tab {TabIdx}. Aborting.", tabIdx);
								return -1;
							}
						}
					}
				}
			}

			// Hide the default control if it's visible
			DefaultControl.Hide();

			// Remove the current tab. We need to ensure the workspace has been shut down before creating a new one with the same log files, etc...
			if (replaceTabIdx != -1)
			{
				WorkspaceControl? oldWorkspace = TabControl.GetTabData(replaceTabIdx) as WorkspaceControl;
				if (oldWorkspace != null)
				{
					oldWorkspace.Hide();
					TabControl.SetTabData(replaceTabIdx, new ErrorPanel(openProjectInfo.SelectedProject));
					oldWorkspace.Dispose();
				}
			}

			// Now that we have the project settings, we can construct the tab
			WorkspaceControl newWorkspace = new WorkspaceControl(this, _dataFolder, _apiUrl, openProjectInfo, _serviceProvider, _settings);

			newWorkspace.Parent = TabPanel;
			newWorkspace.Dock = DockStyle.Fill;
			newWorkspace.Hide();

			// Add the tab
			string newTabName = GetTabName(newWorkspace);
			if (replaceTabIdx == -1)
			{
				int newTabIdx = TabControl.InsertTab(-1, newTabName, newWorkspace, newWorkspace.TintColor);
				ThemeManager?.ApplyThemeRecursively(newWorkspace);
				_logger.LogInformation("  Inserted tab {TabIdx}", newTabIdx);
				return newTabIdx;
			}
			else
			{
				_logger.LogInformation("  Replacing tab {TabIdx}", replaceTabIdx);
				TabControl.InsertTab(replaceTabIdx + 1, newTabName, newWorkspace, newWorkspace.TintColor);
				ThemeManager?.ApplyThemeRecursively(newWorkspace);
				TabControl.RemoveTab(replaceTabIdx);
				return replaceTabIdx;
			}
		}

		public void StreamChanged(WorkspaceControl workspace)
		{
			_mainThreadSynchronizationContext.Post((o) =>
			{
				if (!IsDisposed)
				{
					StreamChangedCallback(workspace);
				}
			}, null);
		}

		public void StreamChangedCallback(WorkspaceControl workspace)
		{
			if (_changingWorkspacesRefCount == 0)
			{
				_changingWorkspacesRefCount++;

				for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
				{
					if (TabControl.GetTabData(idx) == workspace)
					{
						UserSelectedProjectSettings project = workspace.SelectedProject;
						if (TryOpenProject(project, idx) == -1)
						{
							TabControl.RemoveTab(idx);
						}
						break;
					}
				}

				_changingWorkspacesRefCount--;
			}
		}

		void SaveTabSettings()
		{
			_settings.OpenProjects.Clear();
			for (int tabIdx = 0; tabIdx < TabControl.GetTabCount(); tabIdx++)
			{
				IMainWindowTabPanel tabPanel = (IMainWindowTabPanel)TabControl.GetTabData(tabIdx)!;
				_settings.OpenProjects.Add(tabPanel.SelectedProject);
			}
			_settings.Save(_logger);
		}

		void TabControl_OnNewTabClick(Point location, MouseButtons buttons)
		{
			if (buttons == MouseButtons.Left)
			{
				OpenNewProject();
			}
		}

		string GetTabName(WorkspaceControl workspace)
		{
			string tabName = "";
			switch (_settings.TabLabels)
			{
				case TabLabels.Stream:
					tabName = workspace.StreamName ?? tabName;
					break;
				case TabLabels.ProjectFile:
					tabName = workspace.SelectedFileName.FullName;
					break;
				case TabLabels.WorkspaceName:
					tabName = workspace.ClientName;
					break;
				case TabLabels.WorkspaceRoot:
					tabName = workspace.BranchDirectoryName.FullName;
					break;
				default:
					break;
			}

			// if this failes, return something sensible to avoid blank tabs
			if (String.IsNullOrEmpty(tabName))
			{
				_logger.LogInformation("No TabName for {ClientName} for setting {TabSetting}. Defaulting to client name", workspace.ClientName, _settings.TabLabels.ToString());
				tabName = workspace.ClientName;
			}

			return tabName;
		}

		public void SetTabNames(TabLabels newTabNames)
		{
			if (_settings.TabLabels != newTabNames)
			{
				_settings.TabLabels = newTabNames;
				_settings.Save(_logger);

				for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
				{
					WorkspaceControl? workspace = TabControl.GetTabData(idx) as WorkspaceControl;
					if (workspace != null)
					{
						TabControl.SetTabName(idx, GetTabName(workspace));
					}
				}
			}
		}

		private void TabMenu_OpenProject_Click(object sender, EventArgs e)
		{
			EditSelectedProject(_tabMenuTabIdx);
		}

		private void TabMenu_OpenRecentProject_Click(UserSelectedProjectSettings recentProject, int tabIdx)
		{
			TryOpenProject(recentProject, tabIdx);
			SaveTabSettings();
		}

		private void TabMenu_TabNames_Stream_Click(object sender, EventArgs e)
		{
			SetTabNames(TabLabels.Stream);
		}

		private void TabMenu_TabNames_WorkspaceName_Click(object sender, EventArgs e)
		{
			SetTabNames(TabLabels.WorkspaceName);
		}

		private void TabMenu_TabNames_WorkspaceRoot_Click(object sender, EventArgs e)
		{
			SetTabNames(TabLabels.WorkspaceRoot);
		}

		private void TabMenu_TabNames_ProjectFile_Click(object sender, EventArgs e)
		{
			SetTabNames(TabLabels.ProjectFile);
		}

		private void TabMenu_RecentProjects_ClearList_Click(object sender, EventArgs e)
		{
			_settings.RecentProjects.Clear();
			_settings.Save(_logger);
		}

		private void TabMenu_Closed(object sender, ToolStripDropDownClosedEventArgs e)
		{
			TabControl.UnlockHover();
		}

		private void RecentMenu_ClearList_Click(object sender, EventArgs e)
		{
			_settings.RecentProjects.Clear();
			_settings.Save(_logger);
		}

		public void UpdateProgress()
		{
			TaskbarState state = TaskbarState.NoProgress;
			float progress = -1.0f;

			for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
			{
				IMainWindowTabPanel tabPanel = (IMainWindowTabPanel)TabControl.GetTabData(idx)!;

				Tuple<TaskbarState, float> desiredTaskbarState = tabPanel.DesiredTaskbarState;
				if (desiredTaskbarState.Item1 == TaskbarState.Error)
				{
					state = TaskbarState.Error;
					TabControl.SetHighlight(idx, Tuple.Create(Theme.ErrorColor, 1.0f));
				}
				else if (desiredTaskbarState.Item1 == TaskbarState.Paused && state != TaskbarState.Error)
				{
					state = TaskbarState.Paused;
					TabControl.SetHighlight(idx, Tuple.Create(Theme.WarningColor, 1.0f));
				}
				else if (desiredTaskbarState.Item1 == TaskbarState.Normal && state != TaskbarState.Error && state != TaskbarState.Paused)
				{
					state = TaskbarState.Normal;
					progress = Math.Max(progress, desiredTaskbarState.Item2);
					TabControl.SetHighlight(idx, Tuple.Create(Theme.SuccessColor, desiredTaskbarState.Item2));
				}
				else
				{
					TabControl.SetHighlight(idx, null);
				}
			}

			if (IsHandleCreated)
			{
				if (state == TaskbarState.Normal)
				{
					Taskbar.SetState(Handle, TaskbarState.Normal);
					Taskbar.SetProgress(Handle, (ulong)(progress * 1000.0f), 1000);
				}
				else
				{
					Taskbar.SetState(Handle, state);
				}
			}
		}

		public void UpdateTintColors()
		{
			for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
			{
				IMainWindowTabPanel tabPanel = (IMainWindowTabPanel)TabControl.GetTabData(idx)!;
				TabControl.SetTint(idx, tabPanel.TintColor);
			}
		}

		public void ModifyApplicationSettings()
		{
			ApplicationSettingsWindow.Result result = ApplicationSettingsWindow.ShowModal(this, _defaultPerforceSettings, _originalExecutableFileName, _settings, ToolUpdateMonitor, _serviceProvider.GetRequiredService<ILogger<ApplicationSettingsWindow>>());
			if (result == ApplicationSettingsWindow.Result.Restart)
			{
				_updateMonitor.TriggerUpdate(UpdateType.UserInitiated, false);
			}
			else if (result == ApplicationSettingsWindow.Result.RestartAndConfigureUpdate)
			{
				_updateMonitor.TriggerUpdate(UpdateType.UserInitiated, true);
			}
			else if (result == ApplicationSettingsWindow.Result.Quit)
			{
				ForceClose();
			}

			for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
			{
				IMainWindowTabPanel tabPanel = (IMainWindowTabPanel)TabControl.GetTabData(idx)!;
				tabPanel.UpdateSettings();
			}
		}

		private void MainWindow_Load(object sender, EventArgs e)
		{
			if (_settings.WindowBounds != null)
			{
				Rectangle windowBounds = _settings.WindowBounds.Value;
				if (windowBounds.Width > MinimumSize.Width && windowBounds.Height > MinimumSize.Height)
				{
					foreach (Screen screen in Screen.AllScreens)
					{
						if (windowBounds.IntersectsWith(screen.Bounds))
						{
							Location = _settings.WindowBounds.Value.Location;
							Size = _settings.WindowBounds.Value.Size;
							break;
						}
					}
				}
			}

			FormWindowState newWindowState;
			if (Enum.TryParse(_settings.WindowState, true, out newWindowState))
			{
				WindowState = newWindowState;
			}
		}

		bool ShowNotificationForIssue(IssueData issue)
		{
			return issue.Projects.Any(x => ShowNotificationsForProject(x));
		}

		bool ShowNotificationsForProject(string project)
		{
			return String.IsNullOrEmpty(project) || _settings.NotifyProjects.Count == 0 || _settings.NotifyProjects.Any(x => x.Equals(project, StringComparison.OrdinalIgnoreCase));
		}

		public void UpdateAlertWindows()
		{
			if (!DeploymentSettings.Instance.EnableAlerts)
			{
				return;
			}

			HashSet<IssueData> allIssues = new HashSet<IssueData>();
			foreach (IssueMonitor issueMonitor in _workspaceIssueMonitors.Select(x => x._issueMonitor))
			{
				List<IssueData> issues = issueMonitor.GetIssues();
				foreach (IssueData issue in issues)
				{
					IssueAlertReason reason = 0;
					if (issue.FixChange == 0 && !issue.ResolvedAt.HasValue)
					{
						if (issue.Owner == null)
						{
							if (issue.Notify)
							{
								reason |= IssueAlertReason.Normal;
							}
							if (ShowNotificationForIssue(issue) && _settings.NotifyUnassignedMinutes >= 0 && issue.RetrievedAt - issue.CreatedAt >= TimeSpan.FromMinutes(_settings.NotifyUnassignedMinutes))
							{
								reason |= IssueAlertReason.UnassignedTimer;
							}
						}
						else if (!issue.AcknowledgedAt.HasValue)
						{
							if (String.Equals(issue.Owner, issueMonitor.UserName, StringComparison.OrdinalIgnoreCase))
							{
								reason |= IssueAlertReason.Owner;
							}
							else if (ShowNotificationForIssue(issue) && _settings.NotifyUnacknowledgedMinutes >= 0 && issue.RetrievedAt - issue.CreatedAt >= TimeSpan.FromMinutes(_settings.NotifyUnacknowledgedMinutes))
							{
								reason |= IssueAlertReason.UnacknowledgedTimer;
							}
						}
						if (ShowNotificationForIssue(issue) && _settings.NotifyUnresolvedMinutes >= 0 && issue.RetrievedAt - issue.CreatedAt >= TimeSpan.FromMinutes(_settings.NotifyUnresolvedMinutes))
						{
							reason |= IssueAlertReason.UnresolvedTimer;
						}

						IssueAlertReason prevReason;
						if (issueMonitor.IssueIdToAlertReason.TryGetValue(issue.Id, out prevReason))
						{
							reason &= ~prevReason;
						}
					}

					IssueAlertWindow? alertWindow = _alertWindows.FirstOrDefault(x => x.IssueMonitor == issueMonitor && x.Issue.Id == issue.Id);
					if (alertWindow == null)
					{
						if (reason != 0)
						{
							ShowAlertWindow(issueMonitor, issue, reason);
						}
					}
					else
					{
						if (reason != 0)
						{
							alertWindow.SetIssue(issue, reason);
						}
						else
						{
							CloseAlertWindow(alertWindow);
						}
					}
				}
				allIssues.UnionWith(issues);
			}

			// Close any alert windows which don't have an active issues
			for (int idx = 0; idx < _alertWindows.Count; idx++)
			{
				IssueAlertWindow alertWindow = _alertWindows[idx];
				if (!allIssues.Contains(alertWindow.Issue))
				{
					alertWindow.IssueMonitor.IssueIdToAlertReason.Remove(alertWindow.Issue.Id);
					CloseAlertWindow(alertWindow);
					idx--;
				}
			}
		}

		void IssueMonitor_OnUpdateAsync()
		{
			_mainThreadSynchronizationContext.Post((o) => IssueMonitor_OnUpdate(), null);
		}

		void IssueMonitor_OnUpdate()
		{
			UpdateAlertWindows();
		}

		void ShowAlertWindow(IssueMonitor issueMonitor, IssueData issue, IssueAlertReason reason)
		{
			IssueAlertWindow alert = new IssueAlertWindow(issueMonitor, issue, reason);
			alert.AcceptBtn.Click += (s, e) => AcceptIssue(alert);
			alert.DeclineBtn.Click += (s, e) => DeclineIssue(alert);
			alert.DetailsBtn.Click += (s, e) => ShowIssueDetails(alert);

			SetAlertWindowPositions();
			_alertWindows.Add(alert);
			SetAlertWindowPosition(_alertWindows.Count - 1);

			alert.Show(this);

			UpdateAlertPositionsTimer.Enabled = true;
		}

		void AcceptIssue(IssueAlertWindow alert)
		{
			IssueData issue = alert.Issue;

			IssueAlertReason reason;
			alert.IssueMonitor.IssueIdToAlertReason.TryGetValue(issue.Id, out reason);
			alert.IssueMonitor.IssueIdToAlertReason[issue.Id] = reason | alert.Reason;

			IssueUpdateData update = new IssueUpdateData();
			update.Id = issue.Id;
			update.Owner = alert.IssueMonitor.UserName;
			update.NominatedBy = null;
			update.Acknowledged = true;
			alert.IssueMonitor.PostUpdate(update);

			CloseAlertWindow(alert);
		}

		void DeclineIssue(IssueAlertWindow alert)
		{
			IssueData issue = alert.Issue;

			IssueAlertReason reason;
			alert.IssueMonitor.IssueIdToAlertReason.TryGetValue(issue.Id, out reason);
			alert.IssueMonitor.IssueIdToAlertReason[issue.Id] = reason | alert.Reason;

			CloseAlertWindow(alert);
		}

		void ShowIssueDetails(IssueAlertWindow alert)
		{
			for (int idx = 0; idx < TabControl.GetTabCount(); idx++)
			{
				WorkspaceControl? workspace = TabControl.GetTabData(idx) as WorkspaceControl;
				if (workspace != null && workspace.GetIssueMonitor() == alert.IssueMonitor)
				{
					workspace.ShowIssueDetails(alert.Issue);
					break;
				}
			}
		}

		void CloseAlertWindow(IssueAlertWindow alert)
		{
			alert.Close();
			alert.Dispose();

			_alertWindows.Remove(alert);

			for (int idx = 0; idx < _alertWindows.Count; idx++)
			{
				SetAlertWindowPosition(idx);
			}

			if (_alertWindows.Count == 0)
			{
				UpdateAlertPositionsTimer.Enabled = false;
			}
		}

		private void SetAlertWindowPosition(int idx)
		{
			_alertWindows[idx].Location = new Point(_primaryWorkArea.Right - 40 - _alertWindows[idx].Size.Width, _primaryWorkArea.Height - 40 - (idx + 1) * (_alertWindows[idx].Size.Height + 15));
		}

		private void SetAlertWindowPositions()
		{
			if (Screen.PrimaryScreen != null)
			{
				Rectangle newPrimaryWorkArea = Screen.PrimaryScreen.WorkingArea;
				if (newPrimaryWorkArea != _primaryWorkArea)
				{
					_primaryWorkArea = newPrimaryWorkArea;
					for (int idx = 0; idx < _alertWindows.Count; idx++)
					{
						SetAlertWindowPosition(idx);
					}
				}
			}
		}

		private void UpdateAlertPositionsTimer_Tick(object sender, EventArgs e)
		{
			SetAlertWindowPositions();
		}

		public IssueMonitor CreateIssueMonitor(string? apiUrl, string userName)
		{
			WorkspaceIssueMonitor? workspaceIssueMonitor = _workspaceIssueMonitors.FirstOrDefault(x => String.Equals(x._issueMonitor.ApiUrl, apiUrl, StringComparison.OrdinalIgnoreCase) && String.Equals(x._issueMonitor.UserName, userName, StringComparison.OrdinalIgnoreCase));
			if (workspaceIssueMonitor == null)
			{
				string serverId = apiUrl != null ? Regex.Replace(apiUrl, @"^.*://", "") : "noserver";
				serverId = Regex.Replace(serverId, "[^a-zA-Z.]", "+");

				FileReference logFileName = FileReference.Combine(_dataFolder, String.Format("IssueMonitor-{0}-{1}.log", serverId, userName));

				IssueMonitor issueMonitor = new IssueMonitor(apiUrl, userName, TimeSpan.FromSeconds(60.0), _serviceProvider);
				issueMonitor.OnIssuesChanged += IssueMonitor_OnUpdateAsync;

				workspaceIssueMonitor = new WorkspaceIssueMonitor(issueMonitor);
				_workspaceIssueMonitors.Add(workspaceIssueMonitor);
			}

			workspaceIssueMonitor._refCount++;

			if (workspaceIssueMonitor._refCount == 1 && _allowCreatingHandle)
			{
				workspaceIssueMonitor._issueMonitor.Start();
			}

			return workspaceIssueMonitor._issueMonitor;
		}

		public void ReleaseIssueMonitor(IssueMonitor issueMonitor)
		{
			int index = _workspaceIssueMonitors.FindIndex(x => x._issueMonitor == issueMonitor);
			if (index != -1)
			{
				WorkspaceIssueMonitor workspaceIssueMonitor = _workspaceIssueMonitors[index];
				workspaceIssueMonitor._refCount--;

				if (workspaceIssueMonitor._refCount == 0)
				{
					for (int idx = _alertWindows.Count - 1; idx >= 0; idx--)
					{
						IssueAlertWindow alertWindow = _alertWindows[idx];
						if (alertWindow.IssueMonitor == issueMonitor)
						{
							CloseAlertWindow(alertWindow);
						}
					}
					issueMonitor.OnIssuesChanged -= IssueMonitor_OnUpdateAsync;
					issueMonitor.Release();

					_workspaceIssueMonitors.RemoveAt(index);
				}
			}
		}
	}
}
