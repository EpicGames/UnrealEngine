// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;

namespace UnrealGameSync
{
	partial class ModalTaskWindow : ThemedForm
	{
		readonly Task _task;
		readonly CancellationTokenSource _cancellationSource;

		public ModalTaskWindow(string inTitle, string inMessage, FormStartPosition inStartPosition, Task inTask, CancellationTokenSource inCancellationSource)//, Func<CancellationToken, Task> InTaskFunc)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			Text = inTitle;
			MessageLabel.Text = inMessage;
			StartPosition = inStartPosition;
			_task = inTask;
			_cancellationSource = inCancellationSource;
		}

		public void ShowAndActivate()
		{
			Show();
			Activate();
		}

		private void ModalTaskWindow_Load(object sender, EventArgs e)
		{
			SynchronizationContext syncContext = SynchronizationContext.Current!;
			_task.ContinueWith(x => syncContext.Post(y => Close(), null), TaskScheduler.Default);
		}

		private void ModalTaskWindow_FormClosing(object sender, FormClosingEventArgs e)
		{
			_cancellationSource.Cancel();
			if (!_task.IsCompleted)
			{
				e.Cancel = true;
			}
		}
	}
}
