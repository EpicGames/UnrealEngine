// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Drawing;
using System.Windows.Forms;
using System.Windows.Forms.VisualStyles;
using EpicGames.Core;

namespace UnrealGameSync
{
	partial class ModifyBuildStepsWindow : ThemedForm
	{
		readonly List<string> _targetNames;
		readonly List<BuildStep> _steps;
		readonly HashSet<Guid> _projectSteps;
		readonly DirectoryReference _baseDirectory;
		ListViewItem.ListViewSubItem? _mouseDownSubItem = null;
		readonly IReadOnlyDictionary<string, string> _variables;

		public ModifyBuildStepsWindow(List<string> inTargetNames, List<BuildStep> inSteps, HashSet<Guid> inProjectSteps, DirectoryReference inBaseDirectory, IReadOnlyDictionary<string, string> inVariables)
		{
			_targetNames = inTargetNames;
			_steps = inSteps;
			_projectSteps = inProjectSteps;
			_baseDirectory = inBaseDirectory;
			_variables = inVariables;

			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			float dpiScaleX = DeviceDpi / 96.0f;
			NormalSyncColumn.Width = (int)(104 * dpiScaleX);
			ScheduledSyncColumn.Width = (int)(104 * dpiScaleX);
			DescriptionColumn.Width = BuildStepList.ClientSize.Width - NormalSyncColumn.Width - ScheduledSyncColumn.Width - 10;

			BuildStepList.Font = SystemFonts.IconTitleFont;
		}

		private void ModifyBuildStepsWindow_Load(object sender, EventArgs e)
		{
			foreach (BuildStep task in _steps)
			{
				AddTask(task);
			}
			UpdateEnabledButtons();
		}

		void AddTask(BuildStep task)
		{
			ListViewItem item = new ListViewItem(task.Description);
			item.Tag = task;
			item.SubItems.Add(new ListViewItem.ListViewSubItem());
			item.SubItems.Add(new ListViewItem.ListViewSubItem());
			BuildStepList.Items.Add(item);
		}

		private void NewStepButton_Click(object sender, EventArgs e)
		{
			BuildStep newStep = new BuildStep(Guid.NewGuid(), BuildStepList.Items.Count, "Untitled Step", "Running Untitled Step...", 1, null, null, null, null, true);
			newStep.Description = "Untitled Task";
			newStep.EstimatedDuration = 1;

			using BuildStepWindow newStepWindow = new BuildStepWindow(newStep, _targetNames, _baseDirectory, _variables);
			if (newStepWindow.ShowDialog() == DialogResult.OK)
			{
				AddTask(newStep);
			}
		}

		private void EditStepButton_Click(object sender, EventArgs e)
		{
			foreach (ListViewItem? item in BuildStepList.SelectedItems)
			{
				if (item != null)
				{
					using BuildStepWindow editStep = new BuildStepWindow((BuildStep)item.Tag!, _targetNames, _baseDirectory, _variables);
					editStep.ShowDialog();
					item.Text = ((BuildStep)item.Tag!).Description;
					break;
				}
			}
		}

		private void RemoveStepButton_Click(object sender, EventArgs e)
		{
			foreach (ListViewItem? item in BuildStepList.SelectedItems)
			{
				if (item != null)
				{
					if (MessageBox.Show(String.Format("Remove the '{0}' step?", item.Text), "Remove Step", MessageBoxButtons.YesNo) == DialogResult.Yes)
					{
						BuildStepList.Items.Remove(item);
					}
					break;
				}
			}
		}

		private void CloseButton_Click(object sender, EventArgs e)
		{
			_steps.Clear();
			foreach (ListViewItem? item in BuildStepList.Items)
			{
				if (item != null)
				{
					_steps.Add((BuildStep)item.Tag!);
				}
			}
			Close();
		}

		private void UpdateEnabledButtons()
		{
			int selectedIndex = (BuildStepList.SelectedIndices.Count > 0) ? BuildStepList.SelectedIndices[0] : -1;

			bool hasSelection = (selectedIndex != -1);
			EditStepButton.Enabled = hasSelection;
			RemoveStepButton.Enabled = hasSelection && !_projectSteps.Contains(((BuildStep)BuildStepList.SelectedItems[0].Tag!).UniqueId);

			MoveUp.Enabled = (selectedIndex >= 1);
			MoveDown.Enabled = (selectedIndex >= 0 && selectedIndex < BuildStepList.Items.Count - 1);
		}

		private void BuildStepList_SelectedIndexChanged(object sender, EventArgs e)
		{
			UpdateEnabledButtons();
		}

		private void BuildStepList_DrawColumnHeader(object sender, DrawListViewColumnHeaderEventArgs e)
		{
			if (e.Header == null)
			{
				e.DrawDefault = true;
			}
			else
			{
				TextRenderer.DrawText(e.Graphics, e.Header.Text, BuildStepList.Font, e.Bounds, Theme.TextColor, TextFormatFlags.Left | TextFormatFlags.VerticalCenter);
			}
		}

		private void BuildStepList_DrawSubItem(object sender, DrawListViewSubItemEventArgs e)
		{
			if (e.ColumnIndex == 0 || e.Item == null)
			{
				e.DrawDefault = true;
			}
			else
			{
				BuildStep task = (BuildStep)e.Item.Tag!;

				bool enabled;
				if (e.ColumnIndex == 1)
				{
					enabled = task.NormalSync;
				}
				else
				{
					enabled = task.ScheduledSync;
				}

				if (BuildStepList.SelectedItems.Contains(e.Item))
				{
					e.Graphics.FillRectangle(SystemBrushes.Highlight, e.Bounds);
				}

				CheckBoxState state;
				if (enabled)
				{
					state = (_mouseDownSubItem == e.SubItem) ? CheckBoxState.CheckedPressed : CheckBoxState.CheckedNormal;
				}
				else
				{
					state = (_mouseDownSubItem == e.SubItem) ? CheckBoxState.UncheckedPressed : CheckBoxState.UncheckedNormal;
				}

				Size size = CheckBoxRenderer.GetGlyphSize(e.Graphics, state);
				CheckBoxRenderer.DrawCheckBox(e.Graphics, new Point(e.Bounds.Left + (e.Bounds.Width - size.Width) / 2, e.Bounds.Top + (e.Bounds.Height - size.Height) / 2), state);
			}
		}

		private void BuildStepList_MouseDown(object sender, MouseEventArgs e)
		{
			ListViewHitTestInfo hitTest = BuildStepList.HitTest(e.X, e.Y);
			_mouseDownSubItem = hitTest.SubItem;
			if (_mouseDownSubItem != null)
			{
				BuildStepList.Invalidate(_mouseDownSubItem.Bounds, true);
			}
		}

		private void BuildStepList_MouseUp(object sender, MouseEventArgs e)
		{
			ListViewHitTestInfo hitTest = BuildStepList.HitTest(e.X, e.Y);
			if (hitTest.Item != null && hitTest.SubItem != null)
			{
				int columnIndex = hitTest.Item.SubItems.IndexOf(hitTest.SubItem);
				if (columnIndex >= 1 && columnIndex <= 3)
				{
					BuildStep task = (BuildStep)hitTest.Item.Tag!;
					if (columnIndex == 1)
					{
						task.NormalSync ^= true;
					}
					else
					{
						task.ScheduledSync ^= true;
					}
					BuildStepList.Invalidate(hitTest.SubItem.Bounds);
				}
			}
			if (_mouseDownSubItem != null)
			{
				BuildStepList.Invalidate(_mouseDownSubItem.Bounds);
				_mouseDownSubItem = null;
			}
		}

		private void MoveUp_Click(object sender, EventArgs e)
		{
			BuildStepList.BeginUpdate();
			foreach (ListViewItem? item in BuildStepList.SelectedItems)
			{
				if (item != null)
				{
					int index = item.Index;
					if (index > 0)
					{
						BuildStepList.Items.RemoveAt(index);
						BuildStepList.Items.Insert(index - 1, item);
					}
					break;
				}
			}
			BuildStepList.EndUpdate();
			UpdateEnabledButtons();
		}

		private void MoveDown_Click(object sender, EventArgs e)
		{
			BuildStepList.BeginUpdate();
			foreach (ListViewItem? item in BuildStepList.SelectedItems)
			{
				if (item != null)
				{
					int index = item.Index;
					if (index < BuildStepList.Items.Count - 1)
					{
						BuildStepList.Items.RemoveAt(index);
						BuildStepList.Items.Insert(index + 1, item);
					}
					break;
				}
			}
			BuildStepList.EndUpdate();
			UpdateEnabledButtons();
		}

		private void ModifyBuildStepsWindow_FormClosed(object sender, FormClosedEventArgs e)
		{
			_steps.Clear();
			foreach (ListViewItem? item in BuildStepList.Items)
			{
				if (item != null)
				{
					BuildStep step = (BuildStep)item.Tag!;
					step.OrderIndex = _steps.Count;
					_steps.Add(step);
				}
			}
		}
	}
}
