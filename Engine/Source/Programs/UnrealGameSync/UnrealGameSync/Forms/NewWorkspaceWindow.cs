// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	partial class NewWorkspaceWindow : ThemedForm
	{
		class NewWorkspaceSettings
		{
			public string Name { get; }
			public string Stream { get; }
			public DirectoryReference RootDir { get; }

			public NewWorkspaceSettings(string name, string stream, DirectoryReference rootDir)
			{
				Name = name;
				Stream = stream;
				RootDir = rootDir;
			}
		}

		class FindWorkspaceSettingsTask
		{
			public InfoRecord Info { get; }
			public List<ClientsRecord> Clients { get; }
			public string? CurrentStream { get; }

			public FindWorkspaceSettingsTask(InfoRecord info, List<ClientsRecord> clients, string? currentStream)
			{
				Info = info;
				Clients = clients;
				CurrentStream = currentStream;
			}

			public static async Task<FindWorkspaceSettingsTask> RunAsync(IPerforceConnection perforce, string currentWorkspaceName, CancellationToken cancellationToken)
			{
				InfoRecord info = await perforce.GetInfoAsync(InfoOptions.ShortOutput, cancellationToken);
				List<ClientsRecord> clients = await perforce.GetClientsAsync(ClientsOptions.None, perforce.Settings.UserName, cancellationToken);

				string? currentStream = null;
				if (!String.IsNullOrEmpty(currentWorkspaceName))
				{
					currentStream = await perforce.GetCurrentStreamAsync(cancellationToken);
				}

				return new FindWorkspaceSettingsTask(info, clients, currentStream);
			}
		}

		static class NewWorkspaceTask
		{
			public static async Task RunAsync(IPerforceConnection perforce, NewWorkspaceSettings settings, string owner, string hostName, ClientSubmitOptions clientSubmitOptions, CancellationToken cancellationToken)
			{
				PerforceResponseList<ClientsRecord> response = await perforce.TryGetClientsAsync(ClientsOptions.None, settings.Name, -1, null, null, cancellationToken);
				if (!response.Succeeded)
				{
					throw new UserErrorException($"Unable to determine if client already exists.\n\n{response[0]}");
				}
				if (response.Data.Count > 0)
				{
					throw new UserErrorException($"Client '{settings.Name}' already exists.");
				}

				ClientRecord client = new ClientRecord(settings.Name, owner, settings.RootDir.FullName);
				client.Host = hostName;
				client.Stream = settings.Stream;
				client.Options = ClientOptions.Rmdir;
				client.SubmitOptions = clientSubmitOptions;
				await perforce.CreateClientAsync(client, cancellationToken);
			}
		}

		readonly IPerforceSettings _perforceSettings;
		readonly InfoRecord _info;
		readonly List<ClientsRecord> _clients;
		readonly IServiceProvider _serviceProvider;
		NewWorkspaceSettings? _settings;
		readonly DirectoryReference? _defaultRootPath;
		readonly UserSettings _userSettings;

		private NewWorkspaceWindow(IPerforceSettings perforceSettings, string? forceStream, string? defaultStream, InfoRecord info, List<ClientsRecord> clients, IServiceProvider serviceProvider, UserSettings settings)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_perforceSettings = perforceSettings;
			_info = info;
			_clients = clients;
			_serviceProvider = serviceProvider;
			_userSettings = settings;

			Dictionary<DirectoryReference, int> rootPathToCount = new Dictionary<DirectoryReference, int>();
			foreach (ClientsRecord client in clients)
			{
				if (client.Host == null || String.Equals(client.Host, info.ClientHost, StringComparison.OrdinalIgnoreCase))
				{
					if (!String.IsNullOrEmpty(client.Root) && client.Root != ".")
					{
						DirectoryReference? parentDir;
						try
						{
							parentDir = new DirectoryReference(client.Root);
						}
						catch
						{
							parentDir = null;
						}

						if (parentDir != null)
						{
							int count;
							rootPathToCount.TryGetValue(parentDir, out count);
							rootPathToCount[parentDir] = count + 1;
						}
					}
				}
			}

			int rootPathMaxCount = 0;
			foreach (KeyValuePair<DirectoryReference, int> rootPathPair in rootPathToCount)
			{
				if (rootPathPair.Value > rootPathMaxCount)
				{
					_defaultRootPath = rootPathPair.Key;
					rootPathMaxCount = rootPathPair.Value;
				}
			}

			if (forceStream != null)
			{
				StreamTextBox.Text = forceStream;
				StreamTextBox.Enabled = false;
			}
			else
			{
				StreamTextBox.Text = defaultStream ?? "";
				StreamTextBox.Enabled = true;
			}
			StreamTextBox.SelectionStart = StreamTextBox.Text.Length;
			StreamTextBox.SelectionLength = 0;
			StreamTextBox.Focus();

			StreamBrowseBtn.Enabled = (forceStream == null);

			UpdateOkButton();
			UpdateNameCueBanner();
			UpdateRootDirCueBanner();
		}

		public static bool ShowModal(IWin32Window owner, IPerforceSettings perforceSettings, string? forceStreamName, string currentWorkspaceName, UserSettings settings, IServiceProvider serviceProvider, [NotNullWhen(true)] out string? workspaceName)
		{
			ModalTask<FindWorkspaceSettingsTask>? task = PerforceModalTask.Execute(owner, "Checking settings", "Checking settings, please wait...", perforceSettings, (p, c) => FindWorkspaceSettingsTask.RunAsync(p, currentWorkspaceName, c), serviceProvider.GetRequiredService<ILogger<FindWorkspaceSettingsTask>>());
			if (task == null || !task.Succeeded)
			{
				workspaceName = null;
				return false;
			}

			using NewWorkspaceWindow window = new NewWorkspaceWindow(perforceSettings, forceStreamName, task.Result.CurrentStream, task.Result.Info, task.Result.Clients, serviceProvider, settings);
			if (window.ShowDialog(owner) == DialogResult.OK)
			{
				workspaceName = window._settings!.Name;
				return true;
			}
			else
			{
				workspaceName = null;
				return false;
			}
		}

		private void RootDirBrowseBtn_Click(object sender, EventArgs e)
		{
			using FolderBrowserDialog dialog = new FolderBrowserDialog();
			dialog.ShowNewFolderButton = true;
			dialog.SelectedPath = RootDirTextBox.Text;
			if (dialog.ShowDialog() == DialogResult.OK)
			{
				RootDirTextBox.Text = dialog.SelectedPath;
				UpdateOkButton();
			}
		}

		private string GetDefaultWorkspaceName()
		{
			string baseName = Sanitize(String.Format("{0}_{1}_{2}", _info.UserName, _info.ClientHost, StreamTextBox.Text.Replace('/', '_').Trim('_'))).Trim('_');

			string name = baseName;
			for (int idx = 2; _clients.Any(x => x.Name != null && String.Equals(x.Name, name, StringComparison.OrdinalIgnoreCase)); idx++)
			{
				name = String.Format("{0}_{1}", baseName, idx);
			}
			return name;
		}

		private string GetDefaultWorkspaceRootDir()
		{
			string rootDir = "";
			if (_defaultRootPath != null)
			{
				string suffix = String.Join("_", StreamTextBox.Text.Split(new char[] { '/' }, StringSplitOptions.RemoveEmptyEntries).Select(x => Sanitize(x)).Where(x => x.Length > 0));
				if (suffix.Length > 0)
				{
					rootDir = DirectoryReference.Combine(_defaultRootPath, suffix).FullName;
				}
			}
			return rootDir;
		}

		private static string Sanitize(string text)
		{
			StringBuilder result = new StringBuilder();
			for (int idx = 0; idx < text.Length; idx++)
			{
				if (Char.IsLetterOrDigit(text[idx]) || text[idx] == '_' || text[idx] == '.' || text[idx] == '-')
				{
					result.Append(text[idx]);
				}
			}
			return result.ToString();
		}

		private void UpdateNameCueBanner()
		{
			NameTextBox.CueBanner = GetDefaultWorkspaceName();
		}

		private void UpdateRootDirCueBanner()
		{
			RootDirTextBox.CueBanner = GetDefaultWorkspaceRootDir();
		}

		private void UpdateOkButton()
		{
			OkBtn.Enabled = TryGetWorkspaceSettings(out _);
		}

		private bool TryGetWorkspaceSettings([NotNullWhen(true)] out NewWorkspaceSettings? settings)
		{
			string newWorkspaceName = NameTextBox.Text.Trim();
			if (newWorkspaceName.Length == 0)
			{
				newWorkspaceName = GetDefaultWorkspaceName();
				if (newWorkspaceName.Length == 0)
				{
					settings = null;
					return false;
				}
			}

			string newStream = StreamTextBox.Text.Trim();
			if (!newStream.StartsWith("//", StringComparison.Ordinal) || newStream.IndexOf('/', 2) == -1)
			{
				settings = null;
				return false;
			}

			string newRootDir = RootDirTextBox.Text.Trim();
			if (newRootDir.Length == 0)
			{
				newRootDir = GetDefaultWorkspaceRootDir();
				if (newRootDir.Length == 0)
				{
					settings = null;
					return false;
				}
			}

			DirectoryReference newRootDirRef;
			try
			{
				newRootDirRef = new DirectoryReference(newRootDir);
			}
			catch
			{
				settings = null;
				return false;
			}

			settings = new NewWorkspaceSettings(newWorkspaceName, newStream, newRootDirRef);
			return true;
		}

		private void NameTextBox_TextChanged(object sender, EventArgs e)
		{
			UpdateOkButton();
		}

		private void StreamTextBox_TextChanged(object sender, EventArgs e)
		{
			UpdateOkButton();
			UpdateNameCueBanner();
			UpdateRootDirCueBanner();
		}

		private void RootDirTextBox_TextChanged(object sender, EventArgs e)
		{
			UpdateOkButton();
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			if (TryGetWorkspaceSettings(out _settings))
			{
				DirectoryInfo rootDir = _settings.RootDir.ToDirectoryInfo();
				if (rootDir.Exists && rootDir.EnumerateFileSystemInfos().Any(x => x.Name != "." && x.Name != ".."))
				{
					if (MessageBox.Show(this, String.Format("The directory '{0}' is not empty. Are you sure you want to create a workspace there?", rootDir.FullName), "Directory not empty", MessageBoxButtons.YesNo) != DialogResult.Yes)
					{
						return;
					}
				}

				ILogger logger = _serviceProvider.GetRequiredService<ILogger<NewWorkspaceWindow>>();

				ModalTask? result = PerforceModalTask.Execute(Owner, "Creating workspace", "Creating workspace, please wait...", _perforceSettings, (p, c) => NewWorkspaceTask.RunAsync(p, _settings, _info.UserName ?? "", _info.ClientHost ?? "", _userSettings.ClientSubmitOptions, c), logger);
				if (result == null || !result.Succeeded)
				{
					return;
				}

				DialogResult = DialogResult.OK;
				Close();
			}
		}

		private void CancelBtn_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
			Close();
		}

		private void StreamBrowseBtn_Click(object sender, EventArgs e)
		{
			string? streamName = StreamTextBox.Text.Trim();
			if (SelectStreamWindow.ShowModal(this, _perforceSettings, streamName, _serviceProvider, out streamName))
			{
				StreamTextBox.Text = streamName;
			}
		}

		private void RootDirTextBox_Enter(object sender, EventArgs e)
		{
			if (RootDirTextBox.Text.Length == 0)
			{
				RootDirTextBox.Text = RootDirTextBox.CueBanner;
			}
		}

		private void RootDirTextBox_Leave(object sender, EventArgs e)
		{
			if (RootDirTextBox.Text == RootDirTextBox.CueBanner)
			{
				RootDirTextBox.Text = "";
			}
		}

		private void NameTextBox_Enter(object sender, EventArgs e)
		{
			if (NameTextBox.Text.Length == 0)
			{
				NameTextBox.Text = NameTextBox.CueBanner;
			}
		}

		private void NameTextBox_Leave(object sender, EventArgs e)
		{
			if (NameTextBox.Text == NameTextBox.CueBanner)
			{
				NameTextBox.Text = "";
			}
		}
	}
}
