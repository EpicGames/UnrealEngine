// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Drawing;
using System.Drawing.Drawing2D;
using System.Windows.Forms;

namespace UnrealGameSync
{
	public enum NotificationType
	{
		Info,
		Warning,
		Error,
	}

	public partial class NotificationWindow : ThemedForm
	{
		NotificationType _type;

		// Logo
		Rectangle _logoBounds;
		readonly InvertibleImage _logoImage;

		// Caption
		Rectangle _captionBounds;
		string _captionText = "Caption Text";
		Font? _captionFont;

		// Message
		Rectangle _messageBounds;
		string _messageText = "Mesage Text";

		// Close button
		Rectangle _closeButtonBounds;
		bool _hoverOverClose;
		bool _mouseDownOverClose;

		// Notifications
		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public Action? OnMoreInformation { get; set; }
		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public Action? OnDismiss { get; set; }

		public NotificationWindow(Image inLogo)
		{
			_logoImage = new InvertibleImage(new Bitmap(inLogo), true);
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));
		}

		public void Show(NotificationType inType, string inCaption, string inMessage)
		{
			_type = inType;
			_captionText = inCaption;
			_messageText = inMessage;

			base.Show();

			CalculateBounds();

			Rectangle workingArea = Screen.PrimaryScreen!.WorkingArea;
			Location = new Point(workingArea.Right - Size.Width - 16, workingArea.Bottom - Size.Height - 16);

			BringToFront();
			Invalidate();
		}

		protected override void Dispose(bool disposing)
		{
			if (disposing)
			{
				if (_captionFont != null)
				{
					_captionFont.Dispose();
					_captionFont = null;
				}
				_logoImage.Dispose();
				if (components != null)
				{
					components.Dispose();
				}
			}

			base.Dispose(disposing);
		}

		public void CalculateBounds()
		{
			int referenceSize8 = (int)(8 * Font.Height) / 16;
			using (Graphics graphics = CreateGraphics())
			{
				int gutterW = referenceSize8 + referenceSize8 / 2;
				int gutterH = referenceSize8 + referenceSize8 / 2;

				// Get the close button size
				int closeButtonSize = referenceSize8 * 2;

				// Space out the text
				int captionY = gutterH;
				int captionW = TextRenderer.MeasureText(graphics, _captionText, _captionFont ?? Font, new Size(100 * referenceSize8, Font.Height), TextFormatFlags.Left | TextFormatFlags.Top | TextFormatFlags.SingleLine).Width;
				int captionH = (_captionFont ?? Font).Height;

				int messageY = captionY + captionH + referenceSize8 / 2;
				int messageW = TextRenderer.MeasureText(graphics, _messageText, Font, new Size(100 * referenceSize8, Font.Height), TextFormatFlags.Left | TextFormatFlags.Top | TextFormatFlags.SingleLine).Width;
				int messageH = Font.Height;

				// Get the logo dimensions
				int logoH = messageY + messageH - gutterH;
				Image? logoImage = _logoImage.ResolveImage();
				int logoW = (logoImage == null) ? 16 : (int)(logoImage.Width * (float)logoH / (float)logoImage.Height);

				// Set the window size
				int textX = gutterW + logoW + referenceSize8 * 2;
				Size = new Size(textX + Math.Max(messageW, captionW) + closeButtonSize + gutterW, messageY + messageH + gutterH);

				// Set the bounds of the individual elements
				_closeButtonBounds = new Rectangle(Width - gutterW - closeButtonSize, gutterH, closeButtonSize, closeButtonSize);
				_logoBounds = new Rectangle(gutterW + referenceSize8, (Height - logoH) / 2, logoW, logoH);
				_captionBounds = new Rectangle(textX, captionY, captionW, captionH);
				_messageBounds = new Rectangle(textX, messageY, messageW, messageH);
			}
		}

		protected override void OnFontChanged(EventArgs e)
		{
			base.OnFontChanged(e);

			_captionFont?.Dispose();
			_captionFont = new Font(Font.Name, Font.Size * (12.0f / 9.0f), FontStyle.Regular);

			CalculateBounds();
			Invalidate();
		}

		protected override void OnFormClosed(FormClosedEventArgs e)
		{
			base.OnFormClosed(e);
		}

		protected override void OnResize(EventArgs e)
		{
			base.OnResize(e);

			CalculateBounds();
			Invalidate();
		}

		protected override void OnMouseMove(MouseEventArgs e)
		{
			base.OnMouseMove(e);

			bool newHoverOverClose = _closeButtonBounds.Contains(e.Location);
			if (_hoverOverClose != newHoverOverClose)
			{
				_hoverOverClose = newHoverOverClose;
				Invalidate(_closeButtonBounds);
			}
		}

		protected override void OnMouseDown(MouseEventArgs e)
		{
			base.OnMouseDown(e);

			if (e.Button == System.Windows.Forms.MouseButtons.Left)
			{
				_mouseDownOverClose = _closeButtonBounds.Contains(e.Location);
				Invalidate(_closeButtonBounds);
			}
		}

		protected override void OnMouseUp(MouseEventArgs e)
		{
			base.OnMouseUp(e);

			if (e.Button == System.Windows.Forms.MouseButtons.Left)
			{
				if (_mouseDownOverClose)
				{
					_mouseDownOverClose = false;
					Invalidate(_closeButtonBounds);
					Hide();
					if (OnDismiss != null)
					{
						OnDismiss();
					}
				}
				else
				{
					Hide();
					if (OnMoreInformation != null)
					{
						OnMoreInformation();
					}
				}
			}
		}

		protected override void OnPaintBackground(PaintEventArgs e)
		{
			base.OnPaintBackground(e);

			Color tintColor;
			switch(_type)
			{
				case NotificationType.Warning:
					tintColor = Theme.WarningNotificationBGTint;
					break;
				case NotificationType.Error:
					tintColor = Theme.ErrorNotificationBGTint;
					break;
				default:
					tintColor = Theme.StandardNotificationBGTint;
					break;
			}

			Color backgroundColor = Theme.InputBackgroundColor;
			using(Brush backgroundBrush = new LinearGradientBrush(new Point(0, 0), new Point(0, Height), backgroundColor, tintColor))
			{
				e.Graphics.FillRectangle(backgroundBrush, ClientRectangle);
			}

			using (Pen borderPen = new Pen(Theme.ForegroundColor))
			{
				e.Graphics.DrawRectangle(borderPen, new Rectangle(0, 0, Width - 1, Height - 1));
			}

			Image? logoImage = _logoImage.ResolveImage();
			if (logoImage != null)
			{
				e.Graphics.InterpolationMode = InterpolationMode.HighQualityBicubic;
				e.Graphics.DrawImage(logoImage, _logoBounds);
			}
		}

		protected override void OnPaint(PaintEventArgs e)
		{
			base.OnPaint(e);

			e.Graphics.SmoothingMode = SmoothingMode.AntiAlias;

			// Draw the close button
			if (_hoverOverClose)
			{
				using(Brush hoverBrush = new SolidBrush(Theme.ActiveCaption))
				{
					e.Graphics.FillRectangle(hoverBrush, _closeButtonBounds);
				}
			}
			else if(_mouseDownOverClose)
			{
				using(Brush downBrush = new SolidBrush(Theme.InactiveCaption))
				{
					e.Graphics.FillRectangle(downBrush, _closeButtonBounds);
				}
			}
			if(_hoverOverClose || _mouseDownOverClose)
			{
				using(Pen borderPen = new Pen(Theme.InactiveCaptionText, 1.0f))
				{
					e.Graphics.DrawRectangle(borderPen, new Rectangle(_closeButtonBounds.X, _closeButtonBounds.Y, _closeButtonBounds.Width - 1, _closeButtonBounds.Height - 1));
				}
			}
			using(Pen crossPen = new Pen(Theme.ForegroundColor, 2.0f))
			{
				float offset = (_closeButtonBounds.Width - 1) / 4.0f;
				e.Graphics.DrawLine(crossPen, _closeButtonBounds.Left + offset, _closeButtonBounds.Top + offset, _closeButtonBounds.Right - 1 - offset, _closeButtonBounds.Bottom - 1 - offset);
				e.Graphics.DrawLine(crossPen, _closeButtonBounds.Left + offset, _closeButtonBounds.Bottom - 1 - offset, _closeButtonBounds.Right - 1 - offset, _closeButtonBounds.Top + offset);
			}

			// Draw the caption
			TextRenderer.DrawText(e.Graphics, _captionText, _captionFont, _captionBounds, Theme.LinkColor, TextFormatFlags.Left);

			// Draw the message
			TextRenderer.DrawText(e.Graphics, _messageText, Font, _messageBounds, Theme.TextColor, TextFormatFlags.Left);
		}
	}
}
