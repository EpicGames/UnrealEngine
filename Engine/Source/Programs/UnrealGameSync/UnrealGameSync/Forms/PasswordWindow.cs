// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealGameSync
{
	public partial class PasswordWindow : ThemedForm
	{
		public string Password { get; private set; }

		public PasswordWindow(string prompt, string password)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			Password = password;

			PromptLabel.Text = prompt;
			PasswordTextBox.Text = password;
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			Password = PasswordTextBox.Text;
		}

		private void CancelBtn_Click(object sender, EventArgs e)
		{

		}
	}
}
