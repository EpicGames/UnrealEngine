// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	partial class PerforceSyncSettingsWindow : ThemedForm
	{
		readonly UserSettings _settings;
		readonly ILogger _logger;

		public PerforceSyncSettingsWindow(UserSettings settings, ILogger logger)
		{
			_settings = settings;
			_logger = logger;

			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));
		}

		private void PerforceSettingsWindow_Load(object sender, EventArgs e)
		{
			PerforceSyncOptions syncOptions = _settings.SyncOptions;
			numericUpDownMaxCommandsPerBatch.Value = syncOptions.MaxCommandsPerBatch ?? PerforceSyncOptions.DefaultMaxCommandsPerBatch;
			numericUpDownMaxSizePerBatch.Value = (syncOptions.MaxSizePerBatch ?? PerforceSyncOptions.DefaultMaxSizePerBatch) / 1024 / 1024;
			numericUpDownRetriesOnSyncError.Value = syncOptions.NumSyncErrorRetries ?? PerforceSyncOptions.DefaultNumSyncErrorRetries;
			numericUpDownSyncErrorRetryDelay.Value = syncOptions.SyncErrorRetryDelay ?? PerforceSyncOptions.DefaultSyncErrorRetryDelay;

			FastSyncCheckBox.Checked = syncOptions.UseFastSync ?? false;
			UseNativeLibraryCheckBox.Checked = syncOptions.UseNativeClient ?? PerforceSyncOptions.DefaultUseNativeLibrary;
		}

		private void OkButton_Click(object sender, EventArgs e)
		{
			_settings.SyncOptions.MaxCommandsPerBatch = GetValueIfNotDefault((int)numericUpDownMaxCommandsPerBatch.Value, PerforceSyncOptions.DefaultMaxCommandsPerBatch);
			_settings.SyncOptions.MaxSizePerBatch = GetValueIfNotDefault((int)numericUpDownMaxSizePerBatch.Value, PerforceSyncOptions.DefaultMaxSizePerBatch) * 1024 * 1024;
			_settings.SyncOptions.NumSyncErrorRetries = GetValueIfNotDefault((int)numericUpDownRetriesOnSyncError.Value, PerforceSyncOptions.DefaultNumSyncErrorRetries);
			_settings.SyncOptions.SyncErrorRetryDelay = GetValueIfNotDefault((int)numericUpDownSyncErrorRetryDelay.Value, PerforceSyncOptions.DefaultSyncErrorRetryDelay);
			_settings.SyncOptions.UseFastSync = FastSyncCheckBox.Checked;
			_settings.SyncOptions.UseNativeClient = UseNativeLibraryCheckBox.Checked;
			_settings.Save(_logger);

			DialogResult = System.Windows.Forms.DialogResult.OK;
			Close();
		}

		private static int? GetValueIfNotDefault(int value, int defaultValue)
		{
			return (value == defaultValue) ? (int?)null : value;
		}

		private void CancButton_Click(object sender, EventArgs e)
		{
			DialogResult = System.Windows.Forms.DialogResult.Cancel;
			Close();
		}

		private void ResetButton_Click(object sender, EventArgs e)
		{
			numericUpDownMaxCommandsPerBatch.Value = PerforceSyncOptions.DefaultMaxCommandsPerBatch;
			numericUpDownMaxSizePerBatch.Value = PerforceSyncOptions.DefaultMaxSizePerBatch / 1024 / 1024;
			numericUpDownRetriesOnSyncError.Value = PerforceSyncOptions.DefaultNumSyncErrorRetries;
			numericUpDownSyncErrorRetryDelay.Value = PerforceSyncOptions.DefaultSyncErrorRetryDelay;
			FastSyncCheckBox.Checked = PerforceSyncOptions.DefaultUseFastSync;
			UseNativeLibraryCheckBox.Checked = PerforceSyncOptions.DefaultUseNativeLibrary;
		}
	}
}
