// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Windows.Forms;

namespace UnrealGameSync.Forms
{
	public partial class PresetsWindow : ThemedForm
	{
		private const string ForcedDefaultPresetToolTip = "Presets are disabled as a locked default preset is used.";
		private readonly IDictionary<Guid, WorkspaceSyncCategory> _uniqueIdToCategory;
		private readonly IDictionary<string, Preset> _roles;

		private readonly Action<Preset>? _copyToGlobal;
		private readonly Action<Preset>? _copyToWorkspace;

		private Timer _presetComboboxTooltipTimer { get; } = new Timer { Interval = 300 };
		private Timer _copyToTooltipTimer { get; } = new Timer { Interval = 300 };

		private class PresetItem
		{
			public string Name { get; set; } = String.Empty;

			public string Suffix { get; set; } = String.Empty;

			public override string ToString()
			{
				if (String.IsNullOrWhiteSpace(Suffix))
				{
					return Name;
				}

				return $"{Name} {Suffix}";
			}
		}

		[DesignerSerializationVisibility(DesignerSerializationVisibility.Hidden)]
		public string PresetName { get; set; } = String.Empty;

		public PresetsWindow(
			IDictionary<Guid, WorkspaceSyncCategory> uniqueIdToCategory,
			string presetName,
			string defaultPreset,
			bool forceDefaultPreset,
			IDictionary<string, Preset> roles,
			Action<Preset>? copyToGlobal = null,
			Action<Preset>? copyToWorkspace = null
			)
		{
			_uniqueIdToCategory = uniqueIdToCategory;
			_roles = roles;
			_copyToGlobal = copyToGlobal;
			_copyToWorkspace = copyToWorkspace;

			InitializeComponent();

			RoleSelector.BeginUpdate();
			RoleSelector.Items.Clear();

			// add an empty string to be able to clear the current role
			RoleSelector.Items.Add(new PresetItem());
			foreach (string rolesKey in roles.Keys)
			{
				string suffix = String.Empty;
				if (!String.IsNullOrWhiteSpace(defaultPreset) && rolesKey.Equals(defaultPreset, StringComparison.OrdinalIgnoreCase))
				{
					suffix = "(Default)";
				}

				PresetItem item = new PresetItem()
				{
					Name = rolesKey,
					Suffix = suffix
				};

				RoleSelector.Items.Add(item);

				if (presetName.Equals(rolesKey, StringComparison.OrdinalIgnoreCase))
				{
					RoleSelector.SelectedItem = item;
				}
			}

			RoleSelector.SelectedIndexChanged += RoleSelector_SelectedIndexChanged;

			RoleSelector.EndUpdate();

			if (forceDefaultPreset)
			{
				SelectPreset.Enabled = false;
				RoleSelector.Enabled = false;
				CopyToGlobal.Enabled = false;
				CopyToWorkspace.Enabled = false;

				if (RoleSelector.Parent != null)
				{
					RoleSelector.Parent.MouseMove += (sender, args) =>
					{
						_presetComboboxTooltipTimer.Start();
					};

					RoleSelector.Parent.MouseLeave += (sender, args) =>
					{
						_presetComboboxTooltipTimer.Stop();
						PresetComboBoxToolTip.Hide(RoleSelector.Parent);
					};

					_presetComboboxTooltipTimer.Tick += (sender, args) =>
					{
						_presetComboboxTooltipTimer.Stop();
						PresetComboBoxToolTip.SetToolTip(RoleSelector.Parent, ForcedDefaultPresetToolTip);
					};
				}

				if (CopyToWorkspace.Parent != null)
				{
					CopyToWorkspace.Parent.MouseMove += (sender, args) =>
					{
						_copyToTooltipTimer.Start();
					};

					CopyToWorkspace.Parent.MouseLeave += (sender, args) =>
					{
						_copyToTooltipTimer.Stop();
						CopyToToolTip.Hide(CopyToWorkspace.Parent);
					};

					_copyToTooltipTimer.Tick += (sender, args) =>
					{
						_copyToTooltipTimer.Stop();
						CopyToToolTip.SetToolTip(CopyToWorkspace.Parent, ForcedDefaultPresetToolTip);
					};
				}
			}

			SyncFilters.SelectionMode = DataGridViewSelectionMode.CellSelect;

			SetCurrentPreset(presetName);
		}

		private void RoleSelector_SelectedIndexChanged(object? sender, System.EventArgs e)
		{
			if (sender is not ComboBox senderComboBox)
			{
				return;
			}

			if (senderComboBox.SelectedItem is PresetItem selected)
			{
				string? name = selected.Name;
				PopulateGrid(name ?? String.Empty);
				PopulateViews(name ?? String.Empty);
			}
		}

		private void PopulateGrid(string name)
		{
			if (String.IsNullOrWhiteSpace(name))
			{
				SyncFilters.Rows.Clear();
			}
			else
			{
				if (!_roles.TryGetValue(name, out Preset? role))
				{
					return;
				}

				SyncFilters.Rows.Clear();

				foreach (KeyValuePair<Guid, RoleCategory> category in role.Categories)
				{
					if (!_uniqueIdToCategory.TryGetValue(category.Key, out WorkspaceSyncCategory? workspaceSyncCategory))
					{
						continue;
					}

					string[] row = new[]
					{
						workspaceSyncCategory.Name, category.Value.Enabled ? "True" : "False",
						String.Join('\n', workspaceSyncCategory.Paths)
					};

					SyncFilters.Rows.Add(row);
				}
			}
		}

		private void PopulateViews(string name)
		{
			if (String.IsNullOrWhiteSpace(name))
			{
				Views.Items.Clear();
			}
			else
			{
				if (_roles.TryGetValue(name, out Preset? role))
				{
					Views.Items.Clear();

					foreach (string view in role.Views)
					{
						ListViewItem item = new ListViewItem(view);
						Views.Items.Add(item);
					}
				}
			}
		}

		private void SelectRole_Click(object sender, EventArgs e)
		{
			string? newPreset = RoleSelector?.SelectedItem?.ToString();

			SetCurrentPreset(newPreset);
		}

		private void SetCurrentPreset(string? newPreset)
		{
			PresetName = String.IsNullOrWhiteSpace(newPreset) ? String.Empty : newPreset;

			PopulateGrid(PresetName);
			PopulateViews(PresetName);
			CurrentPreset.Text = PresetName;
		}

		private void OkButton_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.OK;
		}

		private void CancButton_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
		}

		private void CopyToGlobal_Click(object sender, EventArgs e)
		{
			CopyTo(_copyToGlobal);
		}

		private void CopyToWorkspace_Click(object sender, EventArgs e)
		{
			CopyTo(_copyToWorkspace);
		}

		private void CopyTo(Action<Preset>? action)
		{
			if (action == null)
			{
				return;
			}

			string? currentRoleName = RoleSelector?.SelectedItem?.ToString();
			if (String.IsNullOrWhiteSpace(currentRoleName))
			{
				return;
			}

			if (_roles.TryGetValue(currentRoleName, out Preset? role))
			{
				action.Invoke(role);
			}
		}
	}
}
