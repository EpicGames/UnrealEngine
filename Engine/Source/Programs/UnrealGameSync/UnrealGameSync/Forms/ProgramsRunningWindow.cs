// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Data;
using System.Linq;
using System.Threading;
using System.Windows.Forms;
using EpicGames.Core;

namespace UnrealGameSync
{
	public partial class ProgramsRunningWindow : ThemedForm
	{
		readonly object _syncObject = new object();
		FileReference[] _programs;
		readonly Func<FileReference[]> _enumeratePrograms;
		ManualResetEvent? _terminateEvent;
		Thread? _backgroundThread;

		public ProgramsRunningWindow(Func<FileReference[]> enumeratePrograms, FileReference[] programs)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_programs = programs.OrderBy(x => x).ToArray();
			_enumeratePrograms = enumeratePrograms;
			ProgramListBox.Items.AddRange(programs);
		}

		protected override void Dispose(bool disposing)
		{
			if (disposing)
			{
				_terminateEvent?.Dispose();
				components?.Dispose();
			}
			base.Dispose(disposing);
		}

		private void ProgramsRunningWindow_Load(object sender, EventArgs e)
		{
			_terminateEvent = new ManualResetEvent(false);

			_backgroundThread = new Thread(() => ExecuteBackgroundWork());
			_backgroundThread.IsBackground = true;
			_backgroundThread.Start();
		}

		private void ExecuteBackgroundWork()
		{
			for (; ; )
			{
				if (_terminateEvent!.WaitOne(TimeSpan.FromSeconds(2.0)))
				{
					break;
				}

				FileReference[] newPrograms = _enumeratePrograms().OrderBy(x => x).ToArray();
				lock (_syncObject)
				{
					if (!Enumerable.SequenceEqual(_programs, newPrograms))
					{
						_programs = newPrograms;
						BeginInvoke(new MethodInvoker(() => UpdatePrograms()));
					}
				}
			}
		}

		private void UpdatePrograms()
		{
			ProgramListBox.Items.Clear();
			ProgramListBox.Items.AddRange(_programs);

			if (_programs.Length == 0)
			{
				DialogResult = DialogResult.OK;
				Close();
			}
		}

		private void ProgramsRunningWindow_FormClosed(object sender, FormClosedEventArgs e)
		{
			if (_backgroundThread != null)
			{
				_terminateEvent!.Set();

				_backgroundThread.Join();
				_backgroundThread = null;

				_terminateEvent.Dispose();
				_terminateEvent = null;
			}
		}
	}
}
