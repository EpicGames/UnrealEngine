// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.Globalization;
using System.Linq;
using System.Windows.Forms;

namespace UnrealGameSync
{
	partial class ScheduleWindow : ThemedForm
	{
		readonly Dictionary<UserSelectedProjectSettings, List<LatestChangeType>> _projectToLatestChangeTypes;

		public ScheduleWindow(
			bool inEnabled,
			TimeSpan inTime,
			bool anyOpenProject,
			IEnumerable<UserSelectedProjectSettings> scheduledProjects,
			IEnumerable<UserSelectedProjectSettings> openProjects,
			Dictionary<UserSelectedProjectSettings, List<LatestChangeType>> inProjectToLatestChangeTypes)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			EnableCheckBox.Checked = inEnabled;

			_projectToLatestChangeTypes = inProjectToLatestChangeTypes;

			DateTime currentTime = DateTime.Now;
			TimePicker.CustomFormat = CultureInfo.CurrentCulture.DateTimeFormat.ShortTimePattern;
			TimePicker.Value = new DateTime(currentTime.Year, currentTime.Month, currentTime.Day, inTime.Hours, inTime.Minutes, inTime.Seconds);

			ProjectListBox.Items.Add("Any open projects", anyOpenProject);

			Dictionary<string, UserSelectedProjectSettings> localFileToProject = new Dictionary<string, UserSelectedProjectSettings>(StringComparer.InvariantCultureIgnoreCase);
			AddProjects(scheduledProjects, localFileToProject);
			AddProjects(openProjects, localFileToProject);

			foreach (UserSelectedProjectSettings project in localFileToProject.Values.OrderBy(x => x.ToString()))
			{
				bool enabled = scheduledProjects.Any(x => x.LocalPath == project.LocalPath);
				ProjectListBox.Items.Add(project, enabled);
			}

			ProjectListBox.ItemCheck += ProjectListBox_ItemCheck;

			SyncTypeDropDown.Closed += SyncTypeDropDown_Closed;

			UpdateEnabledControls();
		}

		private void UpdateSyncTypeDropDownWithProject(UserSelectedProjectSettings? selectedProject)
		{
			SyncTypeDropDown.Items.Clear();

			if (selectedProject != null
				&& _projectToLatestChangeTypes.ContainsKey(selectedProject))
			{
				foreach (LatestChangeType changeType in _projectToLatestChangeTypes[selectedProject])
				{
					System.Windows.Forms.ToolStripMenuItem menuItem = new System.Windows.Forms.ToolStripMenuItem();
					menuItem.Name = changeType.Name;
					menuItem.Text = changeType.Description;
					menuItem.Size = new System.Drawing.Size(189, 22);
					menuItem.Click += (sender, e) => SyncTypeDropDown_Click(changeType.Name);

					SyncTypeDropDown.Items.Add(menuItem);
				}
			}
		}

		private void ProjectListBox_ItemCheck(object? sender, ItemCheckEventArgs e)
		{
			bool isAnyOpenProjectIndex = e.Index == 0;
			if (e.NewValue == CheckState.Checked && !isAnyOpenProjectIndex)
			{
				UpdateSyncTypeDropDownWithProject(ProjectListBox.Items[e.Index] as UserSelectedProjectSettings);

				SyncTypeDropDown.Show(
					ProjectListBox,
					ProjectListBox.GetItemRectangle(e.Index).Location,
					ToolStripDropDownDirection.BelowRight);
			}
		}

		private void SyncTypeDropDown_Closed(object? sender, ToolStripDropDownClosedEventArgs e)
		{
			if (e.CloseReason != ToolStripDropDownCloseReason.ItemClicked)
			{
				ProjectListBox.SetItemChecked(ProjectListBox.SelectedIndex, false);
			}
		}

		private void SyncTypeDropDown_Click(string syncTypeId)
		{
			UserSelectedProjectSettings? projectSetting = ProjectListBox.Items[ProjectListBox.SelectedIndex] as UserSelectedProjectSettings;
			if (projectSetting != null)
			{
				projectSetting.ScheduledSyncTypeId = syncTypeId;
			}
			SyncTypeDropDown.Close();
		}

		private static void AddProjects(IEnumerable<UserSelectedProjectSettings> projects, Dictionary<string, UserSelectedProjectSettings> localFileToProject)
		{
			foreach (UserSelectedProjectSettings project in projects)
			{
				if (project.LocalPath != null)
				{
					localFileToProject[project.LocalPath] = project;
				}
			}
		}

		public void CopySettings(UserSettings settings)
		{
			settings.ScheduleEnabled = EnableCheckBox.Checked;
			settings.ScheduleTime = TimePicker.Value.TimeOfDay;

			settings.ScheduleAnyOpenProject = false;
			settings.ScheduleProjects.Clear();

			foreach (int index in ProjectListBox.CheckedIndices.OfType<int>())
			{
				if (index == 0)
				{
					settings.ScheduleAnyOpenProject = true;
				}
				else
				{
					settings.ScheduleProjects.Add((UserSelectedProjectSettings)ProjectListBox.Items[index]);
				}
			}
		}

		private void EnableCheckBox_CheckedChanged(object sender, EventArgs e)
		{
			UpdateEnabledControls();
		}

		private void UpdateEnabledControls()
		{
			TimePicker.Enabled = EnableCheckBox.Checked;
			ProjectListBox.Enabled = EnableCheckBox.Checked;
		}
	}
}
