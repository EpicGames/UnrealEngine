// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.Diagnostics;
using System.Drawing;
using System.Drawing.Drawing2D;
using System.IO;
using System.Linq;
using System.Windows.Forms;

#nullable enable

namespace UnrealGameSync
{
	public partial class SdkInfoWindow : ThemedForm
	{
		abstract class SdkAction
		{
			public string Name { get; init; }

			public SdkAction(string name)
			{
				Name = name;
			}

			public abstract void Execute();
		}

		class SdkActionRun : SdkAction
		{
			public string Program { get; set; } = "";
			public string Args { get; set; } = "";

			public SdkActionRun(string name, string program = "", string args = "")
				: base(name)
			{
				Program = program;
				Args = args;
			}

			public override void Execute()
			{
				try
				{
					ProcessStartInfo startInfo = new ProcessStartInfo();
					startInfo.FileName = Program;
					startInfo.Arguments = Args;
					startInfo.UseShellExecute = true;
					Process.Start(startInfo);
				}

				catch (Exception ex)
				{
					MessageBox.Show($"Unable to run '{Program} {Args}': {ex.Message}");
				}
			}
		}

		class SdkItem
		{
			public string Category { get; }
			public string Description { get; }

			public List<SdkAction> Actions = new List<SdkAction>();

			public SdkItem(string category, string description)
			{
				Category = category;
				Description = description;
			}
		}

		class BadgeInfo
		{
			public string UniqueId { get; }
			public string Label { get; }
			public Rectangle Rectangle { get; set; }
			public Action? OnClick { get; set; }

			public BadgeInfo(string uniqueId, string label)
			{
				UniqueId = uniqueId;
				Label = label;
			}
		}

		readonly Font _badgeFont;
		string? _hoverBadgeUniqueId;

		public SdkInfoWindow(string[] sdkInfoEntries, Dictionary<string, string> variables, Font badgeFont)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_badgeFont = badgeFont;

			Dictionary<string, ConfigObject> uniqueIdToObject = new Dictionary<string, ConfigObject>(StringComparer.InvariantCultureIgnoreCase);
			foreach (string sdkInfoEntry in sdkInfoEntries)
			{
				ConfigObject obj = new ConfigObject(sdkInfoEntry);

				string uniqueId = obj.GetValue("UniqueId", Guid.NewGuid().ToString());

				ConfigObject? existingObject;
				if (uniqueIdToObject.TryGetValue(uniqueId, out existingObject))
				{
					existingObject.AddOverrides(obj, null);
				}
				else
				{
					uniqueIdToObject.Add(uniqueId, obj);
				}
			}

			List<SdkItem> items = new List<SdkItem>();
			foreach (ConfigObject obj in uniqueIdToObject.Values)
			{
				string category = obj.GetValue("Category", "Other");
				string description = obj.GetValue("Description", "");
				SdkItem item = new SdkItem(category, description);

				List<string> keys = obj.GetKeys().ToList();

				foreach (string key in keys)
				{
					string keyToAdd = key;
					string? value = obj.GetValue(keyToAdd, "");

					if (String.IsNullOrEmpty(value))
					{
						continue;
					}

					if (!value.StartsWith('('))
					{
						// Handle the predefined Install and Browse actions.

						if (keyToAdd == "Install")
						{
							string? installArgument = Utility.ExpandVariables(value, variables);
							if (!installArgument.Contains("$(", StringComparison.Ordinal))
							{
								item.Actions.Add(new SdkActionRun("Install", installArgument));

								// If Browse is not explicitly defined, generate it automatically from the Installer path.
								if (!keys.Contains("Browse"))
								{
									keyToAdd = "Browse";
									value = Path.GetDirectoryName(installArgument);

									if (String.IsNullOrEmpty(value))
									{
										continue;
									}
								}
							}
						}

						if (keyToAdd == "Browse")
						{
							string? browseArgument = Utility.ExpandVariables(value, variables);
							if (!browseArgument.Contains("$(", StringComparison.Ordinal))
							{
								item.Actions.Add(new SdkActionRun("Browse", "explorer.exe", browseArgument.Replace('/', '\\')));
							}
						}
					}
					else
					{
						// Handle generic actions i.e. ones with custom names which may launch arbitrary executables with additional parameters.
						// For instance,
						//		ReleaseNotes=(Program="notepad.exe", Args="notes.txt")
						// will add a "ReleaseNotes" badge which, when clicked, will open "notes.txt" in the notepad.

						ConfigObject valueObj = new ConfigObject(value);

						string program = valueObj.GetValue("Program", "");
						string args = valueObj.GetValue("Args", "");

						program = Utility.ExpandVariables(program, variables);
						if (program.Contains("$(", StringComparison.Ordinal))
						{
							continue;
						}

						args = Utility.ExpandVariables(args, variables);
						if (args.Contains("$(", StringComparison.Ordinal))
						{
							continue;
						}

						if (!String.IsNullOrEmpty(program))
						{
							item.Actions.Add(new SdkActionRun(keyToAdd, program, args));
						}
					}
				}

				items.Add(item);
			}

			foreach (IGrouping<string, SdkItem> itemGroup in items.GroupBy(x => x.Category).OrderBy(x => x.Key))
			{
				ListViewGroup group = new ListViewGroup(itemGroup.Key);
				SdkListView.Groups.Add(group);

				foreach (SdkItem item in itemGroup)
				{
					ListViewItem newItem = new ListViewItem(group);
					newItem.SubItems.Add(item.Description);
					newItem.SubItems.Add(new ListViewItem.ListViewSubItem() { Tag = item });
					SdkListView.Items.Add(newItem);
				}
			}

			System.Reflection.PropertyInfo doubleBufferedProperty = typeof(Control).GetProperty("DoubleBuffered", System.Reflection.BindingFlags.NonPublic | System.Reflection.BindingFlags.Instance)!;
			doubleBufferedProperty.SetValue(SdkListView, true, null);

			if (SdkListView.Items.Count > 0)
			{
				int itemsHeight = SdkListView.Items[^1].Bounds.Bottom + 20;
				Height = SdkListView.Top + itemsHeight + (Height - SdkListView.Bottom);
			}
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.OK;
			Close();
		}

		private void SdkListView_DrawItem(object sender, DrawListViewItemEventArgs e)
		{
			SdkListView.DrawBackground(e.Graphics, e.Item);
		}

		private void SdkListView_DrawSubItem(object sender, DrawListViewSubItemEventArgs e)
		{
			if (e.Item == null || e.SubItem == null)
			{
				e.DrawDefault = true;
			}
			else if (e.ColumnIndex != columnHeader3.Index)
			{
				TextRenderer.DrawText(e.Graphics, e.SubItem.Text, SdkListView.Font, e.Bounds, Theme.TextColor, TextFormatFlags.EndEllipsis | TextFormatFlags.SingleLine | TextFormatFlags.Left | TextFormatFlags.VerticalCenter | TextFormatFlags.NoPrefix);
			}
			else
			{
				e.Graphics.SmoothingMode = SmoothingMode.AntiAlias;

				List<BadgeInfo> badges = GetBadges(e.Item, e.SubItem);
				for (int idx = 0; idx < badges.Count; idx++)
				{
					Color badgeColor = (_hoverBadgeUniqueId == badges[idx].UniqueId) ? Color.FromArgb(140, 180, 230) : Color.FromArgb(112, 146, 190);
					if (badges[idx].OnClick != null)
					{
						DrawBadge(e.Graphics, badges[idx].Label, badges[idx].Rectangle, (idx > 0), (idx < badges.Count - 1), badgeColor);
					}
				}
			}
		}

		private void SdkListView_MouseMove(object sender, MouseEventArgs e)
		{
			string? newHoverUniqueId = null;

			ListViewHitTestInfo hitTest = SdkListView.HitTest(e.Location);
			if (hitTest.Item != null && hitTest.SubItem == hitTest.Item.SubItems[2])
			{
				List<BadgeInfo> badges = GetBadges(hitTest.Item, hitTest.SubItem);
				foreach (BadgeInfo badge in badges)
				{
					if (badge.Rectangle.Contains(e.Location))
					{
						newHoverUniqueId = badge.UniqueId;
					}
				}
			}

			if (newHoverUniqueId != _hoverBadgeUniqueId)
			{
				_hoverBadgeUniqueId = newHoverUniqueId;
				SdkListView.Invalidate();
			}
		}

		private void SdkListView_MouseLeave(object sender, EventArgs e)
		{
			_hoverBadgeUniqueId = null;
		}

		private void SdkListView_MouseDown(object sender, MouseEventArgs e)
		{
			ListViewHitTestInfo hitTest = SdkListView.HitTest(e.Location);
			if (hitTest.Item != null && hitTest.SubItem == hitTest.Item.SubItems[2])
			{
				List<BadgeInfo> badges = GetBadges(hitTest.Item, hitTest.SubItem);
				foreach (BadgeInfo badge in badges)
				{
					if (badge.Rectangle.Contains(e.Location) && badge.OnClick != null)
					{
						badge.OnClick();
					}
				}
			}
		}

		private List<BadgeInfo> GetBadges(ListViewItem item, ListViewItem.ListViewSubItem subItem)
		{
			string uniqueIdPrefix = String.Format("{0}_", item.Index);

			List<BadgeInfo> badges = new List<BadgeInfo>();

			SdkItem sdk = (SdkItem)subItem.Tag!;

			foreach (SdkAction action in sdk.Actions)
			{
				Action clickAction = () => { action.Execute(); };

				badges.Add(new BadgeInfo($"{uniqueIdPrefix}_{action.Name}", action.Name) { OnClick = clickAction });
			}

			int right = subItem.Bounds.Right - 10;
			for (int idx = badges.Count - 1; idx >= 0; idx--)
			{
				Size badgeSize = GetBadgeSize(badges[idx].Label);
				right -= badgeSize.Width;
				badges[idx].Rectangle = new Rectangle(right, subItem.Bounds.Y + (subItem.Bounds.Height - badgeSize.Height) / 2, badgeSize.Width, badgeSize.Height);
			}

			return badges;
		}

		private Size GetBadgeSize(string badgeText)
		{
			Size labelSize = TextRenderer.MeasureText(badgeText, _badgeFont);
			int badgeHeight = _badgeFont.Height + 1;

			return new Size(labelSize.Width + badgeHeight - 4, badgeHeight);
		}

		private void DrawBadge(Graphics graphics, string badgeText, Rectangle badgeRect, bool mergeLeft, bool mergeRight, Color badgeColor)
		{
			using (GraphicsPath path = new GraphicsPath())
			{
				path.StartFigure();
				path.AddLine(badgeRect.Left + (mergeLeft ? 1 : 0), badgeRect.Top, badgeRect.Left - (mergeLeft ? 1 : 0), badgeRect.Bottom);
				path.AddLine(badgeRect.Left - (mergeLeft ? 1 : 0), badgeRect.Bottom, badgeRect.Right - 1 - (mergeRight ? 1 : 0), badgeRect.Bottom);
				path.AddLine(badgeRect.Right - 1 - (mergeRight ? 1 : 0), badgeRect.Bottom, badgeRect.Right - 1 + (mergeRight ? 1 : 0), badgeRect.Top);
				path.AddLine(badgeRect.Right - 1 + (mergeRight ? 1 : 0), badgeRect.Top, badgeRect.Left + (mergeLeft ? 1 : 0), badgeRect.Top);
				path.CloseFigure();

				using (SolidBrush brush = new SolidBrush(badgeColor))
				{
					graphics.FillPath(brush, path);
				}
			}

			TextRenderer.DrawText(graphics, badgeText, _badgeFont, badgeRect, Color.White, TextFormatFlags.HorizontalCenter | TextFormatFlags.VerticalCenter | TextFormatFlags.SingleLine | TextFormatFlags.NoPrefix | TextFormatFlags.PreserveGraphicsClipping);
		}
	}
}
