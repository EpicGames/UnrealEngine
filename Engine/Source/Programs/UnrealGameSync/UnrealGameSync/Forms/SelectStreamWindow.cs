// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.Diagnostics.CodeAnalysis;
using System.Drawing;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace UnrealGameSync
{
	partial class SelectStreamWindow : ThemedForm
	{
		static class EnumerateStreamsTask
		{
			public static async Task<List<StreamsRecord>> RunAsync(IPerforceConnection perforce, CancellationToken cancellationToken)
			{
				return await perforce.GetStreamsAsync(null, cancellationToken);
			}
		}

		class StreamNode
		{
			public StreamsRecord Record;
			public List<StreamNode> ChildNodes = new List<StreamNode>();

			public StreamNode(StreamsRecord record)
			{
				Record = record;
			}

			public void Sort()
			{
				ChildNodes = ChildNodes.OrderBy(x => x.Record.Name).ToList();
				foreach (StreamNode childNode in ChildNodes)
				{
					childNode.Sort();
				}
			}
		}

		class StreamDepot
		{
			public string Name;
			public List<StreamNode> RootNodes = new List<StreamNode>();

			public StreamDepot(string name)
			{
				Name = name;
			}

			public void Sort()
			{
				RootNodes = RootNodes.OrderBy(x => x.Record.Name).ToList();
				foreach (StreamNode rootNode in RootNodes)
				{
					rootNode.Sort();
				}
			}
		}

		private string? _selectedStream;
		private readonly List<StreamDepot> _depots = new List<StreamDepot>();

		private SelectStreamWindow(List<StreamsRecord> streams, string? streamName)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_selectedStream = streamName;

			// Set up the image list
			ImageList perforceImageList = new ImageList();
			perforceImageList.ImageSize = new Size(16, 16);
			perforceImageList.ColorDepth = ColorDepth.Depth32Bit;
			perforceImageList.Images.AddStrip(Properties.Resources.Perforce);
			StreamsTreeView.ImageList = perforceImageList;

			// Build a map of stream names to their nodes
			Dictionary<string, StreamNode> identifierToNode = new Dictionary<string, StreamNode>(StringComparer.InvariantCultureIgnoreCase);
			foreach (StreamsRecord stream in streams)
			{
				if (stream.Stream != null && stream.Name != null)
				{
					identifierToNode[stream.Stream] = new StreamNode(stream);
				}
			}

			// Create all the depots
			Dictionary<string, StreamDepot> nameToDepot = new Dictionary<string, StreamDepot>(StringComparer.InvariantCultureIgnoreCase);
			foreach (StreamNode node in identifierToNode.Values)
			{
				if (node.Record.Parent == null || node.Record.Parent.Equals("none", StringComparison.OrdinalIgnoreCase))
				{
					string? depotName;
					if (PerforceUtils.TryGetDepotName(node.Record.Stream, out depotName))
					{
						StreamDepot? depot;
						if (!nameToDepot.TryGetValue(depotName, out depot))
						{
							depot = new StreamDepot(depotName);
							nameToDepot.Add(depotName, depot);
						}
						depot.RootNodes.Add(node);
					}
				}
				else
				{
					StreamNode? parentNode;
					if (identifierToNode.TryGetValue(node.Record.Parent, out parentNode))
					{
						parentNode.ChildNodes.Add(node);
					}
				}
			}

			// Sort the tree
			_depots = nameToDepot.Values.OrderBy(x => x.Name).ToList();
			foreach (StreamDepot depot in _depots)
			{
				depot.Sort();
			}

			// Update the contents of the tree
			PopulateTree();
			UpdateOkButton();
		}

		private static void GetExpandedNodes(TreeNodeCollection nodes, List<TreeNode> expandedNodes)
		{
			foreach (TreeNode? node in nodes)
			{
				if (node != null)
				{
					expandedNodes.Add(node);
					if (node.IsExpanded)
					{
						GetExpandedNodes(node.Nodes, expandedNodes);
					}
				}
			}
		}

		private void MoveSelection(int delta)
		{
			if (StreamsTreeView.SelectedNode != null)
			{
				List<TreeNode> expandedNodes = new List<TreeNode>();
				GetExpandedNodes(StreamsTreeView.Nodes, expandedNodes);

				int idx = expandedNodes.IndexOf(StreamsTreeView.SelectedNode);
				if (idx != -1)
				{
					int nextIdx = idx + delta;
					if (nextIdx < 0)
					{
						nextIdx = 0;
					}
					if (nextIdx >= expandedNodes.Count)
					{
						nextIdx = expandedNodes.Count - 1;
					}
					StreamsTreeView.SelectedNode = expandedNodes[nextIdx];
				}
			}
		}

		protected override bool ProcessCmdKey(ref Message msg, Keys keyData)
		{
			if (keyData == Keys.Up)
			{
				MoveSelection(-1);
				return true;
			}
			else if (keyData == Keys.Down)
			{
				MoveSelection(+1);
				return true;
			}
			return base.ProcessCmdKey(ref msg, keyData);
		}

		private static bool IncludeNodeInFilter(StreamNode node, string[] filter)
		{
			return filter.All(x => node.Record.Stream.Contains(x, StringComparison.InvariantCultureIgnoreCase) || node.Record.Name.Contains(x, StringComparison.InvariantCultureIgnoreCase));
		}

		private static bool TryFilterTree(StreamNode node, string[] filter, [NotNullWhen(true)] out StreamNode? newNode)
		{
			StreamNode filteredNode = new StreamNode(node.Record);
			foreach (StreamNode childNode in node.ChildNodes)
			{
				StreamNode? filteredChildNode;
				if (TryFilterTree(childNode, filter, out filteredChildNode))
				{
					filteredNode.ChildNodes.Add(filteredChildNode);
				}
			}

			if (filteredNode.ChildNodes.Count > 0 || IncludeNodeInFilter(filteredNode, filter))
			{
				newNode = filteredNode;
				return true;
			}
			else
			{
				newNode = null;
				return false;
			}
		}

		private void PopulateTree()
		{
			StreamsTreeView.BeginUpdate();
			StreamsTreeView.Nodes.Clear();

			string[] filter = FilterTextBox.Text.Split(new char[] { ' ' }, StringSplitOptions.RemoveEmptyEntries);

			List<StreamDepot> filteredDepots = _depots;
			if (filter.Length > 0)
			{
				filteredDepots = new List<StreamDepot>();
				foreach (StreamDepot depot in _depots)
				{
					StreamDepot filteredDepot = new StreamDepot(depot.Name);
					foreach (StreamNode rootNode in depot.RootNodes)
					{
						StreamNode? filteredRootNode;
						if (TryFilterTree(rootNode, filter, out filteredRootNode))
						{
							filteredDepot.RootNodes.Add(filteredRootNode);
						}
					}
					if (filteredDepot.RootNodes.Count > 0)
					{
						filteredDepots.Add(filteredDepot);
					}
				}
			}

			bool expandAll = filter.Length > 0;
			foreach (StreamDepot depot in filteredDepots)
			{
				TreeNode depotTreeNode = new TreeNode(depot.Name);
				depotTreeNode.ImageIndex = 1;
				depotTreeNode.SelectedImageIndex = 1;
				StreamsTreeView.Nodes.Add(depotTreeNode);

				bool expand = expandAll;
				foreach (StreamNode rootNode in depot.RootNodes)
				{
					expand |= AddStreamNodeToTree(rootNode, filter, depotTreeNode, expandAll);
				}
				if (expand)
				{
					depotTreeNode.Expand();
				}
			}

			if (StreamsTreeView.SelectedNode == null && filter.Length > 0 && StreamsTreeView.Nodes.Count > 0)
			{
				for (TreeNode node = StreamsTreeView.Nodes[0]; ; node = node.Nodes[0])
				{
					StreamNode? stream = node.Tag as StreamNode;
					if (stream != null && IncludeNodeInFilter(stream, filter))
					{
						StreamsTreeView.SelectedNode = node;
						break;
					}
					if (node.Nodes.Count == 0)
					{
						break;
					}
				}
			}

			if (StreamsTreeView.SelectedNode != null)
			{
				StreamsTreeView.SelectedNode.EnsureVisible();
			}
			else if (StreamsTreeView.Nodes.Count > 0)
			{
				StreamsTreeView.Nodes[0].EnsureVisible();
			}
			StreamsTreeView.EndUpdate();

			UpdateOkButton();
		}

		private bool AddStreamNodeToTree(StreamNode stream, string[] filter, TreeNode parentTreeNode, bool expandAll)
		{
			TreeNode streamTreeNode = new TreeNode(stream.Record.Name);
			streamTreeNode.ImageIndex = 0;
			streamTreeNode.SelectedImageIndex = 0;
			streamTreeNode.Tag = stream;
			parentTreeNode.Nodes.Add(streamTreeNode);

			if (stream.Record.Name == _selectedStream && IncludeNodeInFilter(stream, filter))
			{
				StreamsTreeView.SelectedNode = streamTreeNode;
			}

			bool expand = expandAll;
			foreach (StreamNode childNode in stream.ChildNodes)
			{
				expand |= AddStreamNodeToTree(childNode, filter, streamTreeNode, expandAll);
			}
			if (expand)
			{
				streamTreeNode.Expand();
			}
			return expand || (_selectedStream == stream.Record.Stream);
		}

		public static bool ShowModal(IWin32Window owner, IPerforceSettings perforce, string? streamName, IServiceProvider serviceProvider, [NotNullWhen(true)] out string? newStreamName)
		{
			ILogger logger = serviceProvider.GetRequiredService<ILogger<SelectStreamWindow>>();

			ModalTask<List<StreamsRecord>>? streamsTask = PerforceModalTask.Execute(owner, "Finding streams", "Finding streams, please wait...", perforce, EnumerateStreamsTask.RunAsync, logger);
			if (streamsTask == null || !streamsTask.Succeeded)
			{
				newStreamName = null;
				return false;
			}

			using SelectStreamWindow selectStream = new SelectStreamWindow(streamsTask.Result, streamName);
			if (selectStream.ShowDialog(owner) == DialogResult.OK && selectStream._selectedStream != null)
			{
				newStreamName = selectStream._selectedStream;
				return true;
			}
			else
			{
				newStreamName = null;
				return false;
			}
		}

		private void FilterTextBox_TextChanged(object sender, EventArgs e)
		{
			PopulateTree();
		}

		private void UpdateOkButton()
		{
			OkBtn.Enabled = (GetSelectedStream() != null);
		}

		private string? GetSelectedStream()
		{
			string? newSelectedStream = null;
			if (StreamsTreeView.SelectedNode != null)
			{
				if (StreamsTreeView.SelectedNode.Tag is StreamNode streamNode)
				{
					newSelectedStream = streamNode.Record.Stream;
				}
			}
			return newSelectedStream;
		}

		private void UpdateSelectedStream()
		{
			_selectedStream = GetSelectedStream();
		}

		private void StreamsTreeView_AfterSelect(object sender, TreeViewEventArgs e)
		{
			if (e.Action != TreeViewAction.Unknown)
			{
				UpdateSelectedStream();
			}
			UpdateOkButton();
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			UpdateSelectedStream();

			if (_selectedStream != null)
			{
				DialogResult = DialogResult.OK;
				Close();
			}
		}

		private void CancelBtn_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
			Close();
		}
	}
}
