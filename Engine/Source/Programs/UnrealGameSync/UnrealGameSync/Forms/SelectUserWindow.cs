// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace UnrealGameSync
{
	partial class SelectUserWindow : ThemedForm
	{
		static class EnumerateUsersTask
		{
			public static async Task<List<UsersRecord>> RunAsync(IPerforceConnection perforce, CancellationToken cancellationToken)
			{
				return await perforce.GetUsersAsync(UsersOptions.None, -1, cancellationToken);
			}
		}

		private int _selectedUserIndex;
		private readonly List<UsersRecord> _users;

		private SelectUserWindow(List<UsersRecord> users, int selectedUserIndex)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_selectedUserIndex = selectedUserIndex;
			_users = users;

			PopulateList();
			UpdateOkButton();
		}

		private void MoveSelection(int delta)
		{
			if (UserListView.Items.Count > 0)
			{
				int currentIndex = -1;
				if (UserListView.SelectedIndices.Count > 0)
				{
					currentIndex = UserListView.SelectedIndices[0];
				}

				int nextIndex = currentIndex + delta;
				if (nextIndex < 0)
				{
					nextIndex = 0;
				}
				else if (nextIndex >= UserListView.Items.Count)
				{
					nextIndex = UserListView.Items.Count - 1;
				}

				if (currentIndex != nextIndex)
				{
					if (currentIndex != -1)
					{
						UserListView.Items[currentIndex].Selected = false;
					}

					UserListView.Items[nextIndex].Selected = true;
					_selectedUserIndex = (int)UserListView.Items[nextIndex].Tag!;
				}
			}
		}

		protected override bool ProcessCmdKey(ref Message msg, Keys keyData)
		{
			if (keyData == Keys.Up)
			{
				MoveSelection(-1);
				return true;
			}
			else if (keyData == Keys.Down)
			{
				MoveSelection(+1);
				return true;
			}
			return base.ProcessCmdKey(ref msg, keyData);
		}

		private static bool IncludeInFilter(UsersRecord user, string[] filterWords)
		{
			foreach (string filterWord in filterWords)
			{
				if (!user.UserName.Contains(filterWord, StringComparison.OrdinalIgnoreCase)
					&& !user.FullName.Contains(filterWord, StringComparison.OrdinalIgnoreCase)
					&& !user.Email.Contains(filterWord, StringComparison.OrdinalIgnoreCase))
				{
					return false;
				}
			}
			return true;
		}

		private void PopulateList()
		{
			UserListView.BeginUpdate();
			UserListView.Items.Clear();

			int selectedItemIndex = -1;

			string[] filter = FilterTextBox.Text.Split(new char[] { ' ' }, StringSplitOptions.RemoveEmptyEntries);
			for (int idx = 0; idx < _users.Count; idx++)
			{
				UsersRecord user = _users[idx];
				if (IncludeInFilter(user, filter))
				{
					ListViewItem item = new ListViewItem(user.UserName);
					item.SubItems.Add(new ListViewItem.ListViewSubItem(item, user.FullName));
					item.SubItems.Add(new ListViewItem.ListViewSubItem(item, user.Email));
					item.Tag = (int)idx;
					UserListView.Items.Add(item);

					if (selectedItemIndex == -1 && idx >= _selectedUserIndex)
					{
						selectedItemIndex = UserListView.Items.Count - 1;
						item.Selected = true;
					}
				}
			}

			if (selectedItemIndex == -1 && UserListView.Items.Count > 0)
			{
				selectedItemIndex = UserListView.Items.Count - 1;
				UserListView.Items[selectedItemIndex].Selected = true;
			}

			if (selectedItemIndex != -1)
			{
				UserListView.EnsureVisible(selectedItemIndex);
			}

			UserListView.EndUpdate();

			UpdateOkButton();
		}

		public static bool ShowModal(IWin32Window owner, IPerforceSettings perforceSettings, IServiceProvider serviceProvider, [NotNullWhen(true)] out string? selectedUserName)
		{
			ILogger<SelectUserWindow> logger = serviceProvider.GetRequiredService<ILogger<SelectUserWindow>>();

			ModalTask<List<UsersRecord>>? usersTask = PerforceModalTask.Execute(owner, "Finding users", "Finding users, please wait...", perforceSettings, EnumerateUsersTask.RunAsync, logger);
			if (usersTask == null || !usersTask.Succeeded)
			{
				selectedUserName = null;
				return false;
			}

			using SelectUserWindow selectUser = new SelectUserWindow(usersTask.Result, 0);
			if (selectUser.ShowDialog(owner) == DialogResult.OK)
			{
				selectedUserName = usersTask.Result[selectUser._selectedUserIndex].UserName;
				return true;
			}
			else
			{
				selectedUserName = null;
				return false;
			}
		}

		private void FilterTextBox_TextChanged(object sender, EventArgs e)
		{
			PopulateList();
		}

		private void UpdateOkButton()
		{
			OkBtn.Enabled = UserListView.SelectedItems.Count > 0;
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			if (UserListView.SelectedItems.Count > 0)
			{
				_selectedUserIndex = (int)UserListView.SelectedItems[0].Tag!;
				DialogResult = DialogResult.OK;
				Close();
			}
		}

		private void CancelBtn_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
			Close();
		}

		private void UserListView_SelectedIndexChanged(object sender, EventArgs e)
		{
			UpdateOkButton();
		}
	}
}
