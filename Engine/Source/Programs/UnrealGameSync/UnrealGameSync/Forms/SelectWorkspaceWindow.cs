// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace UnrealGameSync
{
	partial class SelectWorkspaceWindow : ThemedForm
	{
		class EnumerateWorkspaces
		{
			public InfoRecord Info { get; }
			public List<ClientsRecord> Clients { get; }

			public EnumerateWorkspaces(InfoRecord info, List<ClientsRecord> clients)
			{
				Info = info;
				Clients = clients;
			}

			public static async Task<EnumerateWorkspaces> RunAsync(IPerforceConnection perforce, CancellationToken cancellationToken)
			{
				InfoRecord info = await perforce.GetInfoAsync(InfoOptions.ShortOutput, cancellationToken);
				List<ClientsRecord> clients = await perforce.GetClientsAsync(ClientsOptions.None, perforce.Settings.UserName, cancellationToken);
				return new EnumerateWorkspaces(info, clients);
			}
		}

		readonly InfoRecord _info;
		readonly List<ClientsRecord> _clients;
		string? _workspaceName;

		private SelectWorkspaceWindow(InfoRecord info, List<ClientsRecord> clients, string? workspaceName)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			_info = info;
			_clients = clients;
			_workspaceName = workspaceName;

			UpdateListView();
			UpdateOkButton();
		}

		private void UpdateListView()
		{
			if (WorkspaceListView.SelectedItems.Count > 0)
			{
				_workspaceName = WorkspaceListView.SelectedItems[0].Text;
			}
			else
			{
				_workspaceName = null;
			}

			WorkspaceListView.Items.Clear();

			foreach (ClientsRecord client in _clients.OrderBy(x => x.Name))
			{
				if (!OnlyForThisComputer.Checked || String.Equals(client.Host, _info.ClientHost, StringComparison.OrdinalIgnoreCase))
				{
					ListViewItem item = new ListViewItem(client.Name);
					item.SubItems.Add(new ListViewItem.ListViewSubItem(item, client.Host));
					item.SubItems.Add(new ListViewItem.ListViewSubItem(item, client.Stream));
					item.SubItems.Add(new ListViewItem.ListViewSubItem(item, client.Root));
					item.Selected = (_workspaceName == client.Name);
					WorkspaceListView.Items.Add(item);
				}
			}
		}

		private void UpdateOkButton()
		{
			OkBtn.Enabled = (WorkspaceListView.SelectedItems.Count == 1);
		}

		public static bool ShowModal(IWin32Window owner, IPerforceSettings perforce, string workspaceName, IServiceProvider serviceProvider, out string? newWorkspaceName)
		{
			ModalTask<EnumerateWorkspaces>? task = PerforceModalTask.Execute(owner, "Finding workspaces", "Finding workspaces, please wait...", perforce, EnumerateWorkspaces.RunAsync, serviceProvider.GetRequiredService<ILogger<EnumerateWorkspaces>>());
			if (task == null || !task.Succeeded)
			{
				newWorkspaceName = null;
				return false;
			}

			using SelectWorkspaceWindow selectWorkspace = new SelectWorkspaceWindow(task.Result.Info, task.Result.Clients, workspaceName);
			if (selectWorkspace.ShowDialog(owner) == DialogResult.OK)
			{
				newWorkspaceName = selectWorkspace._workspaceName;
				return true;
			}
			else
			{
				newWorkspaceName = null;
				return false;
			}
		}

		private void OnlyForThisComputer_CheckedChanged(object sender, EventArgs e)
		{
			UpdateListView();
		}

		private void WorkspaceListView_SelectedIndexChanged(object sender, EventArgs e)
		{
			UpdateOkButton();
		}

		private void OkBtn_Click(object sender, EventArgs e)
		{
			if (WorkspaceListView.SelectedItems.Count > 0)
			{
				_workspaceName = WorkspaceListView.SelectedItems[0].Text;
				DialogResult = DialogResult.OK;
				Close();
			}
		}

		private void CancelBtn_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
			Close();
		}

		private void WorkspaceListView_MouseDoubleClick(object sender, MouseEventArgs e)
		{
			if (WorkspaceListView.SelectedItems.Count > 0)
			{
				_workspaceName = WorkspaceListView.SelectedItems[0].Text;
				DialogResult = DialogResult.OK;
				Close();
			}
		}
	}
}
