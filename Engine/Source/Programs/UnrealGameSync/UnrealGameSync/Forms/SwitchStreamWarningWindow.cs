// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Windows.Forms;

namespace UnrealGameSync.Forms
{
	public partial class SwitchStreamWarningWindow : ThemedForm
	{
		public SwitchStreamWarningWindow(List<string> openedFiles)
		{
			InitializeComponent();

			foreach (string openedFile in openedFiles)
			{
				OpenedFiles.Items.Add(openedFile);
			}
		}

		private void OkButton_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.OK;
		}

		private void CancButton_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
		}
	}
}
