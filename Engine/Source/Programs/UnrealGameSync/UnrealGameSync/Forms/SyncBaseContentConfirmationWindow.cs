// Copyright Epic Games, Inc. All Rights Reserved.

using System.Drawing;
using System.Windows.Forms;

namespace UnrealGameSync.Forms
{
	public partial class SyncBaseContentConfirmationWindow : ThemedForm
	{
		public SyncBaseContentConfirmationWindow(string baseContentPaths)
		{
			InitializeComponent();
			BaseContentPaths.Text = baseContentPaths;

			// Style buttons for dark mode visibility
			SyncBaseContentButton.BackColor = Color.FromArgb(0, 122, 204);
			SyncBaseContentButton.ForeColor = Color.White;
			SyncBaseContentButton.FlatStyle = FlatStyle.Flat;
			SyncBaseContentButton.FlatAppearance.BorderColor = Color.FromArgb(0, 122, 204);

			DoNotSyncBaseContentButton.BackColor = Color.FromArgb(90, 90, 90);
			DoNotSyncBaseContentButton.ForeColor = Color.White;
			DoNotSyncBaseContentButton.FlatStyle = FlatStyle.Flat;
			DoNotSyncBaseContentButton.FlatAppearance.BorderColor = Color.FromArgb(90, 90, 90);
		}
	}
}
