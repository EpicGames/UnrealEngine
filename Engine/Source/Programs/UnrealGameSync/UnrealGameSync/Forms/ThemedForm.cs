// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Windows.Forms;

namespace UnrealGameSync
{
	public class ThemedForm : Form
	{
		protected override void OnHandleCreated(EventArgs e)
		{
			base.OnHandleCreated(e);

			ThemeManager?.ApplyThemeRecursively(this);
		}
		
		protected static ApplicationThemeManager? ThemeManager => ProgramApplicationContext.GetThemeManager();
		protected static ApplicationTheme Theme => ProgramApplicationContext.GetTheme();
	}
}
