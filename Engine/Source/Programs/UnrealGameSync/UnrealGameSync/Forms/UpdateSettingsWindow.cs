// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	partial class UpdateSettingsWindow : ThemedForm
	{
		[DllImport("user32.dll")]
		private static extern IntPtr SendMessage(IntPtr hWnd, int msg, int wParam, [MarshalAs(UnmanagedType.LPWStr)] string lParam);

		public delegate Task SyncAndRunDelegate(IPerforceConnection? perforce, LauncherSettings settings, ILogger logWriter, CancellationToken cancellationToken);

		const int EmSetcuebanner = 0x1501;

		readonly LauncherSettings _settings;
		string? _logText;
		readonly SyncAndRunDelegate _syncAndRun;

		public UpdateSettingsWindow(string? prompt, string? logText, LauncherSettings settings, SyncAndRunDelegate syncAndRun)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			if (prompt != null)
			{
				PromptLabel.Text = prompt;
			}

			_logText = logText;
			_settings = settings;

			string defaultHordeServer = DeploymentSettings.Instance.HordeUrl ?? String.Empty;

			if (settings.UpdateSource == LauncherUpdateSource.Perforce)
			{
				PerforceRadioBtn.Checked = true;
			}
			else if (settings.UpdateSource == LauncherUpdateSource.None)
			{
				DisableRadioBtn.Checked = true;
			}
			else
			{
				HordeRadioBtn.Checked = true;
			}

			HordeServerTextBox.Text = String.IsNullOrEmpty(settings.HordeServer) ? defaultHordeServer : settings.HordeServer;
			ServerTextBox.Text = settings.PerforceServerAndPort ?? String.Empty;
			UserNameTextBox.Text = settings.PerforceUserName ?? String.Empty;
			DepotPathTextBox.Text = settings.PerforceDepotPath ?? String.Empty;
			UsePreviewBuildCheckBox.Checked = settings.PreviewBuild;

			_syncAndRun = syncAndRun;

			ViewLogBtn.Visible = logText != null;
		}

		public LauncherResult ShowModal()
		{
			if (ShowDialog() == DialogResult.Cancel)
			{
				return LauncherResult.Exit;
			}
			else if (_settings.UpdateSource == LauncherUpdateSource.None)
			{
				return LauncherResult.Continue;
			}
			else
			{
				return LauncherResult.Exit;
			}
		}

		protected override void OnLoad(EventArgs e)
		{
			base.OnLoad(e);

			SendMessage(ServerTextBox.Handle, EmSetcuebanner, 1, "Default Server");
			SendMessage(UserNameTextBox.Handle, EmSetcuebanner, 1, "Default User");
		}

		private void ViewLogBtn_Click(object sender, EventArgs e)
		{
			using LogWindow log = new LogWindow(_logText ?? String.Empty);
			log.ShowDialog(this);
		}

		private void ConnectBtn_Click(object sender, EventArgs e)
		{
			_settings.UpdateSource = HordeRadioBtn.Checked ? LauncherUpdateSource.Horde : PerforceRadioBtn.Checked ? LauncherUpdateSource.Perforce : LauncherUpdateSource.None;
			_settings.HordeServer = HordeServerTextBox.Text.Trim();
			if (String.Equals(_settings.HordeServer, DeploymentSettings.Instance.HordeUrl, StringComparison.OrdinalIgnoreCase))
			{
				_settings.HordeServer = null;
			}
			_settings.PerforceServerAndPort = ServerTextBox.Text.Trim();
			_settings.PerforceUserName = UserNameTextBox.Text.Trim();
			_settings.PerforceDepotPath = DepotPathTextBox.Text.Trim();
			_settings.PreviewBuild = UsePreviewBuildCheckBox.Checked;
			_settings.Save();

			// Create the P4 connection
			CaptureLogger logger = new CaptureLogger();

			// Create the task for connecting to this server
			ModalTask? task = null;
			if (_settings.UpdateSource == LauncherUpdateSource.Horde)
			{
				task = ModalTask.Execute(this, "Updating", "Checking for updates, please wait...", c => _syncAndRun(null, _settings, logger, c));
			}
			else if (_settings.UpdateSource == LauncherUpdateSource.Perforce)
			{
				PerforceSettings perforceSettings = new PerforceSettings(PerforceSettings.Default);
				if (!String.IsNullOrEmpty(_settings.PerforceServerAndPort))
				{
					perforceSettings.ServerAndPort = _settings.PerforceServerAndPort;
				}
				if (!String.IsNullOrEmpty(_settings.PerforceUserName))
				{
					perforceSettings.UserName = _settings.PerforceUserName;
				}
				perforceSettings.PreferNativeClient = true;

				task = PerforceModalTask.Execute(this, "Updating", "Checking for updates, please wait...", perforceSettings, (p, c) => _syncAndRun(p, _settings, logger, c), logger);
			}

			if (task == null || task.Succeeded)
			{
				_settings.Save();
				DialogResult = DialogResult.OK;
				Close();
				return;
			}

			PromptLabel.Text = task.Error;

			_logText = logger.Render(Environment.NewLine);
			ViewLogBtn.Visible = true;
		}

		private void CancelBtn_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
			Close();
		}

		private void HordeRadioBtn_CheckedChanged(object sender, EventArgs e)
		{
			HordeGroupBox.Enabled = HordeRadioBtn.Checked;
			PerforceGroupBox.Enabled = PerforceRadioBtn.Checked;
			DisableGroupBox.Enabled = DisableRadioBtn.Checked;
		}

		private void PerforceRadioBtn_CheckedChanged(object sender, EventArgs e)
		{
			HordeGroupBox.Enabled = HordeRadioBtn.Checked;
			PerforceGroupBox.Enabled = PerforceRadioBtn.Checked;
			DisableGroupBox.Enabled = DisableRadioBtn.Checked;
		}

		private void DisableRadioBtn_CheckedChanged(object sender, EventArgs e)
		{
			HordeGroupBox.Enabled = HordeRadioBtn.Checked;
			PerforceGroupBox.Enabled = PerforceRadioBtn.Checked;
			DisableGroupBox.Enabled = DisableRadioBtn.Checked;
		}
	}
}
