// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Linq;
using System.Windows.Forms;

#nullable enable

namespace UnrealGameSync
{
	public partial class VariablesWindow : ThemedForm
	{
		static readonly HashSet<string> s_legacyVariables = new HashSet<string>()
		{
			"UE4EditorConfig",
			"UE4EditorDebugArg",
			"UE4EditorExe",
			"UE4EditorCmdExe",
			"UseIncrementalBuilds",
		};

		public delegate void InsertVariableDelegate(string name);

		public event InsertVariableDelegate? OnInsertVariable;

		public VariablesWindow(IReadOnlyDictionary<string, string> variables)
		{
			InitializeComponent();
			Font = new System.Drawing.Font("Segoe UI", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ((byte)(0)));

			ListViewGroup currentProjectGroup = new ListViewGroup("Current Project");
			MacrosList.Groups.Add(currentProjectGroup);

			ListViewGroup environmentGroup = new ListViewGroup("Environment");
			MacrosList.Groups.Add(environmentGroup);

			foreach (KeyValuePair<string, string> pair in variables)
			{
				if (!s_legacyVariables.Contains(pair.Key))
				{
					ListViewItem item = new ListViewItem(String.Format("$({0})", pair.Key));
					item.SubItems.Add(pair.Value);
					item.Group = currentProjectGroup;
					MacrosList.Items.Add(item);
				}
			}

			foreach (DictionaryEntry entry in Environment.GetEnvironmentVariables().OfType<DictionaryEntry>())
			{
				string? key = entry.Key?.ToString();
				if (key != null && entry.Value != null && !variables.ContainsKey(key))
				{
					ListViewItem item = new ListViewItem(String.Format("$({0})", key));
					item.SubItems.Add(entry.Value.ToString());
					item.Group = environmentGroup;
					MacrosList.Items.Add(item);
				}
			}
		}

		protected override bool ShowWithoutActivation => true;

		private void OkButton_Click(object sender, EventArgs e)
		{
			DialogResult = System.Windows.Forms.DialogResult.OK;
			Close();
		}

		private void MacrosList_MouseDoubleClick(object sender, MouseEventArgs args)
		{
			if (args.Button == MouseButtons.Left)
			{
				ListViewHitTestInfo hitTest = MacrosList.HitTest(args.Location);
				if (hitTest.Item != null && OnInsertVariable != null)
				{
					OnInsertVariable(hitTest.Item.Text);
				}
			}
		}

		private void InsertButton_Click(object sender, EventArgs e)
		{
			if (MacrosList.SelectedItems.Count > 0 && OnInsertVariable != null)
			{
				OnInsertVariable(MacrosList.SelectedItems[0].Text);
			}
		}

		private void MacrosList_SelectedIndexChanged(object sender, EventArgs e)
		{
			InsertButton.Enabled = (MacrosList.SelectedItems.Count > 0);
		}
	}
}
