// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Globalization;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace UnrealGameSync
{
	public enum IssueBuildOutcome
	{
		Unknown,
		Success,
		Error,
		Warning,
	}

	public class IssueBuildData
	{
		public long Id { get; set; }
		public string Stream { get; set; } = String.Empty;
		public int Change { get; set; }
		public string JobName { get; set; } = String.Empty;
		public string JobUrl { get; set; } = String.Empty;
		public string JobStepName { get; set; } = String.Empty;
		public string JobStepUrl { get; set; } = String.Empty;
		public string ErrorUrl { get; set; } = String.Empty;
		public IssueBuildOutcome Outcome { get; set; }
	}

	public class IssueDiagnosticData
	{
		public long? BuildId { get; set; }
		public string Message { get; set; } = String.Empty;
		public string Url { get; set; } = String.Empty;
	}

	public class IssueData
	{
		public int Version { get; set; }
		public long Id { get; set; }
		public DateTime CreatedAt { get; set; }
		public DateTime RetrievedAt { get; set; }
		public string Project { get; set; } = String.Empty;
		public string Summary { get; set; } = String.Empty;
		public string Details { get; set; } = String.Empty;
		public string Owner { get; set; } = String.Empty;
		public string NominatedBy { get; set; } = String.Empty;
		public DateTime? AcknowledgedAt { get; set; }
		public int FixChange { get; set; }
		public DateTime? ResolvedAt { get; set; }
		public bool Notify { get; set; }
		public bool IsWarning { get; set; }
		public string BuildUrl { get; set; } = String.Empty;
		public List<string> Streams { get; } = new List<string>();

		HashSet<string>? _cachedProjects;

		public HashSet<string> Projects
		{
			get
			{
				// HACK to infer project names from streams
				if (_cachedProjects == null)
				{
					HashSet<string> newProjects = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
					if (!String.IsNullOrEmpty(Project))
					{
						newProjects.Add(Project);
					}
					if (Streams != null)
					{
						foreach (string stream in Streams)
						{
							Match match = Regex.Match(stream, "^//([^/]+)/");
							if (match.Success)
							{
								string project = match.Groups[1].Value;
								if (project.StartsWith("UE", StringComparison.OrdinalIgnoreCase))
								{
									project = "UE" + project.Substring(2);
								}
								else if (Char.IsLower(project[0]))
								{
									project = Char.ToUpper(project[0], CultureInfo.InvariantCulture) + project.Substring(1);
								}
								newProjects.Add(project);
							}
						}
					}
					if (newProjects.Count == 0)
					{
						newProjects.Add("Default");
					}
					_cachedProjects = newProjects;
				}
				return _cachedProjects;
			}
		}
	}

	public class IssueUpdateData
	{
		public long Id { get; set; }
		public string Owner { get; set; } = String.Empty;
		public string? NominatedBy { get; set; }
		public bool? Acknowledged { get; set; }
		public int? FixChange { get; set; }
		public bool? Resolved { get; set; }
	}

	[Flags]
	public enum IssueAlertReason
	{
		Normal = 1,
		Owner = 2,
		UnassignedTimer = 4,
		UnacknowledgedTimer = 8,
		UnresolvedTimer = 16,
	}

	class IssueMonitor : IDisposable
	{
		public string? ApiUrl { get; }
		public string UserName { get; }
		int _refCount = 1;
		Task? _workerTask;
		readonly ILogger _logger;
#pragma warning disable CA2213 // warning CA2213: 'IssueMonitor' contains field '_cancellationSource' that is of IDisposable type 'CancellationTokenSource', but it is never disposed. Change the Dispose method on 'IssueMonitor' to call Close or Dispose on this field.
		readonly CancellationTokenSource _cancellationSource;
#pragma warning restore CA2213
		readonly AsyncEvent _refreshEvent;
		int _updateIntervalMs;
		readonly List<long> _trackingIssueIds = new List<long>();
		List<IssueData> _issues = new List<IssueData>();
		readonly object _lockObject = new object();
		readonly List<IssueUpdateData> _pendingUpdates = new List<IssueUpdateData>();
		readonly IAsyncDisposer _asyncDisposer;

		public Action? OnIssuesChanged { get; set; }

		// Only used by MainWindow, but easier to just store here
		public Dictionary<long, IssueAlertReason> IssueIdToAlertReason { get; } = new Dictionary<long, IssueAlertReason>();

		public IssueMonitor(string? apiUrl, string userName, TimeSpan updateInterval, IServiceProvider serviceProvider)
		{
			ApiUrl = apiUrl;
			UserName = userName;
			_updateIntervalMs = (int)updateInterval.TotalMilliseconds;
			_logger = serviceProvider.GetRequiredService<ILogger<IssueMonitor>>();
			_cancellationSource = new CancellationTokenSource();
			_asyncDisposer = serviceProvider.GetRequiredService<IAsyncDisposer>();

			if (apiUrl == null)
			{
				LastStatusMessage = "Database functionality disabled due to empty ApiUrl.";
			}
			else
			{
				_logger.LogInformation("Using connection string: {ApiUrl}", ApiUrl);
			}

			_refreshEvent = new AsyncEvent();
		}

		public string LastStatusMessage
		{
			get;
			private set;
		} = String.Empty;

		public List<IssueData> GetIssues()
		{
			return _issues;
		}

		public TimeSpan GetUpdateInterval()
		{
			return TimeSpan.FromMilliseconds(_updateIntervalMs);
		}

		public void SetUpdateInterval(TimeSpan updateInterval)
		{
			_updateIntervalMs = (int)updateInterval.TotalMilliseconds;
			_refreshEvent.Set();
		}

		public void StartTracking(long issueId)
		{
			lock (_lockObject)
			{
				_trackingIssueIds.Add(issueId);
			}
			_refreshEvent.Set();
		}

		public void StopTracking(long issueId)
		{
			lock (_lockObject)
			{
				_trackingIssueIds.RemoveAt(_trackingIssueIds.IndexOf(issueId));
			}
		}

		public bool HasPendingUpdate()
		{
			return _pendingUpdates.Count > 0;
		}

		public void PostUpdate(IssueUpdateData update)
		{
			bool updatedIssues;
			lock (_lockObject)
			{
				_pendingUpdates.Add(update);
				updatedIssues = ApplyPendingUpdate(_issues, update);
			}

			_refreshEvent.Set();

			if (updatedIssues)
			{
				OnIssuesChanged?.Invoke();
			}
		}

		static bool ApplyPendingUpdate(List<IssueData> issues, IssueUpdateData update)
		{
			bool updated = false;
			for (int idx = 0; idx < issues.Count; idx++)
			{
				IssueData issue = issues[idx];
				if (update.Id == issue.Id)
				{
					if (update.Owner != null && update.Owner != issue.Owner)
					{
						issue.Owner = update.Owner;
						updated = true;
					}
					if (update.NominatedBy != null && update.NominatedBy != issue.NominatedBy)
					{
						issue.NominatedBy = update.NominatedBy;
						updated = true;
					}
					if (update.Acknowledged.HasValue && update.Acknowledged.Value != issue.AcknowledgedAt.HasValue)
					{
						issue.AcknowledgedAt = update.Acknowledged.Value ? (DateTime?)DateTime.UtcNow : null;
						updated = true;
					}
					if (update.FixChange.HasValue)
					{
						issue.FixChange = update.FixChange.Value;
						if (issue.FixChange != 0)
						{
							issues.RemoveAt(idx);
						}
						updated = true;
					}
					break;
				}
			}
			return updated;
		}

		public void Start()
		{
			if (ApiUrl != null)
			{
				_workerTask = Task.Run(() => PollForUpdatesAsync(_cancellationSource.Token));
			}
		}

		public void AddRef()
		{
			if (_refCount == 0)
			{
				throw new Exception("Invalid reference count for IssueMonitor (zero)");
			}
			_refCount++;
		}

		public void Release()
		{
			_refCount--;
			if (_refCount < 0)
			{
				throw new Exception("Invalid reference count for IssueMonitor (ltz)");
			}
			if (_refCount == 0)
			{
				DisposeInternal();
			}
		}

		void IDisposable.Dispose()
		{
			DisposeInternal();
		}

		void DisposeInternal()
		{
			OnIssuesChanged = null;

			if (_workerTask != null)
			{
				_cancellationSource.Cancel();
				_asyncDisposer.Add(_workerTask.ContinueWith(_ => _cancellationSource.Dispose(), TaskScheduler.Default));
				_workerTask = null;
			}
		}

		async Task PollForUpdatesAsync(CancellationToken cancellationToken)
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				Task refreshTask = _refreshEvent.Task;

				// Check if there's any pending update
				IssueUpdateData? pendingUpdate;
				lock (_lockObject)
				{
					if (_pendingUpdates.Count > 0)
					{
						pendingUpdate = _pendingUpdates[0];
					}
					else
					{
						pendingUpdate = null;
					}
				}

				// If we have an update, try to post it to the backend and check for another
				if (pendingUpdate != null)
				{
					if (await SendUpdateAsync(pendingUpdate, cancellationToken))
					{
						lock (_lockObject)
						{
							_pendingUpdates.RemoveAt(0);
						}
					}
					else
					{
						await Task.WhenAny(refreshTask, Task.Delay(TimeSpan.FromSeconds(5.0), cancellationToken));
					}
					continue;
				}

				// Read all the current issues
				await ReadCurrentIssuesAsync(cancellationToken);

				// Wait for something else to do
				await Task.WhenAny(refreshTask, Task.Delay(_updateIntervalMs, cancellationToken));
			}
		}

		async Task<bool> SendUpdateAsync(IssueUpdateData update, CancellationToken cancellationToken)
		{
			try
			{
				await RestApi.PutAsync<IssueUpdateData>($"{ApiUrl}/api/issues/{update.Id}", update, cancellationToken);
				return true;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed with exception.");
				LastStatusMessage = String.Format("Failed to send update: ({0})", ex.ToString());
				return false;
			}
		}

		async Task<bool> ReadCurrentIssuesAsync(CancellationToken cancellationToken)
		{
			try
			{
				Stopwatch timer = Stopwatch.StartNew();
				_logger.LogInformation("Polling for issues...");

				// Get the initial number of issues. We won't post updates if this stays at zero.
				int initialNumIssues = _issues.Count;

				// Fetch the new issues
				List<IssueData> newIssues = await RestApi.GetAsync<List<IssueData>>($"{ApiUrl}/api/issues?user={UserName}", cancellationToken);

				// Check if we're tracking a particular issue. If so, we want updates even when it's resolved.
				long[] localTrackingIssueIds;
				lock (_lockObject)
				{
					localTrackingIssueIds = _trackingIssueIds.Distinct().ToArray();
				}
				foreach (long localTrackingIssueId in localTrackingIssueIds)
				{
					if (!newIssues.Any(x => x.Id == localTrackingIssueId))
					{
						try
						{
							IssueData issue = await RestApi.GetAsync<IssueData>($"{ApiUrl}/api/issues/{localTrackingIssueId}", cancellationToken);
							if (issue != null)
							{
								newIssues.Add(issue);
							}
						}
						catch (Exception ex)
						{
							_logger.LogError(ex, "Exception while fetching tracked issue");
						}
					}
				}

				// Update all the builds for each issue
				foreach (IssueData newIssue in newIssues)
				{
					if (newIssue.Version == 0)
					{
						List<IssueBuildData> builds = await RestApi.GetAsync<List<IssueBuildData>>($"{ApiUrl}/api/issues/{newIssue.Id}/builds", cancellationToken);
						if (builds != null && builds.Count > 0)
						{
							newIssue.IsWarning = !builds.Any(x => x.Outcome != IssueBuildOutcome.Warning);

							IssueBuildData? lastBuild = builds.OrderByDescending(x => x.Change).FirstOrDefault();
							if (lastBuild != null && !String.IsNullOrEmpty(lastBuild.ErrorUrl))
							{
								newIssue.BuildUrl = lastBuild.ErrorUrl;
							}
						}
					}
				}

				// Apply any pending updates to this issue list, and update it
				lock (_lockObject)
				{
					foreach (IssueUpdateData pendingUpdate in _pendingUpdates)
					{
						ApplyPendingUpdate(newIssues, pendingUpdate);
					}
					_issues = newIssues;
				}

				// Update the main thread
				if (initialNumIssues > 0 || _issues.Count > 0)
				{
					OnIssuesChanged?.Invoke();
				}

				// Update the stats
				LastStatusMessage = String.Format("Last update took {0}ms", timer.ElapsedMilliseconds);
				_logger.LogInformation("Done in {Time}ms.", timer.ElapsedMilliseconds);
				return true;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed with exception.");
				LastStatusMessage = String.Format("Last update failed: ({0})", ex.ToString());
				return false;
			}
		}
	}
}
