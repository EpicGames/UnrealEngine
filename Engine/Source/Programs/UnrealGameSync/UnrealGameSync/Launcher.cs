// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Reflection;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Tools;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	enum LauncherResult
	{
		Continue, // Continue to run this instance
		Exit, // Other instance has been spawned
	}

	static class Launcher
	{
		// Returns true if the application should keep running, false if it should quit.
		public static LauncherResult SyncAndRunLatest(Mutex instanceMutex, string[] args)
		{
			// Don't do this if we're already running as a spawned instance
			if (args.Any(x => x.StartsWith("-updatespawn=", StringComparison.OrdinalIgnoreCase)))
			{
				return LauncherResult.Continue;
			}

			// Also check we're not running under the 
			DirectoryReference applicationFolder = new DirectoryReference(GetSyncFolder());

			// add application folder to the path
			try
			{
				string? path = Environment.GetEnvironmentVariable("Path", EnvironmentVariableTarget.User);
				if (String.IsNullOrWhiteSpace(path))
				{
					Environment.SetEnvironmentVariable("Path", applicationFolder.FullName, EnvironmentVariableTarget.User);
				}
				else if (!path.Contains(applicationFolder.FullName, StringComparison.OrdinalIgnoreCase))
				{
					Environment.SetEnvironmentVariable("Path", path + $";{applicationFolder.FullName}", EnvironmentVariableTarget.User);
				}
			}
			catch (Exception)
			{
				// Ignore failures to modify the path
			}

			if (new FileReference(Assembly.GetExecutingAssembly().Location).IsUnderDirectory(applicationFolder))
			{
				return LauncherResult.Continue;
			}

			// Figure out if we should sync the unstable build by default
			bool preview = args.Contains("-unstable", StringComparer.InvariantCultureIgnoreCase) || args.Contains("-preview", StringComparer.InvariantCultureIgnoreCase);
			bool openSettings = args.Contains("-settings", StringComparer.OrdinalIgnoreCase);

			// Read the settings
			LauncherSettings launcherSettings = new LauncherSettings();
			launcherSettings.Read();

			async Task SyncAndRunWrapper(IPerforceConnection? perforce, LauncherSettings settings, ILogger logWriter, CancellationToken cancellationToken)
			{
				List<string> childArgs = args.Except(new[] { "-settings", "-updatecheck", "-noupdatecheck" }, StringComparer.OrdinalIgnoreCase).ToList();
				await SyncAndRun(perforce, settings, args, instanceMutex, logWriter, cancellationToken);
			}

			// If the shift key is held down, immediately show the settings window
			if ((Control.ModifierKeys & Keys.Shift) != 0 || openSettings || launcherSettings.UpdateSource == LauncherUpdateSource.Unknown)
			{
				// Show the settings window immediately
				using UpdateSettingsWindow updateWindow = new UpdateSettingsWindow(null, null, launcherSettings, SyncAndRunWrapper);
				return updateWindow.ShowModal();
			}
			else if (launcherSettings.UpdateSource == LauncherUpdateSource.None)
			{
				return LauncherResult.Continue;
			}
			else
			{
				// Try to do a sync with the current settings first
				CaptureLogger logger = new CaptureLogger();

				ModalTask? task;
				if (launcherSettings.UpdateSource == LauncherUpdateSource.Horde)
				{
					task = ModalTask.Execute(null, "Updating", "Checking for updates, please wait...", c => SyncAndRunWrapper(null, launcherSettings, logger, c));
				}
				else
				{
					IPerforceSettings settings = new PerforceSettings(PerforceSettings.Default) { PreferNativeClient = true }.MergeWith(newServerAndPort: launcherSettings.PerforceServerAndPort, newUserName: launcherSettings.PerforceUserName);
					task = PerforceModalTask.Execute(null, "Updating", "Checking for updates, please wait...", settings, (p, c) => SyncAndRunWrapper(p, launcherSettings, logger, c), logger);
				}

				if (task == null)
				{
					logger.LogInformation("Canceled by user");
				}
				else if (task.Succeeded)
				{
					return LauncherResult.Exit;
				}

				using UpdateSettingsWindow updateWindow = new UpdateSettingsWindow("Unable to update UnrealGameSync from Perforce. Verify that your connection settings are correct.", logger.Render(Environment.NewLine), launcherSettings, SyncAndRunWrapper);
				return updateWindow.ShowModal();
			}
		}

		// Values of the Perforce "action" field that means that the file should no longer be synced
		public static readonly string[] s_deleteActions = { "delete", "move/delete", "purge", "archive" };

		class LoggerProviderAdapter : ILoggerProvider
		{
			readonly ILogger _logger;

			public LoggerProviderAdapter(ILogger logger) => _logger = logger;
			public ILogger CreateLogger(string categoryName) => _logger;
			public void Dispose() { }
		}

		static string GetSyncFolder()
		{
			return Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), "UnrealGameSync", "Latest");
		}

		public static async Task SyncAndRun(IPerforceConnection? perforce, LauncherSettings launcherSettings, string[] args, Mutex instanceMutex, ILogger logger, CancellationToken cancellationToken)
		{
			try
			{
				if (launcherSettings.UpdateSource == LauncherUpdateSource.None)
				{
					return;
				}

				// Create the target folder
				string applicationFolder = GetSyncFolder();
				if (!SafeCreateDirectory(applicationFolder))
				{
					throw new UserErrorException($"Couldn't create directory: {applicationFolder}");
				}

				// Check the application exists
				string applicationExe = Path.Combine(applicationFolder, "UnrealGameSync.exe");

				// Read the current version
				string syncVersionFile = Path.Combine(applicationFolder, "SyncVersion.txt");

				string? syncText = null;
				if (File.Exists(applicationExe) && File.Exists(syncVersionFile))
				{
					TryReadAllText(syncVersionFile, out syncText);
				}

				// New command line for launching the updated executable
				string? updatePath = null;
				if (launcherSettings.UpdateSource == LauncherUpdateSource.Horde)
				{
					if (String.IsNullOrEmpty(launcherSettings.HordeServer))
					{
						if (String.IsNullOrEmpty(DeploymentSettings.Instance.HordeUrl))
						{
							throw new Exception("No Horde server is configured");
						}

						launcherSettings.HordeServer = DeploymentSettings.Instance.HordeUrl;
					}

					Uri hordeServerUrl = new Uri(launcherSettings.HordeServer);

					ServiceCollection services = new ServiceCollection();
					services.AddLogging(builder => builder.AddProvider(new LoggerProviderAdapter(logger)));
					services.AddHorde(options => 
					{ 
						options.ServerUrl = hordeServerUrl; 
						options.AllowAuthPrompt = true; 
					});

					await using ServiceProvider serviceProvider = services.BuildServiceProvider();
					IHordeClient hordeClient = serviceProvider.GetRequiredService<IHordeClient>();

					using HordeHttpClient httpClient = hordeClient.CreateHttpClient();

					ToolId toolId = DeploymentSettings.Instance.HordeToolId;

					GetToolResponse tool = await httpClient.GetToolAsync(toolId, cancellationToken: cancellationToken);
					GetToolDeploymentResponse deployment = tool.Deployments[^1];
					Uri deploymentUri = new Uri(hordeServerUrl, $"api/v1/tools/{toolId}/deployments/{deployment.Id}");

					string requiredSyncText = deploymentUri.ToString();
					if (syncText == null || syncText != requiredSyncText)
					{
						// Delete the output directory
						await SafeDeleteDirectoryContentsWithRetryAsync(applicationFolder, cancellationToken);

						// Download and extract the data
						IStorageNamespace storageNamespace = hordeClient.GetStorageNamespace(toolId);
						IBlobRef<DirectoryNode> handle = storageNamespace.CreateBlobRef<DirectoryNode>(deployment.Locator);

						DirectoryNode rootNode = await handle.ReadBlobAsync(cancellationToken: cancellationToken);
						await rootNode.ExtractAsync(new DirectoryInfo(applicationFolder), new ExtractOptions { Progress = new ExtractStatsLogger(0, rootNode.Length, logger) }, logger, cancellationToken);

						// Update the version
						if (!TryWriteAllText(syncVersionFile, requiredSyncText))
						{
							throw new UserErrorException("Couldn't write sync text to {SyncVersionFile}");
						}
					}

					// Query the deployment settings and write them to the output directory if not already set
					JsonObject? parameters = await httpClient.GetParametersAsync("ugs", cancellationToken);
					if (parameters != null && parameters.Count > 0)
					{
						byte[] deploymentData = JsonSerializer.SerializeToUtf8Bytes(parameters, new JsonSerializerOptions { WriteIndented = true });

						FileReference deploymentJson = new FileReference(Path.Combine(applicationFolder, "Deployment.json"));
						await FileReference.WriteAllBytesAsync(deploymentJson, deploymentData, cancellationToken);
					}
				}
				else if (launcherSettings.UpdateSource == LauncherUpdateSource.Perforce)
				{
					bool preview = launcherSettings.PreviewBuild;
					if (perforce == null)
					{
						throw new UserErrorException("No Perforce connection");
					}

					string? baseDepotPath = launcherSettings.PerforceDepotPath;
					if (String.IsNullOrEmpty(baseDepotPath))
					{
						throw new UserErrorException($"Invalid setting for sync path");
					}

					string baseDepotPathPrefix = baseDepotPath.TrimEnd('/');

					// Find the most recent changelist
					string syncPath = baseDepotPathPrefix + (preview ? "/UnstableRelease.zip" : "/Release.zip");
					List<ChangesRecord> changes = await perforce.GetChangesAsync(ChangesOptions.None, 1, ChangeStatus.Submitted, syncPath, cancellationToken);
					if (changes.Count == 0)
					{
						syncPath = baseDepotPathPrefix + (preview ? "/UnstableRelease/..." : "/Release/...");
						changes = await perforce.GetChangesAsync(ChangesOptions.None, 1, ChangeStatus.Submitted, syncPath, cancellationToken);
#pragma warning disable CA1508 // warning CA1508: 'changes.Count == 0' is always 'true'. Remove or refactor the condition(s) to avoid dead code.
						if (changes.Count == 0)
						{
							throw new UserErrorException($"Unable to find any UGS binaries under {syncPath}");
						}
#pragma warning restore CA1508
					}

					int requiredChangeNumber = changes[0].Number;
					logger.LogInformation("Syncing from {SyncPath}", syncPath);

					// Check if the version has changed
					string requiredSyncText = String.Format("{0}\n{1}@{2}", perforce.Settings.ServerAndPort ?? "", syncPath, requiredChangeNumber);
					if (syncText == null || syncText != requiredSyncText)
					{
						// Delete the output directory
						await SafeDeleteDirectoryContentsWithRetryAsync(applicationFolder, cancellationToken);

						// Find all the files in the sync path at this changelist
						List<FStatRecord> fileRecords = await perforce.FStatAsync(FStatOptions.None, $"{syncPath}@{requiredChangeNumber}", cancellationToken).ToListAsync(cancellationToken);
						if (fileRecords.Count == 0)
						{
							throw new UserErrorException($"Couldn't find any matching files for {syncPath}@{requiredChangeNumber}");
						}

						// Sync all the files in this list to the same directory structure under the application folder
						string depotPathPrefix = syncPath.Substring(0, syncPath.LastIndexOf('/') + 1);
						foreach (FStatRecord fileRecord in fileRecords)
						{
							// Skip deleted files
							if (Array.IndexOf(s_deleteActions, fileRecord.Action) != -1)
							{
								continue;
							}

							if (fileRecord.DepotFile == null)
							{
								throw new UserErrorException("Missing depot path for returned file");
							}

							string localPath = Path.Combine(applicationFolder, fileRecord.DepotFile.Substring(depotPathPrefix.Length).Replace('/', Path.DirectorySeparatorChar));
							if (!SafeCreateDirectory(Path.GetDirectoryName(localPath)!))
							{
								throw new UserErrorException($"Couldn't create folder {Path.GetDirectoryName(localPath)}");
							}

							await perforce.PrintAsync(localPath, fileRecord.DepotFile, cancellationToken);
						}

						// If it was a zip file, extract it
						if (syncPath.EndsWith(".zip", StringComparison.OrdinalIgnoreCase))
						{
							string localPath = Path.Combine(applicationFolder, syncPath.Substring(depotPathPrefix.Length).Replace('/', Path.DirectorySeparatorChar));
							await ZipFile.ExtractToDirectoryAsync(localPath, applicationFolder, cancellationToken);
						}

						// Check the application exists
						if (!File.Exists(applicationExe))
						{
							throw new UserErrorException($"Application was not synced from Perforce. Check that UnrealGameSync exists at {syncPath}/UnrealGameSync.exe, and you have access to it.");
						}

						// Update the version
						if (!TryWriteAllText(syncVersionFile, requiredSyncText))
						{
							throw new UserErrorException("Couldn't write sync text to {SyncVersionFile}");
						}
					}

					// Argument for updating in the future
					updatePath = $"{syncPath}@>{requiredChangeNumber}";
				}
				else
				{
					throw new NotSupportedException("Invalid sync type");
				}
				logger.LogInformation("");

				// Create the new argument list
				List<string> newArguments = new List<string>(args);
				newArguments.Add($"-updatespawn={Program.GetCurrentExecutable()}");
				if (updatePath != null)
				{
					newArguments.Add($"-updatepath={updatePath}");
				}
				if (launcherSettings.PreviewBuild)
				{
					newArguments.Add("-unstable");
				}

				// Release the mutex now so that the new application can start up
				instanceMutex.Close();

				// Spawn the application
				string newCommandLine = CommandLineArguments.Join(newArguments);
				logger.LogInformation("Spawning {App} with command line: {CmdLine}", applicationExe, newCommandLine);
				using (Process childProcess = new Process())
				{
					childProcess.StartInfo.FileName = applicationExe;
					childProcess.StartInfo.Arguments = newCommandLine;
					childProcess.StartInfo.UseShellExecute = false;
					childProcess.StartInfo.CreateNoWindow = false;
					if (!childProcess.Start())
					{
						throw new UserErrorException("Failed to start process");
					}
				}
			}
			catch (UserErrorException ex)
			{
				logger.LogError("{Message}", ex.Message);
				throw;
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Error while syncing application.");
				foreach (string line in ex.ToString().Split('\n'))
				{
					logger.LogError("{Line}", line);
				}
				throw;
			}
		}

		static bool TryReadAllText(string fileName, [NotNullWhen(true)] out string? text)
		{
			try
			{
				text = File.ReadAllText(fileName);
				return true;
			}
			catch (Exception)
			{
				text = null;
				return false;
			}
		}

		static bool TryWriteAllText(string fileName, string text)
		{
			try
			{
				File.WriteAllText(fileName, text);
				return true;
			}
			catch (Exception)
			{
				return false;
			}
		}

		static bool SafeCreateDirectory(string directoryName)
		{
			try
			{
				Directory.CreateDirectory(directoryName);
				return true;
			}
			catch (Exception)
			{
				return false;
			}
		}

		static bool SafeDeleteDirectory(string directoryName)
		{
			try
			{
				Directory.Delete(directoryName, true);
				return true;
			}
			catch (Exception)
			{
				return false;
			}
		}

		static bool SafeDeleteDirectoryContents(string directoryName)
		{
			try
			{
				DirectoryInfo directory = new DirectoryInfo(directoryName);
				foreach (FileInfo childFile in directory.EnumerateFiles("*", SearchOption.AllDirectories))
				{
					childFile.Attributes &= ~FileAttributes.ReadOnly;
					childFile.Delete();
				}
				foreach (DirectoryInfo childDirectory in directory.EnumerateDirectories())
				{
					SafeDeleteDirectory(childDirectory.FullName);
				}
				return true;
			}
			catch (Exception)
			{
				return false;
			}
		}

		static async Task SafeDeleteDirectoryContentsWithRetryAsync(string directoryName, CancellationToken cancellationToken)
		{
			// Try to delete the directory contents. Retry for a while, in case we've been spawned by an application in this folder to do an update.
			for (int numRetries = 0; !SafeDeleteDirectoryContents(directoryName); numRetries++)
			{
				if (numRetries > 20)
				{
					throw new UserErrorException($"Couldn't delete contents of {directoryName} (retried {numRetries} times).");
				}
				await Task.Delay(500, cancellationToken);
			}
		}
	}
}
