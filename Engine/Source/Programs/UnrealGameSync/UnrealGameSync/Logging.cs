// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Serilog;
using Serilog.Templates;

namespace UnrealGameSync
{
	static class Logging
	{
		public static ILoggerProvider CreateLoggerProvider(FileReference file)
		{
			Serilog.ILogger logger = new LoggerConfiguration()
				.Enrich.FromLogContext()
				.MinimumLevel.Verbose()
				.WriteTo.File(new ExpressionTemplate("[{@t:HH:mm:ss.fff} {@l:w3}] {Concat('[', Substring(SourceContext, LastIndexOf(SourceContext, '.') + 1, 14), ']'),16} {Indent}{@m}{@x}\n"), file.FullName, rollingInterval: RollingInterval.Day, rollOnFileSizeLimit: true, fileSizeLimitBytes: 20 * 1024 * 1024, retainedFileCountLimit: 10)
				.CreateLogger();

			return new Serilog.Extensions.Logging.SerilogLoggerProvider(logger, true);
		}
	}
}
