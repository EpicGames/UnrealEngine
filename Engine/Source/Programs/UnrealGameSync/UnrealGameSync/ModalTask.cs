// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;

namespace UnrealGameSync
{
	[Flags]
	enum ModalTaskFlags
	{
		None = 0,
		Quiet = 1,
	}

	class ModalTask
	{
		public Task Task { get; }

		public bool Failed => Task.IsFaulted;
		public bool Succeeded => Task.IsCompletedSuccessfully;

		public Exception? Exception => Task.Exception?.InnerException;

		public ModalTask(Task task)
		{
			Task = task;
		}

		public string Error
		{
			get
			{
				if (Succeeded)
				{
					return "Succeeded.";
				}

				Exception? ex = Exception;
				if (ex == null)
				{
					return "Failed.";
				}
				else if (ex is UserErrorException userEx)
				{
					return userEx.Message;
				}
				else
				{
					return $"Unhandled exception ({ex.Message})";
				}
			}
		}

		public static ModalTask? Execute(IWin32Window? owner, string title, string message, Func<CancellationToken, Task> taskFunc, ModalTaskFlags flags = ModalTaskFlags.None)
		{
			async Task<int> TypedTaskFunc(CancellationToken cancellationToken)
			{
				await taskFunc(cancellationToken);
				return 0;
			}
			return Execute(owner, title, message, TypedTaskFunc, flags);
		}

		public static ModalTask<T>? Execute<T>(IWin32Window? owner, string title, string message, Func<CancellationToken, Task<T>> taskFunc, ModalTaskFlags flags = ModalTaskFlags.None)
		{
			using (CancellationTokenSource cancellationSource = new CancellationTokenSource())
			{
				Task<T> backgroundTask = Task.Run(() => taskFunc(cancellationSource.Token));

				using ModalTaskWindow window = new ModalTaskWindow(title, message, (owner == null) ? FormStartPosition.CenterScreen : FormStartPosition.CenterParent, backgroundTask, cancellationSource);
				if (owner == null)
				{
					window.ShowInTaskbar = true;
				}
				window.ShowDialog(owner);

				if (backgroundTask.IsCanceled || (backgroundTask.Exception != null && backgroundTask.Exception.InnerException is OperationCanceledException))
				{
					return null;
				}

				ModalTask<T> result = new ModalTask<T>(backgroundTask);
				if (result.Failed && (flags & ModalTaskFlags.Quiet) == 0)
				{
					MessageBox.Show(owner, result.Error, title);
				}
				return result;
			}
		}
	}

	class ModalTask<T> : ModalTask
	{
		public new Task<T> Task => (Task<T>)base.Task;

		public T Result => Task.Result;

		public ModalTask(Task<T> task) : base(task)
		{
		}
	}
}
