// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	public class PerforceLoginException : Exception
	{
		public PerforceResponse<LoginRecord> Response { get; }

		public PerforceLoginException(PerforceResponse<LoginRecord> response) : base($"Login failed: {response}")
		{
			Response = response;
		}
	}

	static class PerforceModalTask
	{
		public static ModalTask? Execute(IWin32Window? owner, string title, string message, IPerforceSettings perforceSettings, Func<IPerforceConnection, CancellationToken, Task> executeAsync, ILogger logger, ModalTaskFlags flags = ModalTaskFlags.None)
		{
			async Task<int> ExecuteTypedAsync(IPerforceConnection perforce, CancellationToken cancellationToken)
			{
				await executeAsync(perforce, cancellationToken);
				return 0;
			}

			return Execute(owner, title, message, perforceSettings, ExecuteTypedAsync, logger, flags);
		}

		public static ModalTask<T>? Execute<T>(IWin32Window? owner, string title, string message, IPerforceSettings perforceSettings, Func<IPerforceConnection, CancellationToken, Task<T>> executeAsync, ILogger logger, ModalTaskFlags flags = ModalTaskFlags.None)
		{
			IPerforceConnection? connection = null;
			try
			{
				async Task<IPerforceConnection> ConnectAsync()
				{
					connection ??= await PerforceConnection.CreateAsync(perforceSettings, logger);
					return connection;
				}
				return ExecuteInternal(owner, title, message, perforceSettings, ConnectAsync, executeAsync, flags);
			}
			finally
			{
#pragma warning disable CA1508 // warning CA1508: 'connection' is always 'null'. Remove or refactor the condition(s) to avoid dead code.
				connection?.Dispose();
#pragma warning restore CA1508
			}
		}

		public static ModalTask? Execute(IWin32Window? owner, string title, string message, IPerforceConnection perforceConnection, Func<IPerforceConnection, CancellationToken, Task> executeAsync, ILogger logger, ModalTaskFlags flags = ModalTaskFlags.None)
		{
			async Task<int> ExecuteTypedAsync(IPerforceConnection perforce, CancellationToken cancellationToken)
			{
				await executeAsync(perforce, cancellationToken);
				return 0;
			}
			return Execute(owner, title, message, perforceConnection, ExecuteTypedAsync, logger, flags);
		}

		public static ModalTask<T>? Execute<T>(IWin32Window? owner, string title, string message, IPerforceConnection perforceConnection, Func<IPerforceConnection, CancellationToken, Task<T>> executeAsync, ModalTaskFlags flags = ModalTaskFlags.None)
		{
			return ExecuteInternal(owner, title, message, perforceConnection.Settings, () => Task.FromResult(perforceConnection), executeAsync, flags);
		}

		private static ModalTask<T>? ExecuteInternal<T>(IWin32Window? owner, string title, string message, IPerforceSettings perforceSettings, Func<Task<IPerforceConnection>> connectAsync, Func<IPerforceConnection, CancellationToken, Task<T>> executeAsync, ModalTaskFlags flags = ModalTaskFlags.None)
		{
			string? password = perforceSettings.Password;
			for (; ; )
			{
				Task<T> RunAsync(CancellationToken cancellationToken) => LoginAndExecuteAsync(password, connectAsync, executeAsync, cancellationToken);

				ModalTask<T>? result = ModalTask.Execute(owner, title, message, RunAsync, ModalTaskFlags.Quiet);
				if (result != null && result.Failed && (flags & ModalTaskFlags.Quiet) == 0)
				{
					if (result.Exception is PerforceLoginException)
					{
						string passwordPrompt;
						if (String.IsNullOrEmpty(password))
						{
							passwordPrompt = $"Enter the Perforce password for user '{perforceSettings.UserName}' on server '{perforceSettings.ServerAndPort}'.";
						}
						else
						{
							passwordPrompt = $"Perforce authentication failed. Enter the password for user '{perforceSettings.UserName}' on server '{perforceSettings.ServerAndPort}'.";
						}

						using PasswordWindow passwordWindow = new PasswordWindow(passwordPrompt, password ?? String.Empty);
						if (owner == null)
						{
							passwordWindow.ShowInTaskbar = true;
							passwordWindow.StartPosition = FormStartPosition.CenterScreen;
						}
						if (passwordWindow.ShowDialog(owner) != DialogResult.OK)
						{
							return null;
						}

						password = passwordWindow.Password;
						continue;
					}
					MessageBox.Show(owner, result.Error, title, MessageBoxButtons.OK);
				}

				return result;
			}
		}

		private static async Task<T> LoginAndExecuteAsync<T>(string? password, Func<Task<IPerforceConnection>> connectAsync, Func<IPerforceConnection, CancellationToken, Task<T>> executeAsync, CancellationToken cancellationToken)
		{
			IPerforceConnection perforce = await connectAsync();

			// If we've got a password, execute the login command
			PerforceResponse<LoginRecord> response;
			if (String.IsNullOrEmpty(password))
			{
				response = await perforce.TryGetLoginStateAsync(cancellationToken);
			}
			else
			{
				response = await perforce.TryLoginAsync(password, cancellationToken);
			}

			if (!response.Succeeded)
			{
				throw new PerforceLoginException(response);
			}

			// Execute the inner task
			return await executeAsync(perforce, cancellationToken);
		}
	}
}
