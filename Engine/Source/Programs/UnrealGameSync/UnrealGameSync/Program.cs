// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.OIDC;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Sentry;
using Sentry.Infrastructure;

namespace UnrealGameSync
{
	/// <summary>
	/// Delegate used to create a telemetry sink
	/// </summary>
	/// <param name="userName">The default Perforce user name</param>
	/// <param name="sessionId">Unique identifier for this session</param>
	/// <param name="logger">Log writer</param>
	/// <returns>New telemetry sink instance</returns>
	public delegate ITelemetrySink CreateTelemetrySinkDelegate(string userName, string sessionId, ILogger logger);

	static partial class Program
	{
		/// <summary>
		/// Delegate used to create a new telemetry sink
		/// </summary>
		static CreateTelemetrySinkDelegate CreateTelemetrySink { get; } = (userName, sessionId, log) => new NullTelemetrySink();

		// Session lock tracking
		private static SessionLockTracker SessionLockTracker { get; } = new SessionLockTracker();

		public static string GetVersionString()
		{
			AssemblyInformationalVersionAttribute? version = Assembly.GetExecutingAssembly().GetCustomAttribute<AssemblyInformationalVersionAttribute>();
			return version?.InformationalVersion ?? "Unknown";
		}

		public static string? SyncVersion = null;

		public static void CaptureException(Exception exception)
		{
			if (DeploymentSettings.Instance.SentryDsn != null)
			{
				SentrySdk.CaptureException(exception);
			}
		}

		[STAThread]
		static void Main(string[] args)
		{
			if (DeploymentSettings.Instance.SentryDsn != null)
			{
				SentryOptions sentryOptions = new SentryOptions();
				sentryOptions.Dsn = DeploymentSettings.Instance.SentryDsn;
				sentryOptions.StackTraceMode = StackTraceMode.Enhanced;
				sentryOptions.AttachStacktrace = true;
				sentryOptions.TracesSampleRate = 1.0;
				sentryOptions.SendDefaultPii = true;
				sentryOptions.Debug = true;
				sentryOptions.AutoSessionTracking = true;
				sentryOptions.DetectStartupTime = StartupTimeDetectionMode.Best;
				sentryOptions.ReportAssembliesMode = ReportAssembliesMode.InformationalVersion;
				sentryOptions.DiagnosticLogger = new TraceDiagnosticLogger(SentryLevel.Debug);
				SentrySdk.Init(sentryOptions);

				Application.ThreadException += Application_ThreadException_Sentry;
				AppDomain.CurrentDomain.UnhandledException += CurrentDomain_UnhandledException_Sentry;
				TaskScheduler.UnobservedTaskException += Application_UnobservedException_Sentry;
			}

			try
			{
				RealMain(args);
			}
			catch (Exception ex)
			{
				CaptureException(ex);
			}
		}

		static void RealMain(string[] args)
		{
			bool firstInstance;
			using (Mutex instanceMutex = new Mutex(true, "UnrealGameSyncRunning", out firstInstance))
			{
				Application.EnableVisualStyles();
				Application.SetCompatibleTextRenderingDefault(false);
				Application.SetHighDpiMode(HighDpiMode.PerMonitorV2);

				// Don't auto install (or - more importantly- auto *un-install*) the winforms sync context. We want to be able to access it from the
				// constructor of our ApplicationContext, which will be after the temporary install/uninstall prompted by spawning the settings dialog.
				WindowsFormsSynchronizationContext.AutoInstall = false;

				using WindowsFormsSynchronizationContext synchronizationContext = new WindowsFormsSynchronizationContext();
				SynchronizationContext.SetSynchronizationContext(synchronizationContext);

				using (EventWaitHandle activateEvent = new EventWaitHandle(false, EventResetMode.AutoReset, "ActivateUnrealGameSync"))
				{
					bool runUpdateCheck = ShouldRunAutoUpdate(args);

					// Check for a newer version of the application
					if (runUpdateCheck && firstInstance && Launcher.SyncAndRunLatest(instanceMutex, args) != LauncherResult.Continue)
					{
						return;
					}

					// Handle any url passed in, possibly exiting
					if (UriHandler.ProcessCommandLine(args, firstInstance, activateEvent))
					{
						return;
					}

					// Handle any .uartifact downloads
					if (ArtifactDownload.ProcessCommandLine(args))
					{
						return;
					}

					// Launch the application proper
					if (firstInstance)
					{
						InnerMain(instanceMutex, activateEvent, args, runUpdateCheck);
					}
					else
					{
						activateEvent.Set();
					}
				}
			}
		}

		static void InnerMain(Mutex instanceMutex, EventWaitHandle activateEvent, string[] args, bool runUpdateCheck)
		{
			LauncherSettings launcherSettings = new LauncherSettings();
			launcherSettings.Read();

			List<string> remainingArgs = new List<string>(args);

			string? updateSpawn;
			ParseArgument(remainingArgs, "-updatespawn=", out updateSpawn);

			string? updatePath;
			ParseArgument(remainingArgs, "-updatepath=", out updatePath);

			bool restoreState;
			ParseOption(remainingArgs, "-restorestate", out restoreState);

			bool unstable;
			ParseOption(remainingArgs, "-unstable", out unstable);
			bool preview;
			ParseOption(remainingArgs, "-preview", out preview);
			preview |= unstable;

			string? projectFileName;
			ParseArgument(remainingArgs, "-project=", out projectFileName);

			string? uri;
			ParseArgument(remainingArgs, "-uri=", out uri);

			FileReference updateConfigFile = FileReference.Combine(new FileReference(Assembly.GetExecutingAssembly().Location).Directory, "AutoUpdate.ini");
			MergeUpdateSettings(updateConfigFile, ref updatePath, ref updateSpawn);

			// Set the current working directory to the update directory to prevent child-process file handles from disrupting auto-updates
			if (updateSpawn != null)
			{
				if (File.Exists(updateSpawn))
				{
					Directory.SetCurrentDirectory(Path.GetDirectoryName(updateSpawn)!);
				}
				else
				{
					updateSpawn = null;
				}
			}

			string syncVersionFile = Path.Combine(Path.GetDirectoryName(Assembly.GetExecutingAssembly().Location!)!, "SyncVersion.txt");
			if (File.Exists(syncVersionFile))
			{
				try
				{
					SyncVersion = File.ReadAllText(syncVersionFile).Trim();
				}
				catch (Exception)
				{
					SyncVersion = null;
				}
			}

			ArtifactDownload.RegisterFileAssociations(updateSpawn ?? GetCurrentExecutable());

			DirectoryReference dataFolder = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData)!, "UnrealGameSync");
			DirectoryReference.CreateDirectory(dataFolder);

			// Create a new logger
			using (ILoggerProvider loggerProvider = Logging.CreateLoggerProvider(FileReference.Combine(dataFolder, "UnrealGameSync.log")))
			{
				ServiceCollection services = new ServiceCollection();
				services.AddLogging(builder => builder.AddProvider(loggerProvider));
				services.AddSingleton<IAsyncDisposer, AsyncDisposer>();
				services.AddSingleton(sp => TokenStoreFactory.CreateTokenStore());
				services.AddSingleton<OidcTokenManager>();

				services.AddCloudStorage();
				if (launcherSettings.HordeServer != null)
				{
					services.AddHorde(options =>
					{
						options.ServerUrl = new Uri(launcherSettings.HordeServer);
						options.AllowAuthPrompt = false;
					});
				}

				ServiceProvider serviceProvider = services.BuildServiceProvider();
				try
				{
					ILoggerFactory loggerFactory = serviceProvider.GetRequiredService<ILoggerFactory>();

					ILogger logger = loggerFactory.CreateLogger("Startup");
					logger.LogInformation("Application version: {Version}", Assembly.GetExecutingAssembly().GetName().Version);
					logger.LogInformation("Started at {Time}", DateTime.Now.ToString());

					Utility.TraceException += ex => TraceException(ex, logger);

					string sessionId = Guid.NewGuid().ToString();
					logger.LogInformation("SessionId: {SessionId}", sessionId);

					if (launcherSettings.PerforceServerAndPort == null || launcherSettings.PerforceUserName == null)
					{
						logger.LogInformation("Missing server settings; finding defaults.");
						launcherSettings.PerforceServerAndPort ??= DeploymentSettings.Instance.DefaultPerforceServer ?? PerforceSettings.Default.ServerAndPort;
						launcherSettings.PerforceUserName ??= PerforceSettings.Default.UserName;
						launcherSettings.Save();
					}

					ILogger telemetryLogger = loggerProvider.CreateLogger("Telemetry");
					telemetryLogger.LogInformation("Creating telemetry sink for session {SessionId}", sessionId);

					using (ITelemetrySink telemetrySink = CreateTelemetrySink(launcherSettings.PerforceUserName, sessionId, telemetryLogger))
					{
						ITelemetrySink? prevTelemetrySink = UgsTelemetry.ActiveSink;
						try
						{
							UgsTelemetry.ActiveSink = telemetrySink;

							UgsTelemetry.SendEvent("Startup", new { User = Environment.UserName, Machine = System.Net.Dns.GetHostName() });

							AppDomain.CurrentDomain.UnhandledException += CurrentDomain_UnhandledException;

							IPerforceSettings defaultSettings = new PerforceSettings(launcherSettings.PerforceServerAndPort, launcherSettings.PerforceUserName) { PreferNativeClient = true };

							ProtocolHandlerUtils.InstallQuiet(logger);

							UpdateMonitor updateMonitor = CreateUpdateMonitor(launcherSettings, defaultSettings, updatePath, runUpdateCheck, serviceProvider);
							try
							{
								using ProgramApplicationContext context = new ProgramApplicationContext(defaultSettings, updateMonitor, DeploymentSettings.Instance.ApiUrl, dataFolder, activateEvent, restoreState, updateSpawn, projectFileName, preview, serviceProvider, uri);
								Application.Run(context);

								if (updateMonitor.IsUpdateAvailable)
								{
									instanceMutex.Close();
									Utility.SpawnProcess(updateSpawn ?? GetCurrentExecutable(), "-restorestate" + (updateMonitor.OpenSettings ? " -settings" : ""));
								}
							}
							finally
							{
								AsyncDispose(updateMonitor);
							}
						}
						catch (Exception ex)
						{
							UgsTelemetry.SendEvent("Crash", new { User = Environment.UserName, Machine = System.Net.Dns.GetHostName(), Exception = ex.ToString() });
							throw;
						}
						finally
						{
							UgsTelemetry.ActiveSink = prevTelemetrySink;
						}
					}
				}
				finally
				{
					AsyncDispose(serviceProvider);
				}
			}
		}

		static void AsyncDispose(IAsyncDisposable disposable)
		{
			// Force the dispose to run on a task without a synchronization context, so we don't have to worry about waiting for it on the Winforms thread.
			Task task = Task.Run(async () => await disposable.DisposeAsync());
			task.GetAwaiter().GetResult();
		}

		public static string GetCurrentExecutable()
		{
			string originalExecutable = Assembly.GetEntryAssembly()!.Location;
			if (Path.GetExtension(originalExecutable).Equals(".dll", StringComparison.OrdinalIgnoreCase))
			{
				string newExecutable = Path.ChangeExtension(originalExecutable, ".exe");
				if (File.Exists(newExecutable))
				{
					return newExecutable;
				}
			}
			return originalExecutable;
		}

		static bool ShouldRunAutoUpdate(string[] args)
		{
#if WITH_AUTOUPDATE
			return !args.Contains("-NoUpdateCheck", StringComparer.OrdinalIgnoreCase);
#else
			return args.Contains("-UpdateCheck", StringComparer.OrdinalIgnoreCase) || args.Contains("-Settings", StringComparer.OrdinalIgnoreCase);
#endif
		}

		private static UpdateMonitor CreateUpdateMonitor(LauncherSettings launcherSettings, IPerforceSettings defaultSettings, string? updatePath, bool runUpdateCheck, IServiceProvider serviceProvider)
		{
			if (!runUpdateCheck)
			{
				return new NullUpdateMonitor();
			}
			else if (launcherSettings.UpdateSource == LauncherUpdateSource.Horde)
			{
				return new HordeUpdateMonitor(SyncVersion ?? String.Empty, serviceProvider);
			}
			else
			{
				return new PerforceUpdateMonitor(defaultSettings, updatePath, serviceProvider);
			}
		}

		private static void TraceException(Exception ex, ILogger logger)
		{
			if (DeploymentSettings.Instance.SentryDsn != null)
			{
				SentrySdk.CaptureException(ex);
			}
			logger.LogError(ex, "Trace exception: {Ex}", ex.ToString());
		}

		private static void CurrentDomain_UnhandledException(object sender, UnhandledExceptionEventArgs args)
		{
			Exception? ex = args.ExceptionObject as Exception;
			if (ex != null)
			{
				UgsTelemetry.SendEvent("Crash", new { User = Environment.UserName, Machine = System.Net.Dns.GetHostName(), Exception = ex.ToString() });
			}
		}

		private static void CurrentDomain_UnhandledException_Sentry(object sender, UnhandledExceptionEventArgs args)
		{
			Exception? ex = args.ExceptionObject as Exception;
			if (ex != null)
			{
				SentrySdk.CaptureException(ex);
			}
		}

		private static void Application_ThreadException_Sentry(object sender, ThreadExceptionEventArgs e)
		{
			SentrySdk.CaptureException(e.Exception);

			using ThreadExceptionDialog dialog = new ThreadExceptionDialog(e.Exception);

			if (SessionLockTracker.IsSessionLocked)
			{
				dialog.Show();
			}
			else
			{
				dialog.ShowDialog();
			}
		}

		private static void Application_UnobservedException_Sentry(object? sender, UnobservedTaskExceptionEventArgs args)
		{
			Exception? innerException = args.Exception?.InnerException;
			if (innerException != null)
			{
				SentrySdk.CaptureException(innerException, s => s.SetTag("Unobserved", "1"));
			}
		}

		static void MergeUpdateSettings(FileReference updateConfigFile, ref string? updatePath, ref string? updateSpawn)
		{
			try
			{
				ConfigFile updateConfig = new ConfigFile();
				if (FileReference.Exists(updateConfigFile))
				{
					updateConfig.Load(updateConfigFile);
				}

				if (updatePath == null)
				{
					updatePath = updateConfig.GetValue("Update.Path", null);
				}
				else
				{
					updateConfig.SetValue("Update.Path", updatePath);
				}

				if (updateSpawn == null)
				{
					updateSpawn = updateConfig.GetValue("Update.Spawn", null);
				}
				else
				{
					updateConfig.SetValue("Update.Spawn", updateSpawn);
				}

				updateConfig.Save(updateConfigFile);
			}
			catch (Exception)
			{
			}
		}

		static bool ParseOption(List<string> remainingArgs, string option, out bool value)
		{
			for (int idx = 0; idx < remainingArgs.Count; idx++)
			{
				if (remainingArgs[idx].Equals(option, StringComparison.OrdinalIgnoreCase))
				{
					value = true;
					remainingArgs.RemoveAt(idx);
					return true;
				}
			}

			value = false;
			return false;
		}

		static bool ParseArgument(List<string> remainingArgs, string prefix, [NotNullWhen(true)] out string? value)
		{
			for (int idx = 0; idx < remainingArgs.Count; idx++)
			{
				if (remainingArgs[idx].StartsWith(prefix, StringComparison.OrdinalIgnoreCase))
				{
					value = remainingArgs[idx].Substring(prefix.Length);
					remainingArgs.RemoveAt(idx);
					return true;
				}
			}

			value = null;
			return false;
		}

		public static IEnumerable<string> GetPerforcePaths()
		{
			string? pathList = Environment.GetEnvironmentVariable("PATH");
			if (!String.IsNullOrEmpty(pathList))
			{
				foreach (string pathEntry in pathList.Split(Path.PathSeparator))
				{
					string? perforcePath = null;
					try
					{
						string testPerforcePath = Path.Combine(pathEntry, "p4.exe");
						if (File.Exists(testPerforcePath))
						{
							perforcePath = testPerforcePath;
						}
					}
					catch
					{
					}

					if (perforcePath != null)
					{
						yield return perforcePath;
					}
				}
			}
		}

		public static void SpawnP4Vc(string arguments)
		{
			string executable = "p4vc.exe";

			foreach (string perforcePath in GetPerforcePaths())
			{
				string? perforceDir = Path.GetDirectoryName(perforcePath);
				if (perforceDir != null && File.Exists(Path.Combine(perforceDir, "p4vc.bat")) && !File.Exists(Path.Combine(perforceDir, "p4vc.exe")))
				{
					executable = Path.Combine(perforceDir, "p4v.exe");
					arguments = "-p4vc " + arguments;
					break;
				}
			}

			if (!Utility.SpawnHiddenProcess(executable, arguments))
			{
				MessageBox.Show("Unable to spawn p4vc. Check you have P4V installed.");
			}
		}
	}
}
