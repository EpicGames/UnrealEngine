// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Drawing;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Threading;
using System.Threading.Tasks;
using System.Windows.Forms;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace UnrealGameSync
{
	class ProgramApplicationContext : ApplicationContext
	{
		static ProgramApplicationContext? s_instance { get; set; }
		readonly SynchronizationContext _mainThreadSynchronizationContext;

		readonly IPerforceSettings _defaultPerforceSettings;
		readonly UpdateMonitor _updateMonitor;
		readonly string? _apiUrl;
		readonly DirectoryReference _dataFolder;
		readonly DirectoryReference _cacheFolder;
		readonly bool _restoreState;
		readonly string? _updateSpawn;
		readonly bool _preview;
		bool _isClosing;
		readonly string? _uri;

		readonly IServiceProvider _serviceProvider;
		readonly ILogger _logger;
		readonly UserSettings _settings;
		readonly ApplicationThemeManager _themeManager;
		ActivationListener _activationListener;

		Container _components = new Container();
		NotifyIcon _notifyIcon;
#pragma warning disable CA2213 // warning CA2213: 'ProgramApplicationContext' contains field '_notifyMenuOpenUnrealGameSync' that is of IDisposable type 'ToolStripMenuItem', but it is never disposed. Change the Dispose method on 'ProgramApplicationContext' to call Close or Dispose on this field.
		// Disposed via _components
		readonly ContextMenuStrip _notifyMenu;
		readonly ToolStripMenuItem _notifyMenuOpenUnrealGameSync;
		readonly ToolStripSeparator _notifyMenuOpenUnrealGameSyncSeparator;
		readonly ToolStripMenuItem _notifyMenuSyncNow;
		readonly ToolStripMenuItem _notifyMenuLaunchEditor;
		readonly ToolStripSeparator _notifyMenuExitSeparator;
		readonly ToolStripMenuItem _notifyMenuExit;
#pragma warning restore CA2213

		CancellationTokenSource _startupCancellationSource = new CancellationTokenSource();
		readonly Task _startupTask;
		ModalTaskWindow? _startupWindow;
#pragma warning disable CA2213 // warning CA2213: 'ProgramApplicationContext' contains field '_mainWindowInstance' that is of IDisposable type 'MainWindow?', but it is never disposed. Change the Dispose method on 'ProgramApplicationContext' to call Close or Dispose on this field.
		MainWindow? _mainWindowInstance;
#pragma warning restore CA2213

		WindowsFormsSynchronizationContext? _synchronizationContext;

		public ProgramApplicationContext(IPerforceSettings defaultPerforceSettings, UpdateMonitor updateMonitor, string? apiUrl, DirectoryReference dataFolder, EventWaitHandle activateEvent, bool restoreState, string? updateSpawn, string? projectFileName, bool preview, IServiceProvider serviceProvider, string? uri)
		{
			s_instance = this;
			_defaultPerforceSettings = defaultPerforceSettings;
			_updateMonitor = updateMonitor;
			_apiUrl = apiUrl;
			_dataFolder = dataFolder;
			_cacheFolder = DirectoryReference.Combine(dataFolder, "Cache");
			_restoreState = restoreState;
			_updateSpawn = updateSpawn;
			_preview = preview;
			_serviceProvider = serviceProvider;
			_logger = serviceProvider.GetRequiredService<ILogger<ProgramApplicationContext>>();
			_uri = uri;

			// Create the directories
			DirectoryReference.CreateDirectory(dataFolder);
			DirectoryReference.CreateDirectory(_cacheFolder);

			// Make sure a synchronization context is set. We spawn a bunch of threads (eg. UpdateMonitor) at startup, and need to make sure we can post messages 
			// back to the main thread at any time.
			if (SynchronizationContext.Current == null)
			{
				_synchronizationContext = new WindowsFormsSynchronizationContext();
				SynchronizationContext.SetSynchronizationContext(_synchronizationContext);
			}

			// Capture the main thread's synchronization context for callbacks
			_mainThreadSynchronizationContext = SynchronizationContext.Current!;
			
			// Create the theme manager before the settings so that the user's platform theme preference is detected first
			_themeManager = new ApplicationThemeManager();

			// Read the user's settings
			_settings = UserSettings.Create(dataFolder, serviceProvider.GetRequiredService<ILogger<UserSettings>>());
			if (!String.IsNullOrEmpty(projectFileName))
			{
				string fullProjectFileName = Path.GetFullPath(projectFileName);
				if (!_settings.OpenProjects.Any(x => x.LocalPath != null && String.Equals(x.LocalPath, fullProjectFileName, StringComparison.OrdinalIgnoreCase)))
				{
					_settings.OpenProjects.Add(new UserSelectedProjectSettings(null, null, UserSelectedProjectType.Local, null, fullProjectFileName));
				}
			}

			// Update the settings to the latest version
			if (_settings.Version < UserSettingsVersion.Latest)
			{
				// Clear out the server settings for anything using the default server
				if (_settings.Version < UserSettingsVersion.DefaultServerSettings)
				{
					_logger.LogInformation("Clearing project settings for default server");
					for (int idx = 0; idx < _settings.OpenProjects.Count; idx++)
					{
						_settings.OpenProjects[idx] = UpgradeSelectedProjectSettings(_settings.OpenProjects[idx]);
					}
					for (int idx = 0; idx < _settings.RecentProjects.Count; idx++)
					{
						_settings.RecentProjects[idx] = UpgradeSelectedProjectSettings(_settings.RecentProjects[idx]);
					}
				}

				// Save the new settings
				_settings.Version = UserSettingsVersion.Latest;
				_settings.Save(_logger);
			}

			((PerforceSettings)_defaultPerforceSettings).PreferNativeClient = _settings.SyncOptions.UseNativeClient ?? PerforceSyncOptions.DefaultUseNativeLibrary;

			// Register the update listener
			updateMonitor.OnUpdateAvailable += OnUpdateAvailableCallback;

			// Create the activation listener
			_activationListener = new ActivationListener(activateEvent);
			_activationListener.Start();
			_activationListener.OnActivate += OnActivationListenerAsyncCallback;

			// Create the notification menu items
			_notifyMenuOpenUnrealGameSync = new ToolStripMenuItem();
			_notifyMenuOpenUnrealGameSync.Name = nameof(_notifyMenuOpenUnrealGameSync);
			_notifyMenuOpenUnrealGameSync.Size = new Size(196, 22);
			_notifyMenuOpenUnrealGameSync.Text = "Open UnrealGameSync";
			_notifyMenuOpenUnrealGameSync.Click += new EventHandler(NotifyMenu_OpenUnrealGameSync_Click);
			_notifyMenuOpenUnrealGameSync.Font = new Font(_notifyMenuOpenUnrealGameSync.Font, FontStyle.Bold);

			_notifyMenuOpenUnrealGameSyncSeparator = new ToolStripSeparator();
			_notifyMenuOpenUnrealGameSyncSeparator.Name = nameof(_notifyMenuOpenUnrealGameSyncSeparator);
			_notifyMenuOpenUnrealGameSyncSeparator.Size = new Size(193, 6);

			_notifyMenuSyncNow = new ToolStripMenuItem();
			_notifyMenuSyncNow.Name = nameof(_notifyMenuSyncNow);
			_notifyMenuSyncNow.Size = new Size(196, 22);
			_notifyMenuSyncNow.Text = "Sync Now";
			_notifyMenuSyncNow.Click += new EventHandler(NotifyMenu_SyncNow_Click);

			_notifyMenuLaunchEditor = new ToolStripMenuItem();
			_notifyMenuLaunchEditor.Name = nameof(_notifyMenuLaunchEditor);
			_notifyMenuLaunchEditor.Size = new Size(196, 22);
			_notifyMenuLaunchEditor.Text = "Launch Editor";
			_notifyMenuLaunchEditor.Click += new EventHandler(NotifyMenu_LaunchEditor_Click);

			_notifyMenuExitSeparator = new ToolStripSeparator();
			_notifyMenuExitSeparator.Name = nameof(_notifyMenuExitSeparator);
			_notifyMenuExitSeparator.Size = new Size(193, 6);

			_notifyMenuExit = new ToolStripMenuItem();
			_notifyMenuExit.Name = nameof(_notifyMenuExit);
			_notifyMenuExit.Size = new Size(196, 22);
			_notifyMenuExit.Text = "Exit";
			_notifyMenuExit.Click += new EventHandler(NotifyMenu_Exit_Click);

			// Create the notification menu
			_notifyMenu = new ContextMenuStrip(_components);
			_notifyMenu.Name = nameof(_notifyMenu);
			_notifyMenu.Size = new System.Drawing.Size(197, 104);
			_notifyMenu.SuspendLayout();
			_notifyMenu.Items.Add(_notifyMenuOpenUnrealGameSync);
			_notifyMenu.Items.Add(_notifyMenuOpenUnrealGameSyncSeparator);
			_notifyMenu.Items.Add(_notifyMenuSyncNow);
			_notifyMenu.Items.Add(_notifyMenuLaunchEditor);
			_notifyMenu.Items.Add(_notifyMenuExitSeparator);
			_notifyMenu.Items.Add(_notifyMenuExit);
			_notifyMenu.ResumeLayout(false);

			// Create the notification icon
			_notifyIcon = new NotifyIcon(_components);
			_notifyIcon.ContextMenuStrip = _notifyMenu;
			_notifyIcon.Icon = Properties.Resources.Icon;
			_notifyIcon.Text = "UnrealGameSync";
			_notifyIcon.Visible = true;
			_notifyIcon.DoubleClick += new EventHandler(NotifyIcon_DoubleClick);
			_notifyIcon.MouseDown += new MouseEventHandler(NotifyIcon_MouseDown);

			// Create the startup tasks
			List<(UserSelectedProjectSettings, ModalTask<OpenProjectInfo>)> startupTasks = new List<(UserSelectedProjectSettings, ModalTask<OpenProjectInfo>)>();
			foreach (UserSelectedProjectSettings projectSettings in _settings.OpenProjects)
			{
				if (IsValidProject(projectSettings, _logger))
				{
					ILogger<OpenProjectInfo> logger = serviceProvider.GetRequiredService<ILogger<OpenProjectInfo>>();
					Task<OpenProjectInfo> startupTask = Task.Run(() => OpenProjectInfo.CreateAsync(defaultPerforceSettings, projectSettings, _settings, false, logger, _startupCancellationSource.Token), _startupCancellationSource.Token);
					startupTasks.Add((projectSettings, new ModalTask<OpenProjectInfo>(startupTask)));
				}
			}

			IHordeClient? hordeAuthState = _serviceProvider.GetService<IHordeClient>();
			_startupTask = Task.Run(() => WaitForStartupTasks(startupTasks, hordeAuthState));

			_startupWindow = new ModalTaskWindow("Opening Projects", "Opening projects, please wait...", FormStartPosition.CenterScreen, _startupTask, _startupCancellationSource);
			_components.Add(_startupWindow);

			if (restoreState)
			{
				if (_settings.WindowVisible)
				{
					_startupWindow.Show();
				}
			}
			else
			{
				_startupWindow.Show();
				_startupWindow.Activate();
			}
			_startupWindow.FormClosed += (s, e) => OnStartupComplete(startupTasks);
		}

		public static ApplicationThemeManager? GetThemeManager()
		{
			return s_instance?._themeManager;
		}

		public static ApplicationTheme GetTheme()
		{
			return s_instance?._settings.Theme ?? ApplicationTheme.ConstructDefaultTheme();
		}

		static bool IsValidProject(UserSelectedProjectSettings projectSettings, ILogger logger)
		{
			try
			{
				bool valid = projectSettings.Type != UserSelectedProjectType.Local || File.Exists(projectSettings.LocalPath);
				return valid;
			}
			catch
			{
				logger.LogWarning("Invalid project: {Project}", projectSettings);
				return false;
			}
		}

		static async Task WaitForStartupTasks(List<(UserSelectedProjectSettings, ModalTask<OpenProjectInfo>)> startupTasks, IHordeClient? hordeClient)
		{
			foreach ((_, ModalTask<OpenProjectInfo> modalTask) in startupTasks)
			{
				try
				{
					await modalTask.Task;
				}
				catch (PerforceException)
				{
				}
				catch (OperationCanceledException)
				{
				}
				catch (UserErrorException)
				{
				}
				catch (Exception ex)
				{
					Program.CaptureException(ex);
				}
			}

			if (hordeClient != null)
			{
				await hordeClient.GetAccessTokenAsync(false, CancellationToken.None);
			}
		}

		private UserSelectedProjectSettings UpgradeSelectedProjectSettings(UserSelectedProjectSettings project)
		{
			if (project.ServerAndPort == null || String.Equals(project.ServerAndPort, _defaultPerforceSettings.ServerAndPort, StringComparison.OrdinalIgnoreCase))
			{
				if (project.UserName == null || String.Equals(project.UserName, _defaultPerforceSettings.UserName, StringComparison.OrdinalIgnoreCase))
				{
					project = new UserSelectedProjectSettings(null, null, project.Type, project.ClientPath, project.LocalPath);
				}
			}
			return project;
		}

		private void OnStartupComplete(List<(UserSelectedProjectSettings, ModalTask<OpenProjectInfo>)> startupTasks)
		{
			if (_startupWindow == null)
			{
				return;
			}

			// Close the startup window
			bool visible = _startupWindow!.Visible;
			_startupWindow = null;

			// Clear out the cache folder
			Utility.ClearPrintCache(_cacheFolder);

			// Get the application path
			string originalExe = Assembly.GetExecutingAssembly().Location;
			if (Path.GetExtension(originalExe).Equals(".dll", StringComparison.OrdinalIgnoreCase))
			{
				string newExecutable = Path.ChangeExtension(originalExe, ".exe");
				if (File.Exists(newExecutable))
				{
					originalExe = newExecutable;
				}
			}

			// Create the main window 
			_mainWindowInstance = new MainWindow(_updateMonitor, _apiUrl, _dataFolder, _updateSpawn ?? originalExe, _preview, startupTasks, _defaultPerforceSettings, _serviceProvider, _settings, _uri);
			_components.Add(_mainWindowInstance);
			if (visible)
			{
				_mainWindowInstance.Show();
				if (!_restoreState)
				{
					_mainWindowInstance.Activate();
				}
			}
			_mainWindowInstance.FormClosed += MainWindowInstance_FormClosed;
		}

		private void MainWindowInstance_FormClosed(object? sender, FormClosedEventArgs e)
		{
			ExitThread();
		}

		private void OnActivationListenerCallback()
		{
			if (_mainWindowInstance != null && !_mainWindowInstance.IsDisposed)
			{
				_mainWindowInstance.ShowAndActivate();
			}
		}

		private void OnActivationListenerAsyncCallback()
		{
			_mainThreadSynchronizationContext.Post((o) => OnActivationListenerCallback(), null);
		}

		private void OnUpdateAvailable(UpdateType type)
		{
			if (_mainWindowInstance != null && !_isClosing)
			{
				if (type == UpdateType.UserInitiated || _mainWindowInstance.CanPerformUpdate())
				{
					_isClosing = true;
					_mainWindowInstance.ForceClose();
					_mainWindowInstance = null;
				}
			}
		}

		private void OnUpdateAvailableCallback(UpdateType type)
		{
			_mainThreadSynchronizationContext.Post((o) => OnUpdateAvailable(type), null);
		}

		protected override void Dispose(bool disposing)
		{
			base.Dispose(disposing);

			if (_activationListener != null)
			{
				_activationListener.OnActivate -= OnActivationListenerAsyncCallback;
				_activationListener.Stop();
				_activationListener.Dispose();
				_activationListener = null!;
			}

			if (_components != null)
			{
				_components.Dispose();
				_components = null!;
			}

			if (_notifyIcon != null)
			{
				_notifyIcon.Dispose();
				_notifyIcon = null!;
			}

			if (_mainWindowInstance != null)
			{
				_mainWindowInstance.ForceClose();
				_mainWindowInstance = null!;
			}

			if (_startupWindow != null)
			{
				_startupWindow.Close();
				_startupWindow = null;
			}

			if (_startupCancellationSource != null)
			{
				_startupCancellationSource.Dispose();
				_startupCancellationSource = null!;
			}

			if (_synchronizationContext != null)
			{
				_synchronizationContext.Dispose();
				_synchronizationContext = null;
			}
			
			_themeManager.Dispose();
		}

		private void NotifyIcon_MouseDown(object? sender, MouseEventArgs e)
		{
			// Have to set up this stuff here, because the menu is laid out before Opening() is called on it after mouse-up.
			bool canSyncNow = _mainWindowInstance != null && _mainWindowInstance.CanSyncNow();
			bool canLaunchEditor = _mainWindowInstance != null && _mainWindowInstance.CanLaunchEditor();
			_notifyMenuSyncNow.Visible = canSyncNow;
			_notifyMenuLaunchEditor.Visible = canLaunchEditor;
			_notifyMenuExitSeparator.Visible = canSyncNow || canLaunchEditor;

			// Show the startup window, if not already visible
			_startupWindow?.Show();
		}

		private void NotifyIcon_DoubleClick(object? sender, EventArgs e)
		{
			_mainWindowInstance?.ShowAndActivate();
		}

		private void NotifyMenu_OpenUnrealGameSync_Click(object? sender, EventArgs e)
		{
			_startupWindow?.ShowAndActivate();
			_mainWindowInstance?.ShowAndActivate();
		}

		private void NotifyMenu_SyncNow_Click(object? sender, EventArgs e)
		{
			_mainWindowInstance?.SyncLatestChange();
		}

		private void NotifyMenu_LaunchEditor_Click(object? sender, EventArgs e)
		{
			_mainWindowInstance?.LaunchEditor();
		}

		private void NotifyMenu_Exit_Click(object? sender, EventArgs e)
		{
			if (_mainWindowInstance != null && !_mainWindowInstance.ConfirmClose())
			{
				return;
			}

			if (_startupWindow != null)
			{
				_startupWindow.Close();
				_startupWindow = null;
			}

			if (_mainWindowInstance != null)
			{
				_mainWindowInstance.ForceClose();
				_mainWindowInstance = null;
			}

			ExitThread();
		}

		protected override void ExitThreadCore()
		{
			base.ExitThreadCore();

			if (_notifyIcon != null)
			{
				_notifyIcon.Visible = false;
			}
		}
	}
}
