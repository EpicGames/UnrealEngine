// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Text;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;

namespace UnrealGameSync
{
	public class RestException : Exception
	{
		public RestException(HttpMethod method, string? uri, Exception innerException)
			: base(String.Format("Error executing {0} {1}", method, uri), innerException)
		{
		}

		public override string ToString()
		{
			return String.Format("{0}\n\n{1}", Message, InnerException!.ToString());
		}
	}

	public static class RestApi
	{
		static readonly HttpClient s_httpClient = new HttpClient();

		private static async Task<string> SendRequestInternalAsync(string url, HttpMethod method, string? requestBody, CancellationToken cancellationToken)
		{
			using (HttpRequestMessage request = new HttpRequestMessage(method, url))
			{
				// Add json to request body
				if (!String.IsNullOrEmpty(requestBody))
				{
					if (method == HttpMethod.Post || method == HttpMethod.Put)
					{
						request.Content = new ByteArrayContent(Encoding.UTF8.GetBytes(requestBody));
						request.Content.Headers.ContentType = new MediaTypeHeaderValue("application/json");
					}
				}

				try
				{
					using (HttpResponseMessage response = await s_httpClient.SendAsync(request, cancellationToken))
					{
						return await response.Content.ReadAsStringAsync(cancellationToken);
					}
				}
				catch (Exception ex)
				{
					throw new RestException(method, request.RequestUri?.ToString(), ex);
				}
			}
		}

		public static Task<string> PostAsync(string url, string requestBody, CancellationToken cancellationToken)
		{
			return SendRequestInternalAsync(url, HttpMethod.Post, requestBody, cancellationToken);
		}

		public static Task<string> GetAsync(string url, CancellationToken cancellationToken)
		{
			return SendRequestInternalAsync(url, HttpMethod.Get, null, cancellationToken);
		}

		public static async Task<T> GetAsync<T>(string url, CancellationToken cancellationToken)
		{
			return JsonSerializer.Deserialize<T>(await GetAsync(url, cancellationToken), Utility.DefaultJsonSerializerOptions)!;
		}

		public static Task<string> PutAsync<T>(string url, T obj, CancellationToken cancellationToken)
		{
			return SendRequestInternalAsync(url, HttpMethod.Put, JsonSerializer.Serialize(obj), cancellationToken);
		}
	}
}
