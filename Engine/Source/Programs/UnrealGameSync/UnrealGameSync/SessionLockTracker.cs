// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Win32;

namespace UnrealGameSync
{
	public class SessionLockTracker
	{
		private volatile bool _sessionLocked = false;
		
		public bool IsSessionLocked => _sessionLocked;

		public SessionLockTracker()
		{
			SystemEvents.SessionSwitch += OnSessionSwitch;
		}

		private void OnSessionSwitch(object sender, SessionSwitchEventArgs e)
		{
			_sessionLocked = e.Reason switch
			{
				SessionSwitchReason.SessionLock => true,
				SessionSwitchReason.SessionUnlock => false,
				SessionSwitchReason.ConsoleDisconnect or SessionSwitchReason.RemoteDisconnect => true,
				SessionSwitchReason.ConsoleConnect or SessionSwitchReason.RemoteConnect => false,
				_ => _sessionLocked
			};
		}
	}
}
