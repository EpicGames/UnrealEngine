// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;

namespace UnrealGameSync
{
	enum TaskbarState
	{
		NoProgress = 0,
		Indeterminate = 0x1,
		Normal = 0x2,
		Error = 0x4,
		Paused = 0x8
	}

	static class Taskbar
	{
		[ComImport, Guid("ea1afb91-9e28-4b86-90e9-9e9f8a5eefaf"), InterfaceType(ComInterfaceType.InterfaceIsIUnknown)]
		interface ITaskbarList3
		{
			void HrInit();
			void AddTab(IntPtr hWnd);
			void DeleteTab(IntPtr hWnd);
			void ActivateTab(IntPtr hWnd);
			void SetActiveAlt(IntPtr hWnd);
			void MarkFullscreenWindow(IntPtr hWnd, int fFullscreen);
			void SetProgressValue(IntPtr hWnd, ulong ullCompleted, ulong ullTotal);
			void SetProgressState(IntPtr hWnd, TaskbarState state);
		}

		[ComImport, Guid("56FDF344-FD6D-11d0-958A-006097C9A090"), ClassInterfaceAttribute(ClassInterfaceType.None)]
		class TaskbarList3
		{
		}

		static readonly ITaskbarList3? _interface = CreateTaskbarListInterface();

		static ITaskbarList3? CreateTaskbarListInterface()
		{
			if (Environment.OSVersion.Version >= new Version(6, 1))
			{
				return new TaskbarList3() as ITaskbarList3;
			}
			else
			{
				return null;
			}
		}

		public static void SetState(IntPtr windowHandle, TaskbarState state)
		{
			if (_interface != null)
			{
				try
				{
					_interface.SetProgressState(windowHandle, state);
				}
				catch
				{
				}
			}
		}

		public static void SetProgress(IntPtr windowHandle, ulong completed, ulong total)
		{
			if (_interface != null)
			{
				try
				{
					_interface.SetProgressValue(windowHandle, completed, total);
				}
				catch
				{
				}
			}
		}
	}
}
