<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <OutputType>WinExe</OutputType>
    <TargetFramework>net10.0-windows</TargetFramework>
    <UseWindowsForms>true</UseWindowsForms>
    <Product>UnrealGameSync</Product>
    <Company>Epic Games, Inc</Company>
    <Copyright>Copyright Epic Games, Inc. All Rights Reserved.</Copyright>
    <RootNamespace>UnrealGameSync</RootNamespace>
    <AssemblyName>UnrealGameSync</AssemblyName>
    <StartupObject>UnrealGameSync.Program</StartupObject>
    <ApplicationIcon>Resources\Icon.ico</ApplicationIcon>
    <ApplicationManifest>Properties\app.manifest</ApplicationManifest>
    <NeutralLanguage>en-US</NeutralLanguage>
    <Configurations>$(Configurations);DebugAutoUpdate;ReleaseAutoUpdate</Configurations>
  </PropertyGroup>

  <PropertyGroup Condition="'$(Configuration)|$(Platform)'=='Release|AnyCPU'">
    <DefineConstants>$(DefineConstants);WITH_TELEMETRY</DefineConstants>
  </PropertyGroup>
  <PropertyGroup Condition="'$(Configuration)|$(Platform)'=='Debug|AnyCPU'">
    <DefineConstants>$(DefineConstants);WITH_TELEMETRY</DefineConstants>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)' == 'DebugAutoUpdate' or '$(Configuration)' == 'ReleaseAutoUpdate'">
    <DefineConstants>$(DefineConstants);WITH_AUTOUPDATE</DefineConstants>
  </PropertyGroup>
  <ItemGroup>
    <PackageReference Include="EnvDTE" Version="17.14.40260" />
    <PackageReference Include="Microsoft.VisualStudio.Setup.Configuration.Interop" Version="3.14.2075" />
    <PackageReference Include="Sentry" Version="3.15.0" />
    <PackageReference Include="Serilog.Expressions" Version="3.2.1" />
    <PackageReference Include="Serilog.Extensions.Hosting" Version="4.2.0" />
    <PackageReference Include="Serilog.Extensions.Logging" Version="3.1.0" />
    <PackageReference Include="Serilog.Sinks.File" Version="5.0.0" />
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.OIDC\EpicGames.OIDC.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Perforce\EpicGames.Perforce.csproj" />
    <ProjectReference Include="..\UnrealGameSyncShared\UnrealGameSyncShared.csproj" />
  </ItemGroup>

  <ItemGroup Condition="Exists('..\..\..\..\Binaries\Win64\zen.exe')">
    <None Include="..\..\..\..\Binaries\Win64\zen.exe" TargetPath="Binaries\Win64\zen.exe" Link="Binaries\zen.exe">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Include="..\..\..\..\Binaries\Linux\zen" TargetPath="Binaries\Linux\zen" Link="Binaries\zen">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Include="..\..\..\..\Binaries\Mac\zen" TargetPath="Binaries\Mac\zen" Link="Binaries\zen">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
  </ItemGroup>

  <ItemGroup>
    <Compile Update="Properties\Settings.Designer.cs">
      <DesignTimeSharedInput>True</DesignTimeSharedInput>
      <AutoGen>True</AutoGen>
      <DependentUpon>Settings.settings</DependentUpon>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <None Update="Properties\Settings.settings">
      <Generator>SettingsSingleFileGenerator</Generator>
      <LastGenOutput>Settings.Designer.cs</LastGenOutput>
    </None>
  </ItemGroup>
  <ItemGroup>
    <Compile Include="..\..\..\..\Restricted\NotForLicensees\Source\Programs\UnrealGameSync\UnrealGameSync\*.cs">
      <Link>NotForLicensees\%(FileName).cs</Link>
    </Compile>
  </ItemGroup>
  <Import Project="..\..\..\..\Restricted\NotForLicensees\Source\Programs\UnrealGameSync\UnrealGameSync\UnrealGameSync.csproj" Condition="Exists('..\..\..\..\Restricted\NotForLicensees\Source\Programs\UnrealGameSync\UnrealGameSync\UnrealGameSync.csproj')" />
</Project>
