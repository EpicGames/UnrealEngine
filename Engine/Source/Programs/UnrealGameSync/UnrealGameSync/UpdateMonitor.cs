// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Tools;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	enum UpdateType
	{
		Background,
		UserInitiated,
	}

	abstract class UpdateMonitor : IAsyncDisposable
	{
		public bool IsUpdateAvailable
		{
			get;
			private set;
		}

		public Action<UpdateType>? OnUpdateAvailable;

		public bool OpenSettings
		{
			get;
			private set;
		}

		public abstract ValueTask DisposeAsync();

		public void TriggerUpdate(UpdateType updateType, bool openSettings)
		{
			OpenSettings = openSettings;
			IsUpdateAvailable = true;
			if (OnUpdateAvailable != null)
			{
				OnUpdateAvailable(updateType);
			}
		}
	}

	class NullUpdateMonitor : UpdateMonitor
	{
		public override ValueTask DisposeAsync() => default;
	}

	class HordeUpdateMonitor : UpdateMonitor
	{
		readonly ToolId _toolId;
		readonly string _currentVersion;
		readonly IServiceProvider _serviceProvider;
		readonly BackgroundTask _backgroundTask;
		readonly ILogger _logger;

		public HordeUpdateMonitor(string currentVersion, IServiceProvider serviceProvider)
			: this(DeploymentSettings.Instance.HordeToolId, currentVersion, serviceProvider)
		{
		}

		public HordeUpdateMonitor(ToolId toolId, string currentVersion, IServiceProvider serviceProvider)
		{
			_toolId = toolId;
			_currentVersion = currentVersion;
			_serviceProvider = serviceProvider;
			_logger = serviceProvider.GetRequiredService<ILogger<HordeUpdateMonitor>>();
			_backgroundTask = BackgroundTask.StartNew(ctx => CheckForUpdatesLoopAsync(ctx));
		}

		public override async ValueTask DisposeAsync()
		{
			await _backgroundTask.DisposeAsync();
		}

		public async Task CheckForUpdatesLoopAsync(CancellationToken cancellationToken)
		{
			for (; ; )
			{
				// Check if there's a new build available on the server
				HordeHttpClient hordeHttpClient = _serviceProvider.GetRequiredService<HordeHttpClient>();
				try
				{
					GetToolResponse response = await hordeHttpClient.GetToolAsync(_toolId, cancellationToken);
					if (response.Deployments.Count == 0)
					{
						_logger.LogWarning("No deployments on Horde server for tool {ToolId}", _toolId);
					}
					else
					{
						string latestUrl = new Uri(hordeHttpClient.BaseUrl, $"api/v1/tools/{_toolId}/deployments/{response.Deployments[^1].Id}").ToString();
						if (String.Equals(latestUrl, _currentVersion, StringComparison.OrdinalIgnoreCase))
						{
							_logger.LogInformation("Currently running latest version ({LatestUrl})", latestUrl);
						}
						else
						{
							_logger.LogInformation("Triggering update request {CurrentUrl} -> {LatestUrl}", _currentVersion, latestUrl);
							TriggerUpdate(UpdateType.Background, false);
						}
					}
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Error while checking for tool updates: {Message}", ex.Message);
				}

				// Wait a while before checking again
				await Task.Delay(TimeSpan.FromMinutes(5.0), cancellationToken);
			}
		}
	}

	class PerforceUpdateMonitor : UpdateMonitor
	{
		Task? _workerTask;
#pragma warning disable CA2213 // warning CA2213: 'UpdateMonitor' contains field '_cancellationSource' that is of IDisposable type 'CancellationTokenSource', but it is never disposed. Change the Dispose method on 'UpdateMonitor' to call Close or Dispose on this field.
		readonly CancellationTokenSource _cancellationSource = new CancellationTokenSource();
#pragma warning restore CA2213
		readonly ILogger _logger;

		public PerforceUpdateMonitor(IPerforceSettings perforceSettings, string? watchPath, IServiceProvider serviceProvider)
		{
			_logger = serviceProvider.GetRequiredService<ILogger<UpdateMonitor>>();

			if (watchPath != null)
			{
				_logger.LogInformation("Watching for updates on {WatchPath}", watchPath);
				_workerTask = Task.Run(() => PollForUpdatesAsync(perforceSettings, watchPath, _cancellationSource.Token));
			}
		}

		public override async ValueTask DisposeAsync()
		{
			OnUpdateAvailable = null;

			if (_workerTask != null)
			{
				await _cancellationSource.CancelAsync();

				await _workerTask;
				_workerTask = null;

				_cancellationSource.Dispose();
			}
		}

		async Task PollForUpdatesAsync(IPerforceSettings perforceSettings, string watchPath, CancellationToken cancellationToken)
		{
			for (; ; )
			{
				try
				{
					await Task.Delay(TimeSpan.FromMinutes(5.0), cancellationToken);
				}
				catch (OperationCanceledException)
				{
					break;
				}

				IPerforceConnection? perforce = null;
				try
				{
					perforce = await PerforceConnection.CreateAsync(perforceSettings, _logger);

					PerforceResponseList<ChangesRecord> changes = await perforce.TryGetChangesAsync(ChangesOptions.None, -1, ChangeStatus.Submitted, watchPath, cancellationToken);
					if (changes.Succeeded && changes.Data.Count > 0)
					{
						TriggerUpdate(UpdateType.Background, false);
					}
				}
				catch (PerforceException ex)
				{
					_logger.LogInformation(ex, "Perforce exception while attempting to poll for updates.");
				}
				catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
				{
					break;
				}
				catch (Exception ex)
				{
					_logger.LogWarning(ex, "Exception while attempting to poll for updates.");
					Program.CaptureException(ex);
				}
				finally
				{
					perforce?.Dispose();
				}
			}
		}
	}
}
