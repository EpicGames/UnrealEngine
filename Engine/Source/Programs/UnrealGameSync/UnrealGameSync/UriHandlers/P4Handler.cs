// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealGameSync
{

	/// <summary>
	/// P4 URI handler
	/// </summary>
	static class P4Handler
	{
		[UriHandler(true)]
		public static UriResult P4V(string depotPath)
		{
			string commandLine = String.Format("-s \"{0}\"", depotPath);

			if (!Utility.SpawnHiddenProcess("p4v.exe", commandLine))
			{
				return new UriResult() { Error = String.Format("Error spawning p4v.exe with command line: {0}", commandLine) };
			}

			return new UriResult() { Success = true };
		}

		[UriHandler(true)]
		public static UriResult Timelapse(string depotPath, int line = -1)
		{
			string commandLine = String.Format("timelapse {0}{1}", line == -1 ? "" : String.Format(" -l {0} ", line), depotPath);

			Program.SpawnP4Vc(commandLine);

			return new UriResult() { Success = true };
		}

		[UriHandler(true)]
		public static UriResult Change(int number)
		{
			Program.SpawnP4Vc($"change {number}");
			return new UriResult() { Success = true };
		}
	}
}