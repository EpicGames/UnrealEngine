// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Runtime.InteropServices;
using System.Runtime.InteropServices.ComTypes;

namespace UnrealGameSync
{
	static class RdpHandler
	{
		const uint CredTypeGeneric = 1;
		const uint CredTypeDomainPassword = 2;
		const uint CredPersistLocalMachine = 2;

#pragma warning disable IDE0044
#pragma warning disable IDE1006
		[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Auto)]
		struct CredentialAttribute
		{
			string Keyword;
			uint Flags;
			uint ValueSize;
			IntPtr Value;
		}

		[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Unicode)]
		class Credential
		{
			public uint Flags;
			public uint Type;
			public string? TargetName;
			public string? Comment;
			public FILETIME LastWritten;
			public int CredentialBlobSize;
			public IntPtr CredentialBlob;
			public uint Persist;
			public uint AttributeCount;
			public IntPtr Attributes;
			public string? TargetAlias;
			public string? UserName;
		}
#pragma warning restore IDE0044
#pragma warning restore IDE1006

		[DllImport("Advapi32.dll", EntryPoint = "CredReadW", CharSet = CharSet.Unicode, SetLastError = true)]
		static extern bool CredRead(string target, uint type, int reservedFlag, out IntPtr buffer);

		[DllImport("Advapi32.dll", EntryPoint = "CredWriteW", CharSet = CharSet.Unicode, SetLastError = true)]
		static extern bool CredWrite(Credential userCredential, uint flags);

		[DllImport("advapi32.dll", SetLastError = true)]
		static extern bool CredFree(IntPtr buffer);

		public static bool ReadCredential(string name, uint type, [NotNullWhen(true)] out string? userName, [NotNullWhen(true)] out string? password)
		{
			IntPtr buffer = IntPtr.Zero;
			try
			{
				if (!CredRead(name, type, 0, out buffer))
				{
					userName = null;
					password = null;
					return false;
				}

				Credential? credential = Marshal.PtrToStructure<Credential>(buffer);
				if (credential == null || credential.UserName == null || credential.CredentialBlob == IntPtr.Zero)
				{
					userName = null;
					password = null;
					return false;
				}

				userName = credential.UserName;
				password = Marshal.PtrToStringUni(credential.CredentialBlob, credential.CredentialBlobSize / sizeof(char));
				return true;
			}
			finally
			{
				if (buffer != IntPtr.Zero)
				{
					CredFree(buffer);
				}
			}
		}

		public static void WriteCredential(string name, uint type, string userName, string password)
		{
			Credential credential = new Credential();
			try
			{
				credential.Type = type;
				credential.Persist = CredPersistLocalMachine;
				credential.CredentialBlobSize = password.Length * sizeof(char);
				credential.TargetName = name;
				credential.CredentialBlob = Marshal.StringToCoTaskMemUni(password);
				credential.UserName = userName;
				CredWrite(credential, 0);
			}
			finally
			{
				if (credential.CredentialBlob != IntPtr.Zero)
				{
					Marshal.FreeCoTaskMem(credential.CredentialBlob);
				}
			}
		}

		[UriHandler(terminate: true)]
		public static UriResult Rdp(string host)
		{
			// Copy the credentials from a generic Windows credential to 
			if (!ReadCredential(host, CredTypeDomainPassword, out _, out _))
			{
				if (ReadCredential("UnrealGameSync:RDP", CredTypeGeneric, out string? userName, out string? password))
				{
					WriteCredential(host, CredTypeDomainPassword, userName, password);
				}
			}

			using (Process process = new Process())
			{
				process.StartInfo.FileName = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.System), "mstsc.exe");
				process.StartInfo.ArgumentList.Add($"/v:{host}");
				process.StartInfo.ArgumentList.Add($"/f");
				process.Start();
			}

			return new UriResult() { Success = true };
		}
	}
}

