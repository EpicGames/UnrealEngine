// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Windows.Forms;

namespace UnrealGameSync
{
	/// <summary>
	/// UGS handler 
	/// </summary>
	static class UgsHandler
	{
		[UriHandler]
		public static UriResult OpenProject(string stream, string project, bool sync = false)
		{
			// Create the request
			using (MemoryStream inputDataStream = new MemoryStream())
			{
				using (BinaryWriter writer = new BinaryWriter(inputDataStream))
				{
					writer.Write(stream);
					writer.Write(project);
				}

				AutomationRequestInput input = new AutomationRequestInput(sync ? AutomationRequestType.SyncProject : AutomationRequestType.OpenProject, inputDataStream.GetBuffer());
				return new UriResult() { Success = true, Request = new AutomationRequest(input) };
			}
		}

		[UriHandler]
		public static UriResult BuildStep(string project, string stream, string step, string changelist, string arguments)
		{
			MessageBox.Show(String.Format("Project: {0}\nStream: {1}\nStep: {2}\nChange: {3}\nArguments: {4}", project, stream, step, changelist, arguments), "UGS Build Step Handler");

			return new UriResult() { Success = true };
		}

		[UriHandler]
		public static UriResult Execute(string stream, int changelist, string command, string project = "")
		{
			using (MemoryStream inputDataStream = new MemoryStream())
			{
				using (BinaryWriter writer = new BinaryWriter(inputDataStream))
				{
					writer.Write(stream);
					writer.Write(changelist);
					writer.Write(command);
					writer.Write(project);
				}

				AutomationRequestInput input = new AutomationRequestInput(AutomationRequestType.ExecCommand, inputDataStream.GetBuffer());
				return new UriResult() { Success = true, Request = new AutomationRequest(input) };
			}
		}

		[UriHandler]
		public static UriResult OpenIssue(int id)
		{
			using (MemoryStream inputDataStream = new MemoryStream())
			{
				using (BinaryWriter writer = new BinaryWriter(inputDataStream))
				{
					writer.Write(id);
				}

				AutomationRequestInput input = new AutomationRequestInput(AutomationRequestType.OpenIssue, inputDataStream.GetBuffer());
				return new UriResult() { Success = true, Request = new AutomationRequest(input) };
			}
		}

		[UriHandler]
		public static UriResult RunCustomTool(string toolName, string args)
		{
			using (MemoryStream inputDataStream = new MemoryStream())
			{
				using (BinaryWriter writer = new BinaryWriter(inputDataStream))
				{
					writer.Write(toolName);
					writer.Write(args);
				}
				AutomationRequestInput input = new AutomationRequestInput(AutomationRequestType.RunCustomTool, inputDataStream.GetBuffer());

				return new UriResult() { Success = true, Request = new AutomationRequest(input) };
			}
		}
	}
}