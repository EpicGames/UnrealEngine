// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.Specialized;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Threading;
using System.Web;
using System.Windows.Forms;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;

namespace UnrealGameSync
{

	/// <summary>
	/// Uri handler result
	/// </summary>
	class UriResult
	{
		public bool Success = false;
		public string? Error;
		public AutomationRequest? Request = null;
	}

	/// <summary>
	/// Main handler for Uri requests
	/// </summary>
	static class UriHandler
	{
		public static UriResult HandleUri(string uriIn)
		{
			try
			{
				Uri uri = new Uri(uriIn);

				// Check if this is a registered uri request
				if (!Handlers.TryGetValue(uri.Host, out MethodInfo? info))
				{
					return new UriResult() { Error = String.Format("Unknown Uri {0}", uri.Host) };
				}

				NameValueCollection query = HttpUtility.ParseQueryString(uri.Query);

				List<object> parameters = new List<object>();

				foreach (ParameterInfo param in info.GetParameters())
				{
					string? value = query.Get(param.Name);

					if (value == null)
					{
						if (!param.HasDefaultValue)
						{
							return new UriResult() { Error = String.Format("Uri {0} is missing required parameter {1}", uri.Host, param.Name) };
						}

						parameters.Add(param.DefaultValue!);
						continue;
					}

					if (param.ParameterType == typeof(string))
					{
						parameters.Add(value);
					}
					else if (param.ParameterType == typeof(bool))
					{
						if (value.Equals("true", StringComparison.OrdinalIgnoreCase))
						{
							parameters.Add(true);
						}
						else if (value.Equals("false", StringComparison.OrdinalIgnoreCase))
						{
							parameters.Add(false);
						}
						else
						{
							return new UriResult() { Error = String.Format("Uri {0} bool parameter {1} must be true or false", uri.Host, param.Name) };
						}
					}
					else if (param.ParameterType == typeof(int))
					{
						int numberValue;
						if (!Int32.TryParse(value, out numberValue))
						{
							return new UriResult() { Error = String.Format("Uri {0} invalid integer parameter {1} : {2}", uri.Host, param.Name, value) };
						}
						parameters.Add(numberValue);
					}
					else if (param.ParameterType == typeof(float))
					{
						float numberValue;
						if (!Single.TryParse(value, out numberValue))
						{
							return new UriResult() { Error = String.Format("Uri {0} invalid number parameter {1} : {2}", uri.Host, param.Name, value) };
						}
						parameters.Add(numberValue);
					}
				}

				return (UriResult)info.Invoke(null, parameters.ToArray())!;
			}
			catch (Exception ex)
			{
				return new UriResult() { Error = ex.Message };
			}
		}

		public const string InstallHandlerArg = "-InstallHandler";
		public const string UninstallHandlerArg = "-UninstallHandler";
		public const string ElevatedArg = "-Elevated";

		/// <summary>
		/// Handle URI passed in via command lines
		/// </summary>
		public static bool ProcessCommandLine(string[] args, bool firstInstance, EventWaitHandle? activateEvent = null)
		{
			if (args.Any(x => x.Equals(InstallHandlerArg, StringComparison.OrdinalIgnoreCase)))
			{
				if (args.Any(x => x.Equals(ElevatedArg, StringComparison.OrdinalIgnoreCase)))
				{
					ProtocolHandlerUtils.InstallElevated();
				}
				else
				{
					ProtocolHandlerUtils.Install();
				}
				return true;
			}
			else if (args.Any(x => x.Equals(UninstallHandlerArg, StringComparison.OrdinalIgnoreCase)))
			{
				if (args.Any(x => x.Equals(ElevatedArg, StringComparison.OrdinalIgnoreCase)))
				{
					ProtocolHandlerUtils.UninstallElevated();
				}
				else
				{
					ProtocolHandlerUtils.Uninstall();
				}
				return true;
			}
			else
			{
				string uriIn = String.Empty;
				for (int idx = 0; idx < args.Length; idx++)
				{
					const string prefix = "-uri=";
					if (args[idx].StartsWith(prefix, StringComparison.OrdinalIgnoreCase))
					{
						uriIn = args[idx].Substring(prefix.Length);
					}
				}

				if (String.IsNullOrEmpty(uriIn))
				{
					return false;
				}

				Uri uri;
				try
				{
					uri = new Uri(uriIn);
				}
				catch
				{
					MessageBox.Show(String.Format("Invalid URI: {0}", uriIn));
					return true;
				}

				MethodInfo? handler;
				if (!Handlers.TryGetValue(uri.Host, out handler))
				{
					MessageBox.Show(String.Format("Unknown action from URI request ('{0}')", uri.Host));
					return true;
				}

				UriHandlerAttribute attribute = handler.GetCustomAttribute<UriHandlerAttribute>()!;

				// handle case where we terminate after invoking handler
				if (attribute.Terminate)
				{
					UriResult result = HandleUri(uriIn);
					if (!result.Success)
					{
						MessageBox.Show(result.Error);
					}
					return true;
				}

				if (!firstInstance)
				{
					if (activateEvent != null)
					{
						activateEvent.Set();
					}

					// send to main UGS process using IPC
					AutomationServer.SendUri(uriIn);
					return true;
				}

				// we're in the main UGS process, which was also launched, defer handling to after main window is created
				return false;
			}
		}

		static UriHandler()
		{
			foreach (Type type in Assembly.GetExecutingAssembly().GetTypes())
			{
				List<MethodInfo> handlerMethods = new List<MethodInfo>(type.GetMethods().Where(methodInfo => methodInfo.GetCustomAttribute<UriHandlerAttribute>() != null));
				foreach (MethodInfo methodInfo in handlerMethods)
				{
					if (!methodInfo.IsStatic)
					{
						throw new Exception(String.Format("UriHandler method {0} must be static", methodInfo.Name));
					}

					if (methodInfo.ReturnType != typeof(UriResult))
					{
						throw new Exception(String.Format("UriHandler method {0} must return UriResult type", methodInfo.Name));
					}

					if (Handlers.ContainsKey(methodInfo.Name))
					{
						throw new Exception(String.Format("UriHandler method {0} clashes with another handler", methodInfo.Name));
					}

					foreach (ParameterInfo parameterInfo in methodInfo.GetParameters())
					{
						Type parameterType = parameterInfo.ParameterType;
						if (parameterType != typeof(bool) && parameterType != typeof(string) && parameterType != typeof(float) && parameterType != typeof(int))
						{
							throw new Exception(String.Format("UriHandler method parameter {0} must be bool, string, int, or float", parameterInfo.Name));
						}
					}

					Handlers[methodInfo.Name] = methodInfo;
				}
			}
		}

		/// <summary>
		/// All available handler infos
		/// </summary>
		static readonly Dictionary<string, MethodInfo> Handlers = new Dictionary<string, MethodInfo>(StringComparer.OrdinalIgnoreCase);

	}

	/// <summary>
	/// Method attribute for Uri handlers
	/// </summary>
	[AttributeUsage(AttributeTargets.Method)]
	sealed class UriHandlerAttribute : Attribute
	{
		public bool Terminate { get; }

		public UriHandlerAttribute(bool terminate = false)
		{
			Terminate = terminate;
		}
	}

	enum ProtocolHandlerState
	{
		Unknown,
		Installed,
		NotInstalled,
	}

	/// <summary>
	/// Development utilities to register protocol binary, this or a variation should go in installer/updater
	/// Needs administrator to edit registry
	/// </summary>
	static class ProtocolHandlerUtils
	{
		class RegistrySetting
		{
			public RegistryKey RootKey;
			public string KeyName;
			public string? ValueName;
			public string Value;

			public RegistrySetting(RegistryKey rootKey, string keyName, string? valueName, string value)
			{
				RootKey = rootKey;
				KeyName = keyName;
				ValueName = valueName;
				Value = value;
			}
		}

		public static string CurrentProcessFilePath => Path.ChangeExtension(Path.GetFullPath(Environment.ProcessPath ?? "UnrealGameSync.exe"), ".exe");

		static List<RegistrySetting> GetGlobalRegistrySettings()
		{
			string applicationPath = CurrentProcessFilePath;

			List<RegistrySetting> keys = new List<RegistrySetting>();
			keys.Add(new RegistrySetting(Registry.ClassesRoot, "UGS", null, "URL:UGS Protocol"));
			keys.Add(new RegistrySetting(Registry.ClassesRoot, "UGS", "URL Protocol", ""));
			keys.Add(new RegistrySetting(Registry.ClassesRoot, "UGS\\DefaultIcon", null, String.Format("\"{0}\",0", applicationPath)));
			keys.Add(new RegistrySetting(Registry.ClassesRoot, "UGS\\shell\\open\\command", null, String.Format("\"{0}\" -uri=\"%1\"", applicationPath)));
			return keys;
		}

		static List<RegistrySetting> GetUserRegistrySettings()
		{
			string applicationPath = CurrentProcessFilePath;

			List<RegistrySetting> keys = new List<RegistrySetting>();
			keys.Add(new RegistrySetting(Registry.CurrentUser, "Software\\Classes\\UGS", null, "URL:UGS Protocol"));
			keys.Add(new RegistrySetting(Registry.CurrentUser, "Software\\Classes\\UGS", "URL Protocol", ""));
			keys.Add(new RegistrySetting(Registry.CurrentUser, "Software\\Classes\\UGS\\DefaultIcon", null, String.Format("\"{0}\",0", applicationPath)));
			keys.Add(new RegistrySetting(Registry.CurrentUser, "Software\\Classes\\UGS\\shell\\open\\command", null, String.Format("\"{0}\" -uri=\"%1\"", applicationPath)));
			return keys;
		}

		public static ProtocolHandlerState GetState()
		{
			return GetState(GetGlobalRegistrySettings());
		}

		private static ProtocolHandlerState GetState(IEnumerable<RegistrySetting> keys)
		{
			try
			{
				bool hasAny = false;
				bool hasAll = true;

				foreach (IGrouping<RegistryKey, RegistrySetting> rootKeyGroup in keys.GroupBy(x => x.RootKey))
				{
					foreach (IGrouping<string, RegistrySetting> keyNameGroup in rootKeyGroup.GroupBy(x => x.KeyName))
					{
						using (RegistryKey? registryKey = rootKeyGroup.Key.OpenSubKey(keyNameGroup.Key))
						{
							if (registryKey == null)
							{
								hasAll = false;
							}
							else
							{
								hasAll &= keyNameGroup.All(x => (registryKey.GetValue(x.ValueName) as string) == x.Value);
								hasAny = true;
							}
						}
					}
				}

				return hasAll ? ProtocolHandlerState.Installed : hasAny ? ProtocolHandlerState.Unknown : ProtocolHandlerState.NotInstalled;
			}
			catch
			{
				return ProtocolHandlerState.Unknown;
			}
		}

		public static void Install()
		{
			RunElevated(String.Format("{0} {1}", UriHandler.InstallHandlerArg, UriHandler.ElevatedArg));
		}

		public static void InstallElevated()
		{
			try
			{
				ApplyRegistrySettings(GetGlobalRegistrySettings());
			}
			catch (Exception ex)
			{
				MessageBox.Show(String.Format("Unable to register protocol handler: {0}", ex));
			}
		}

		public static void InstallQuiet(ILogger logger)
		{
			try
			{
				ApplyRegistrySettings(GetUserRegistrySettings());
				logger.LogInformation("Installed protocol handler to user hive");
			}
			catch (Exception ex)
			{
				logger.LogWarning(ex, "Unable to install protocol handler");
			}
		}

		public static void Uninstall()
		{
			RunElevated(String.Format("{0} {1}", UriHandler.UninstallHandlerArg, UriHandler.ElevatedArg));
		}

		public static void UninstallElevated()
		{
			try
			{
				Registry.ClassesRoot.DeleteSubKeyTree("UGS", false);
			}
			catch (Exception ex)
			{
				MessageBox.Show(String.Format("Unable to register protocol handler: {0}", ex));
			}
		}

		private static void RunElevated(string arguments)
		{
			using (Process process = new Process())
			{
				process.StartInfo.FileName = Path.ChangeExtension(Assembly.GetExecutingAssembly().Location, ".exe");
				process.StartInfo.Arguments = arguments;
				process.StartInfo.Verb = "runas";
				process.StartInfo.UseShellExecute = true;
				process.Start();
				process.WaitForExit();
			}
		}

		private static void ApplyRegistrySettings(IEnumerable<RegistrySetting> settings)
		{
			foreach (IGrouping<RegistryKey, RegistrySetting> rootKeyGroup in settings.GroupBy(x => x.RootKey))
			{
				foreach (IGrouping<string, RegistrySetting> keyNameGroup in rootKeyGroup.GroupBy(x => x.KeyName))
				{
					using (RegistryKey registryKey = rootKeyGroup.Key.CreateSubKey(keyNameGroup.Key))
					{
						foreach (RegistrySetting setting in keyNameGroup)
						{
							registryKey.SetValue(setting.ValueName, setting.Value);
						}
					}
				}
			}
		}
	}
}
