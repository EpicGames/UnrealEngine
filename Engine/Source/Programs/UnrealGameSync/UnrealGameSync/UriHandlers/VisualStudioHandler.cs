// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging.Abstractions;

namespace UnrealGameSync
{
	/// <summary>
	/// VisualStudio Uri Handler
	/// </summary>
	static class VisualStudioHandler
	{
		[UriHandler(true)]
		public static UriResult VsOpen(string depotPath, int line = -1)
		{
			string tempFileName = P4Automation.PrintToTempFile(null, depotPath, NullLogger.Instance).GetAwaiter().GetResult();

			string? errorMessage;
			if (!VisualStudioAutomation.OpenFile(tempFileName, out errorMessage, line))
			{
				return new UriResult() { Error = errorMessage ?? "Unknown Visual Studio Error" };
			}

			return new UriResult() { Success = true };
		}
	}
}