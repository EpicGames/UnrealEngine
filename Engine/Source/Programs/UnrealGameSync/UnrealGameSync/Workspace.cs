// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	public sealed class Workspace : IDisposable
	{
		public IPerforceSettings PerforceSettings { get; }
		readonly WorkspaceStateWrapper _stateWrapper;
		public ReadOnlyWorkspaceState State { get; private set; }
		public ProjectInfo Project { get; }
		public DirectoryReference RootDir => _stateWrapper.RootDir;

		public SynchronizationContext SynchronizationContext { get; }
		public WorkspaceLock Lock { get; }
		readonly ILogger _logger;

		bool Syncing => _currentUpdate != null;

		WorkspaceUpdate? _currentUpdate;

		public event Action<WorkspaceUpdateContext, WorkspaceUpdateResult, string>? OnUpdateComplete;

		public event Action<ReadOnlyWorkspaceState>? OnStateChanged;

		readonly IAsyncDisposer _asyncDisposer;

		public Workspace(IPerforceSettings perforceSettings, ProjectInfo project, WorkspaceStateWrapper stateWrapper, ConfigFile projectConfigFile, IReadOnlyList<string>? projectStreamFilter, ILogger logger, IServiceProvider serviceProvider)
		{
			PerforceSettings = perforceSettings;
			Project = project;
			_stateWrapper = stateWrapper;
			_stateWrapper.OnModified += OnStateChangedInternal;
			State = stateWrapper.Current;

			Lock = new WorkspaceLock(project.LocalRootPath);
			Lock.OnChange += OnLockChangedInternal;

			SynchronizationContext = SynchronizationContext.Current!;
			_logger = logger;
			_asyncDisposer = serviceProvider.GetRequiredService<IAsyncDisposer>();

			ProjectConfigFile = projectConfigFile;
			ProjectStreamFilter = projectStreamFilter;
		}

		public bool IsExternalSyncActive() => Lock.IsLockedByOtherProcess();

		public void ModifyState(Action<WorkspaceState> action)
		{
			_stateWrapper.Modify(x =>
			{
				x.ResetForProject(Project);
				action(x);
			});
		}

		private void OnLockChangedInternal(bool locked)
		{
			SynchronizationContext.Post(OnLockChangedInternalMainThread, null);
		}

		private void OnLockChangedInternalMainThread(object? obj)
		{
			OnStateChanged?.Invoke(State);
		}

		private void OnStateChangedInternal(ReadOnlyWorkspaceState state)
		{
			SynchronizationContext.Post(x => OnStateChangedInternalMainThread(state), null);
		}

		private void OnStateChangedInternalMainThread(ReadOnlyWorkspaceState state)
		{
			State = state.ResetForProject(Project);
			OnStateChanged?.Invoke(state);
		}

		public void Dispose()
		{
			CancelUpdate();
			if (_prevUpdateTask != null)
			{
				_prevUpdateTask = _prevUpdateTask.ContinueWith(x => FinishDispose(), TaskScheduler.Default);
				_asyncDisposer.Add(_prevUpdateTask);
			}
			else
			{
				FinishDispose();
			}
		}

		void FinishDispose()
		{
			_stateWrapper.Dispose();
			Lock.Dispose();
		}

		public ConfigFile ProjectConfigFile
		{
			get; private set;
		}

		public IReadOnlyList<string>? ProjectStreamFilter
		{
			get; private set;
		}

#pragma warning disable CA2213 // warning CA2213: 'Workspace' contains field '_prevCancellationSource' that is of IDisposable type 'CancellationTokenSource?', but it is never disposed. Change the Dispose method on 'Workspace' to call Close or Dispose on this field.
		CancellationTokenSource? _prevCancellationSource;
#pragma warning restore CA2213
		Task _prevUpdateTask = Task.CompletedTask;

		public void Update(WorkspaceUpdateContext context)
		{
			CancelUpdate();

			Task prevUpdateTaskCopy = _prevUpdateTask;

			WorkspaceUpdate update = new WorkspaceUpdate(context);
			_currentUpdate = update;

			CancellationTokenSource cancellationSource = new CancellationTokenSource();
			_prevCancellationSource = cancellationSource;
			_prevUpdateTask = Task.Run(() => UpdateWorkspaceMini(update, prevUpdateTaskCopy, cancellationSource.Token));
		}

		public void CancelUpdate()
		{
			// Cancel the current task. We actually terminate the operation asynchronously, but we can signal the cancellation and 
			// send a cancelled event, then wait for the heavy lifting to finish in the new update task.
			if (_prevCancellationSource != null)
			{
				CancellationTokenSource prevCancellationSourceCopy = _prevCancellationSource;
				prevCancellationSourceCopy.Cancel();
				_prevUpdateTask = _prevUpdateTask.ContinueWith(task => prevCancellationSourceCopy.Dispose(), TaskScheduler.Default);
				_prevCancellationSource = null;
			}
			if (_currentUpdate != null)
			{
				CompleteUpdate(_currentUpdate, WorkspaceUpdateResult.Canceled, "Cancelled");
			}
		}

		async Task UpdateWorkspaceMini(WorkspaceUpdate update, Task prevUpdateTask, CancellationToken cancellationToken)
		{
			if (prevUpdateTask != null)
			{
				await prevUpdateTask;
			}

			WorkspaceUpdateContext context = update.Context;
			context.ProjectConfigFile = ProjectConfigFile;
			context.ProjectStreamFilter = ProjectStreamFilter;

			string statusMessage;
			WorkspaceUpdateResult result = WorkspaceUpdateResult.FailedToSync;

			if (!await Lock.TryAcquireAsync())
			{
				statusMessage = "Command line sync already in progress";
				_logger.LogError("Another process is already syncing this workspace.");
			}
			else
			{
				try
				{
					(result, statusMessage) = await update.ExecuteAsync(PerforceSettings, Project, _stateWrapper, _logger, cancellationToken);
					if (result != WorkspaceUpdateResult.Success)
					{
						_logger.LogError("{Message}", statusMessage);
					}
				}
				catch (OperationCanceledException)
				{
					statusMessage = "Canceled.";
					_logger.LogError("Canceled.");
				}
				catch (Exception ex)
				{
					statusMessage = "Failed with exception - " + ex.ToString();
					_logger.LogError(ex, "Failed with exception");
				}
				finally
				{
					await Lock.ReleaseAsync();
				}
			}

			ProjectConfigFile = context.ProjectConfigFile;
			ProjectStreamFilter = context.ProjectStreamFilter;

			SynchronizationContext.Post(x => CompleteUpdate(update, result, statusMessage), null);
		}

		void CompleteUpdate(WorkspaceUpdate update, WorkspaceUpdateResult result, string statusMessage)
		{
			if (_currentUpdate == update)
			{
				WorkspaceUpdateContext context = update.Context;

				ModifyState(x =>
				{
					if (result == WorkspaceUpdateResult.Canceled)
					{
						x.LastSyncChangeNumber = update.Context.ChangeNumber;
						x.LastSyncResult = WorkspaceUpdateResult.Canceled;
						x.LastSyncResultMessage = null;
						x.LastSyncTime = null;
						x.LastSyncDurationSeconds = 0;
						x.LastSyncEditorArchive = "0";
					}
					x.SetLastSyncState(result, context, statusMessage);
				});

				_currentUpdate = null;

				OnUpdateComplete?.Invoke(context, result, statusMessage);
			}
		}

		public Dictionary<string, string> GetVariables(BuildConfig editorConfig, int? overrideChange = null, int? overrideCodeChange = null)
		{
			FileReference editorReceiptFile = ConfigUtils.GetEditorReceiptFile(Project, ProjectConfigFile, editorConfig);

			TargetReceipt? editorReceipt;
			if (!ConfigUtils.TryReadEditorReceipt(Project, editorReceiptFile, out editorReceipt))
			{
				editorReceipt = ConfigUtils.CreateDefaultEditorReceipt(Project, ProjectConfigFile, editorConfig);
			}

			Dictionary<string, string> variables = ConfigUtils.GetWorkspaceVariables(Project, overrideChange ?? State.CurrentChangeNumber, overrideCodeChange ?? State.CurrentCodeChangeNumber, editorReceipt, ProjectConfigFile, PerforceSettings);
			return variables;
		}

		public bool IsBusy()
		{
			return Syncing;
		}

		public int CurrentChangeNumber => State.CurrentChangeNumber;

		public int PendingChangeNumber => _currentUpdate?.Context?.ChangeNumber ?? CurrentChangeNumber;

		public string ClientName => PerforceSettings.ClientName!;

		public Tuple<string, float> CurrentProgress => _currentUpdate?.CurrentProgress ?? new Tuple<string, float>("", 0.0f);
	}
}
