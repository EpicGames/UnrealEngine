// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Net;
using System.Net.Sockets;
using System.Text;
using Microsoft.Win32;

namespace UnrealGameSyncAutomation
{
	class Program
	{
		enum AutomationRequestType
		{
			SyncProject,
			FindProject,
			OpenProject
		}

		enum AutomationRequestResult
		{
			Ok,
			Invalid,
			Busy,
			Canceled,
			Error,
			NotFound
		}

		static void Main()
		{
			// Create the request data
			MemoryStream InputDataStream = new MemoryStream();
			using (BinaryWriter Writer = new BinaryWriter(InputDataStream))
			{
				Writer.Write("//UE4/Main");
				Writer.Write("/Samples/Games/ShooterGame/ShooterGame.uproject");
			}

			Tuple<AutomationRequestResult, byte[]> Response = SendRequest(AutomationRequestType.OpenProject, InputDataStream.ToArray());

			string ResponseString = Encoding.UTF8.GetString(Response.Item2);
			Console.WriteLine("{0}: {1}", Response.Item1, ResponseString);
		}

		static Tuple<AutomationRequestResult, byte[]> SendRequest(AutomationRequestType Request, byte[] RequestData)
		{
			object PortValue = Registry.GetValue("HKEY_CURRENT_USER\\Software\\Epic Games\\UnrealGameSync", "AutomationPort", null)
				?? throw new InvalidOperationException("AutomationPort registry value is not set.");
			if (!int.TryParse(PortValue.ToString(), out int PortNumber) || PortNumber < 1 || PortNumber > 65535)
			{
				throw new InvalidOperationException($"AutomationPort registry value '{PortValue}' is not a valid TCP port number (1-65535).");
			}
			using (TcpClient Client = new TcpClient())
			{
				Client.Connect(new IPEndPoint(IPAddress.Loopback, PortNumber));

				using (NetworkStream Stream = Client.GetStream())
				{
					// Send the request
					using BinaryWriter Writer = new BinaryWriter(Stream);
					Writer.Write((int)Request);
					Writer.Write(RequestData.Length);
					Writer.Write(RequestData);

					// Read the response
					using BinaryReader Reader = new BinaryReader(Stream);
					AutomationRequestResult Result = (AutomationRequestResult)Reader.ReadInt32();
					int ResponseLength = Reader.ReadInt32();
					byte[] ResponseData = Reader.ReadBytes(ResponseLength);

					// Return the response
					return Tuple.Create(Result, ResponseData);
				}
			}
		}
	}
}
