// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;

// General Information about an assembly is controlled through the following
// set of attributes. Change these attribute values to modify the information
// associated with an assembly.
[assembly: AssemblyTitle("UnrealGameSyncAutomation")]
[assembly: AssemblyDescription("")]
[assembly: AssemblyConfiguration("")]
[assembly: AssemblyCompany("")]
[assembly: AssemblyProduct("UnrealGameSyncAutomation")]
[assembly: AssemblyCopyright("Copyright Epic Games, Inc. All Rights Reserved.")]
[assembly: AssemblyTrademark("")]
[assembly: AssemblyCulture("")]
[assembly: SupportedOSPlatform("windows")]

// Setting ComVisible to false makes the types in this assembly not visible
// to COM components.  If you need to access a type in this assembly from
// COM, set the ComVisible attribute to true on that type.
[assembly: ComVisible(false)]

// The following GUID is for the ID of the typelib if this project is exposed to COM
[assembly: Guid("7c9d0a33-c2a8-4415-b9ed-9f40337dbec6")]

// Version information for an assembly consists of the following four values:
//
//      Major Version
//      Minor Version
//      Build Number
//      Revision
//
// You can specify all the values or you can default the Build and Revision Numbers
// by using the '*' as shown below:
// [assembly: AssemblyVersion("1.0.*")]
[assembly: AssemblyVersion("1.0.0.0")]
[assembly: AssemblyFileVersion("1.0.0.0")]
