<?xml version="1.0" encoding="utf-8"?>
<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <OutputType>WinExe</OutputType>
    <TargetFramework>net10.0-windows</TargetFramework>
    <UseWindowsForms>true</UseWindowsForms>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <RootNamespace>UnrealGameSyncAutomation</RootNamespace>
    <AssemblyName>UnrealGameSyncAutomation</AssemblyName>
  </PropertyGroup>
</Project>