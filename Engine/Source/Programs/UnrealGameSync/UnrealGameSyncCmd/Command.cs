// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;
using UnrealGameSync;

namespace UnrealGameSyncCmd
{
	internal interface ICommand
	{

	}

	internal abstract class Command : ICommand
	{
		internal static BuildConfig EditorConfig => BuildConfig.Development;

		public abstract Task ExecuteAsync(CommandContext context);
	}
}
