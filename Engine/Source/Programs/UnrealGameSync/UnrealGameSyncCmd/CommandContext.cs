// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;

using UnrealGameSync;
using UnrealGameSyncCmd.Utils;

namespace UnrealGameSyncCmd
{
	internal class CommandContext : IDisposable
	{
		public CommandLineArguments? Arguments { get; private set; }
		
		public IServiceProvider ServiceProvider { get; private set; }

		public ILogger? Logger { get; private set; }

		public ILoggerFactory? LoggerFactory { get; private set; }

		public UserSettings? GlobalSettings { get; private set; }

		public GlobalSettingsFile? UserSettings { get; private set; }

		public IHordeClient? HordeClient { get; private set; }

		public ICloudStorage? CloudStorage { get; private set; }

		public UserWorkspaceSettings? UserWorkspaceSettings { get; private set; }

		public UserProjectSettings? UserProjectSettings { get; private set; }

		public IPerforceConnection? PerforceClient { get; private set; }

		public WorkspaceStateWrapper? WorkspaceState { get; private set; }

		public ProjectInfo? ProjectInfo { get; private set; }

		public ConfigFile? ProjectConfig { get; private set; }

		private CommandContext(IServiceProvider serviceProvider)
		{
			ServiceProvider = serviceProvider;
		}

		public void Dispose()
		{
			PerforceClient?.Dispose();
		}

		public static async Task<CommandContext> CreateAsync(
			CommandLineArguments arguments,
			IServiceProvider serviceProvider,
			ILogger logger,
			ILoggerFactory loggerFactory,
			GlobalSettingsFile userSettings,
			UserSettings? globalSettings,
			IHordeClient? hordeClient,
			ICloudStorage? cloudStorage)
		{
			CommandContext context = new CommandContext(serviceProvider);
			context.Arguments = arguments;
			context.Logger = logger;
			context.LoggerFactory = loggerFactory;
			context.GlobalSettings = globalSettings;
			context.UserSettings = userSettings;
			context.HordeClient = hordeClient;
			context.CloudStorage = cloudStorage;
			context.UserWorkspaceSettings = UserSettingsUtils.ReadRequiredUserWorkspaceSettings();
			context.PerforceClient = await PerforceConnectionUtils.ConnectAsync(context.UserWorkspaceSettings!, context.LoggerFactory!);
			

			// get the local path for the project
			List<WhereRecord> records = await context.PerforceClient.WhereAsync($"//{context.UserWorkspaceSettings.ClientName}{context.UserWorkspaceSettings.ProjectPath}", CancellationToken.None).Where(x => !x.Unmap).ToListAsync();
			if(records.Count == 0)
			{
				throw new Exception($"Unable to find a local path for the project. Client: {context.UserWorkspaceSettings.ClientName}, ProjectPath: {context.UserWorkspaceSettings.ProjectPath}");
			}

			string localProjectPath = records[0].Path;

			context.UserWorkspaceSettings.LocalProjectPath = new FileReference(localProjectPath);

			context.WorkspaceState = await WorkspaceStateUtils.ReadWorkspaceState(context.PerforceClient, context.UserWorkspaceSettings, context.UserSettings!, context.Logger!);

			context.ProjectInfo = new ProjectInfo(
				context.UserWorkspaceSettings.RootDir,
				context.WorkspaceState.Current.ClientName,
				context.WorkspaceState.Current.BranchPath,
				context.WorkspaceState.Current.ProjectPath, 
				localProjectPath,
				context.WorkspaceState.Current.StreamName,
				context.WorkspaceState.Current.ProjectIdentifier,
				context.WorkspaceState.Current.IsEnterpriseProject);
			context.ProjectConfig = await ConfigUtils.ReadProjectConfigFileAsync(context.PerforceClient, context.ProjectInfo, context.Logger, CancellationToken.None);
			context.UserProjectSettings = context.UserSettings!.FindOrAddProjectSettings(context.ProjectInfo, context.UserWorkspaceSettings, context.Logger);

			if (String.IsNullOrWhiteSpace(context.UserProjectSettings.Version))
			{
				context.UserProjectSettings.Preset = context.UserWorkspaceSettings.Preset;
				context.UserProjectSettings.Save(logger);
			}
			
			bool forceDefaultPreset = false;

			if (ConfigUtils.GetForceDefaultPreset(context.ProjectConfig, context.ProjectInfo!.ProjectIdentifier, out bool configForceDefaultPreset))
			{
				forceDefaultPreset = configForceDefaultPreset;
			}

			string defaultPreset = String.Empty;
			if (ConfigUtils.GetDefaultPreset(context.ProjectConfig, context.ProjectInfo!.ProjectIdentifier, out string configDefaultPreset))
			{
				defaultPreset = configDefaultPreset;
			}

			// set the initial value to null as string.Empty has a different meaning in that context
			string? currentPreset = null;

			// if the default preset is set, and it must be use ensure that it is used if it is available
			if (forceDefaultPreset && !String.IsNullOrWhiteSpace(defaultPreset))
			{
				if (ConfigUtils.IsPresetAvailable(context.ProjectConfig, context.ProjectInfo!.ProjectIdentifier, defaultPreset))
				{
					currentPreset = defaultPreset;
				}
			}
			else
			{
				// if there is no current preset, and we have a default preset, and it is available: set the preset to tbe the default preset
				if (   !context.UserProjectSettings.PresetSetByUser
					&& !String.IsNullOrWhiteSpace(defaultPreset)
				    && ConfigUtils.IsPresetAvailable(context.ProjectConfig, context.ProjectInfo!.ProjectIdentifier, defaultPreset))
				{
					currentPreset = defaultPreset;
				}

				// if there is a current preset still verify that it is available
				else if (!ConfigUtils.IsPresetAvailable(context.ProjectConfig, context.ProjectInfo!.ProjectIdentifier, context.UserProjectSettings.Preset))
				{
					currentPreset = String.Empty;
				}
			}

			if (currentPreset != null)
			{
				context.UserWorkspaceSettings.Preset = currentPreset;
				context.UserProjectSettings.Preset = currentPreset;
				context.UserProjectSettings.Save(logger);
			}

			return context;
		}
	}
}
