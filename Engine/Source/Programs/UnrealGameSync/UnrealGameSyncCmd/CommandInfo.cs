// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealGameSyncCmd
{
	internal class CommandInfo
	{
		public string Name { get; }
		public Type Type { get; }
		public Type? OptionsType { get; }
		public string? Usage { get; }
		public string? Brief { get; }

		public CommandInfo(string name, Type type, Type? optionsType, string? usage, string? brief)
		{
			Name = name;
			Type = type;
			OptionsType = optionsType;
			Usage = usage;
			Brief = brief;
		}
	}
}
