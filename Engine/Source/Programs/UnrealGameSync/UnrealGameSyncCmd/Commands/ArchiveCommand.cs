// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;

using Microsoft.Extensions.Logging;

using EpicGames.Core;

using UnrealGameSync;

namespace UnrealGameSyncCmd.Commands
{
	internal class ArchiveCommandOptions
	{
		[CommandLine("-List")]
		public bool List { get; set; } = false;

		[CommandLine("-Changes")]
		public bool Changes { get; set; } = false;

		[CommandLine("-Set")]
		public bool Set { get; set; } = false;

		[CommandLine("-Unset")]
		public bool Unset { get; set; } = false;

		[CommandLine("-ArchiveType")]
		public string ArchiveType { get; set; } = String.Empty;

		[CommandLine("-ArchiveName")]
		public string ArchiveName { get; set; } = String.Empty;

		[CommandLine("-ArchiveSources", ListSeparator = ';')]
		public List<string> ArchiveSources { get; set; } = new List<string>();
	}

	class ArchiveCommand : Command
	{
		public override async Task ExecuteAsync(CommandContext context)
		{
			ILogger logger = context.Logger!;
			context.Arguments!.TryGetPositionalArgument(out string? _);

			ArchiveCommandOptions options = new ArchiveCommandOptions();
			context.Arguments.ApplyTo(options);
			context.Arguments.CheckAllArgumentsUsed();

			HashSet<string> archiveSources = BaseArchive.DefaultArchiveSources;
			if (options.ArchiveSources.Any())
			{
				archiveSources = options.ArchiveSources.ToHashSet(StringComparer.OrdinalIgnoreCase);
			}

			List<BaseArchiveChannel> archives = await BaseArchive.EnumerateChannelsAsync(
				context.PerforceClient!, 
				context.HordeClient!, 
				context.CloudStorage, 
				context.ProjectConfig!, 
				context.WorkspaceState!.Current.ProjectIdentifier,
				archiveSources,
				CancellationToken.None
				);
			List<IArchiveChannel> selected = GetSelectedArchiveChannels(context.GlobalSettings!, archives);

			if (options.List)
			{
				foreach (BaseArchiveChannel archive in archives)
				{
					if (!String.IsNullOrWhiteSpace(options.ArchiveType) && !options.ArchiveType.Equals(archive.Type, StringComparison.OrdinalIgnoreCase))
					{
						continue;
					}

					if (!String.IsNullOrWhiteSpace(options.ArchiveName) && !options.ArchiveName.Equals(archive.Name, StringComparison.OrdinalIgnoreCase))
					{
						continue;
					}

					DisplayArchive(logger, selected, archive);
				}
			}

			if (options.Changes)
			{
				foreach (BaseArchiveChannel archive in archives)
				{
					if (!String.IsNullOrWhiteSpace(options.ArchiveType) && !options.ArchiveType.Equals(archive.Type, StringComparison.OrdinalIgnoreCase))
					{
						continue;
					}

					if (!String.IsNullOrWhiteSpace(options.ArchiveName) && !options.ArchiveName.Equals(archive.Name, StringComparison.OrdinalIgnoreCase))
					{
						continue;
					}

					DisplayArchive(logger, selected, archive);
					DisplayChanges(logger, archive);
				}
			}

			if (options.Set)
			{
				if (String.IsNullOrWhiteSpace(options.ArchiveType))
				{
					logger.LogError("Parameter ArchiveType is required to set an archive.");
					return;
				}

				if (String.IsNullOrWhiteSpace(options.ArchiveName))
				{
					logger.LogError("Parameter ArchiveName is required to set an archive.");
					return;
				}

				List<BaseArchiveChannel> enabled = archives.Where(a => a.Type.Equals(options.ArchiveType, StringComparison.OrdinalIgnoreCase) && a.Name.Equals(options.ArchiveName, StringComparison.OrdinalIgnoreCase)).ToList();
				if(enabled.Any())
				{
					ResetArchiveSettings(logger, context.GlobalSettings!);
					foreach (BaseArchiveChannel archive in enabled)
					{
						ArchiveSettings? archiveSettings = context.GlobalSettings!.Archives.FirstOrDefault(x => x.Type == archive.Type);
						if (archiveSettings != null)
						{
							archiveSettings.Enabled = true;
							archiveSettings.Order.Remove(archive.Name);
							archiveSettings.Order.Insert(0, archive.Name);
						}
					}

					context.GlobalSettings!.Save(logger);
				}
			}
			else if (options.Unset)
			{
				ResetArchiveSettings(logger, context.GlobalSettings!);
			}
		}

		private static void ResetArchiveSettings(ILogger logger, UserSettings userSettings)
		{
			foreach (ArchiveSettings archiveSettings in userSettings.Archives)
			{
				archiveSettings.Enabled = false;
			}

			userSettings.Save(logger);
		}

		private static List<IArchiveChannel> GetSelectedArchiveChannels(UserSettings userSettings, IReadOnlyList<IArchiveChannel> archives)
		{
			Dictionary<string, KeyValuePair<IArchiveChannel, int>> archiveTypeToSelection = new Dictionary<string, KeyValuePair<IArchiveChannel, int>>();
			foreach (IArchiveChannel archive in archives)
			{
				ArchiveSettings? archiveSettings = userSettings.Archives.FirstOrDefault(x => x.Type == archive.Type);
				if (archiveSettings != null && archiveSettings.Enabled)
				{
					int preference = archiveSettings.Order.IndexOf(archive.Name);
					if (preference == -1)
					{
						preference = archiveSettings.Order.Count;
					}

					KeyValuePair<IArchiveChannel, int> existingItem;
					if (!archiveTypeToSelection.TryGetValue(archive.Type, out existingItem) || existingItem.Value > preference)
					{
						archiveTypeToSelection[archive.Type] = new KeyValuePair<IArchiveChannel, int>(archive, preference);
					}
				}
			}
			return archiveTypeToSelection.Select(x => x.Value.Key).ToList();
		}

		private static void DisplayArchive(ILogger logger, List<IArchiveChannel> selected, BaseArchiveChannel archive)
		{

			string annotations = String.Empty;
			switch (archive)
			{
				case PerforceArchiveChannel perforceArchiveChannel:
					annotations = $"perforce:{perforceArchiveChannel.DepotPath}";
					break;
				case HordeArchiveChannel _:
					annotations = $"horde";
					break;
				case CloudArchiveChannel _:
					annotations = $"cloud";
					break;
				default:
					break;
			}

			logger.LogInformation("...channel {Selected} | {Type} | {Name} | {Annotations}", selected.Contains(archive), archive.Type, archive.Name, annotations);
		}

		private static void DisplayChanges(ILogger logger, BaseArchiveChannel archive)
		{
			foreach (KeyValuePair<int, IArchive> change in archive.ChangeNumberToArchive)
			{
				logger.LogInformation("...change {ChangeNumber} {ArchiveKey}", change.Key, change.Value.Key);
			}
		}
	}
}
