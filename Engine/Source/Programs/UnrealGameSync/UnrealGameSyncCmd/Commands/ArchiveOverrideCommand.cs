// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealGameSync;

namespace UnrealGameSyncCmd.Commands
{
	internal class ArchiveOverrideCommandOptions
	{
		[CommandLine("-List")]
		public bool List { get; set; } = false;

		[CommandLine("-Set")]
		public bool Set { get; set; } = false;

		[CommandLine("-Unset")]
		public bool Unset { get; set; } = false;

		[CommandLine("-SetCompileLocally")]
		public bool SetCompileLocally { get; set; } = false;

		[CommandLine("-ArchiveType")]
		public string ArchiveType { get; set; } = String.Empty;

		[CommandLine("-ArchiveName")]
		public string ArchiveName { get; set; } = String.Empty;

		[CommandLine("-ArchiveSources", ListSeparator = ';')]
		public List<string> ArchiveSources { get; set; } = new List<string>();
	}

	internal class ArchiveOverrideCommand : Command
	{
		public override async Task ExecuteAsync(CommandContext context)
		{
			ILogger logger = context.Logger!;
			context.Arguments!.TryGetPositionalArgument(out string? _);

			ArchiveOverrideCommandOptions options = new ArchiveOverrideCommandOptions();
			context.Arguments.ApplyTo(options);
			context.Arguments.CheckAllArgumentsUsed();

			HashSet<string> archiveSources = BaseArchive.DefaultArchiveSources;
			if (options.ArchiveSources.Any())
			{
				archiveSources = options.ArchiveSources.ToHashSet(StringComparer.OrdinalIgnoreCase);
			}

			List<BaseArchiveChannel> archives = await BaseArchive.EnumerateChannelsAsync(
				context.PerforceClient!,
				context.HordeClient!,
				context.CloudStorage,
				context.ProjectConfig!,
				context.WorkspaceState!.Current.ProjectIdentifier,
				archiveSources,
				CancellationToken.None);
			List<IArchiveChannel> selected = GetSelectedArchiveChannels(context.UserProjectSettings!, archives);

			if (options.List)
			{
				foreach (BaseArchiveChannel archive in archives)
				{
					if (!String.IsNullOrWhiteSpace(options.ArchiveType) && !options.ArchiveType.Equals(archive.Type, StringComparison.OrdinalIgnoreCase))
					{
						continue;
					}

					if (!String.IsNullOrWhiteSpace(options.ArchiveName) && !options.ArchiveName.Equals(archive.Name, StringComparison.OrdinalIgnoreCase))
					{
						continue;
					}

					DisplayArchive(logger, selected, archive);
				}
			}

			if (options.Set)
			{
				if (String.IsNullOrWhiteSpace(options.ArchiveType))
				{
					logger.LogError("Parameter ArchiveType is required to set an archive.");
					return;
				}

				if (String.IsNullOrWhiteSpace(options.ArchiveName))
				{
					logger.LogError("Parameter ArchiveName is required to set an archive.");
					return;
				}

				List<BaseArchiveChannel> matchingArchives = archives.Where(a => a.Type.Equals(options.ArchiveType, StringComparison.OrdinalIgnoreCase) && a.Name.Equals(options.ArchiveName, StringComparison.OrdinalIgnoreCase)).ToList();
				if (matchingArchives.Any())
				{
					ResetArchiveSettings(logger, context.UserProjectSettings!);
					foreach (BaseArchiveChannel archive in matchingArchives)
					{
						ArchiveSettings? archiveSettings = context.UserProjectSettings!.Archives.FirstOrDefault(x => x.Type == archive.Type);
						if (archiveSettings == null)
						{
							archiveSettings = new ArchiveSettings(true, archive.Type, new string[] { archive.Name });
							context.UserProjectSettings.Archives.Add(archiveSettings);
						}
						else
						{
							archiveSettings.Enabled = true;
							archiveSettings.Order.Remove(archive.Name);
							archiveSettings.Order.Insert(0, archive.Name);
						}
					}

					context.UserProjectSettings!.Save(logger);
				}
			}
			else if(options.SetCompileLocally)
			{
				ResetArchiveSettings(logger, context.UserProjectSettings!);
				context.UserProjectSettings!.ArchivesDeactivated = true;
				context.UserProjectSettings!.Save(logger);
				logger.LogInformation("Archive override set to compile locally.");
			}
			else if (options.Unset)
			{
				ResetArchiveSettings(logger, context.UserProjectSettings!);
			}
		}

		private static void ResetArchiveSettings(ILogger logger, UserProjectSettings userSettings)
		{
			foreach (ArchiveSettings archiveSettings in userSettings.Archives)
			{
				archiveSettings.Enabled = false;
			}

			userSettings.ArchivesDeactivated = false;
			userSettings.Save(logger);
		}

		private static List<IArchiveChannel> GetSelectedArchiveChannels(UserProjectSettings userSettings, IReadOnlyList<IArchiveChannel> archives)
		{
			Dictionary<string, KeyValuePair<IArchiveChannel, int>> archiveTypeToSelection = new Dictionary<string, KeyValuePair<IArchiveChannel, int>>();
			foreach (IArchiveChannel archive in archives)
			{
				ArchiveSettings? archiveSettings = userSettings.Archives.FirstOrDefault(x => x.Type == archive.Type);
				if (archiveSettings != null && archiveSettings.Enabled)
				{
					int preference = archiveSettings.Order.IndexOf(archive.Name);
					if (preference == -1)
					{
						preference = archiveSettings.Order.Count;
					}

					KeyValuePair<IArchiveChannel, int> existingItem;
					if (!archiveTypeToSelection.TryGetValue(archive.Type, out existingItem) || existingItem.Value > preference)
					{
						archiveTypeToSelection[archive.Type] = new KeyValuePair<IArchiveChannel, int>(archive, preference);
					}
				}
			}
			return archiveTypeToSelection.Select(x => x.Value.Key).ToList();
		}

		private static void DisplayArchive(ILogger logger, List<IArchiveChannel> selected, BaseArchiveChannel archive)
		{

			string annotations = String.Empty;
			switch (archive)
			{
				case PerforceArchiveChannel perforceArchiveChannel:
					annotations = $"perforce:{perforceArchiveChannel.DepotPath}";
					break;
				case HordeArchiveChannel _:
					annotations = $"horde";
					break;
				case CloudArchiveChannel _:
					annotations = $"cloud";
					break;
				default:
					break;
			}

			logger.LogInformation("...channel {Selected} | {Type} | {Name} | {Annotations}", selected.Contains(archive), archive.Type, archive.Name, annotations);
		}
	}
}
