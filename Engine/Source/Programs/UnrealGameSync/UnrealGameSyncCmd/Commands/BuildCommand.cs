// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.OIDC;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using UnrealGameSync;

using UserErrorException = UnrealGameSyncCmd.Exceptions.UserErrorException;

namespace UnrealGameSyncCmd.Commands
{
	internal class BuildCommandOptions
	{
		[CommandLine("-List")]
		public bool ListOnly { get; set; }
	}

	internal class BuildCommand : Command
	{
		public override async Task ExecuteAsync(CommandContext context)
		{
			ILogger logger = context.Logger!;
			IPerforceConnection perforceClient = context.PerforceClient!;
			ProjectInfo projectInfo = context.ProjectInfo!;
			ConfigFile projectConfig = context.ProjectConfig!;
			UserProjectSettings projectSettings = context.UserProjectSettings!;
			WorkspaceStateWrapper state = context.WorkspaceState!;
			IServiceProvider serviceProvider = context.ServiceProvider;

			context.Arguments!.TryGetPositionalArgument(out string? target);

			BuildCommandOptions options = new BuildCommandOptions();
			context.Arguments.ApplyTo(options);
			context.Arguments.CheckAllArgumentsUsed();

			FileReference editorTarget = ConfigUtils.GetEditorTargetFile(projectInfo, projectConfig);

			Dictionary<Guid, ConfigObject> buildStepObjects = ConfigUtils.GetDefaultBuildStepObjects(
				projectInfo, 
				editorTarget.GetFileNameWithoutAnyExtensions(), 
				EditorConfig, 
				projectConfig, 
				false);

			if (context.GlobalSettings != null)
			{
				BuildStep.MergeBuildStepObjects(buildStepObjects, projectConfig.GetValues("Build.Step", [], projectInfo.ProjectIdentifier).Select(x => new ConfigObject(x)));
				BuildStep.MergeBuildStepObjects(buildStepObjects, projectSettings.BuildSteps);
			}

			if (options.ListOnly)
			{
				List<BuildStep> buildSteps = buildStepObjects.Values.Select(x => new BuildStep(x)).OrderBy(x => x.NormalSync).ToList();

				int longestDescription = buildSteps.Max(b => String.IsNullOrWhiteSpace(b.Description) ? 0 : b.Description.Length);
				string descriptionHeader = "Description";
				string descriptionSpace = new string(Enumerable.Repeat(' ', longestDescription + 1 - descriptionHeader.Length).ToArray());
				string descriptionDashes = new string(Enumerable.Repeat('-', longestDescription + 2).ToArray());
				string format = $"  {{Id,-36}} | {{Name,-{longestDescription}}} | {{Type,-10}} | {{Enabled,-8}}";

				logger.LogInformation("Available build steps:");
				logger.LogInformation("");
				logger.LogInformation("  Id                                   | {DescriptionHeader}{DescriptionSpace}| Type       | Enabled", descriptionHeader, descriptionSpace);
				logger.LogInformation("  -------------------------------------|{DescriptionDashes}|------------|-----------------", descriptionDashes);
				foreach (BuildStep buildStep in buildStepObjects.Values.Select(x => new BuildStep(x)).OrderBy(x => x.NormalSync))
				{
					#pragma warning disable CA2254 // Template should be a static expression
					logger.LogInformation(format, buildStep.UniqueId, buildStep.Description, buildStep.Type, buildStep.NormalSync);
					#pragma warning restore CA2254
				}
				return;
			}

			HashSet<Guid>? steps = new HashSet<Guid>();
			if (target != null)
			{
				if (!Guid.TryParse(target, out Guid id))
				{
					logger.LogError("Unable to parse '{Target}' as a GUID. Pass -List to show all available build steps and their identifiers.", target);
					return;
				}

				steps.Add(id);
			}

			// check that the tools are installed
			List<Guid> uninstalledTools = new List<Guid>();
			if (context.GlobalSettings != null)
			{
				using ToolUpdateMonitor? toolsMonitor = await GetToolsMonitor(logger, context, perforceClient);

				if (toolsMonitor != null)
				{
					foreach (Guid step in steps)
					{
						if (buildStepObjects.TryGetValue(step, out ConfigObject? config))
						{
							BuildStep buildStep = new BuildStep(config);
							if (CanRunStep(buildStep, toolsMonitor))
							{
								continue;
							}

							logger.LogWarning("Build step '{Description}' cannot be run as the tool is not installed.", buildStep.Description);
							uninstalledTools.Add(step);
						}
					}
				}
			}

			foreach (Guid uninstalledTool in uninstalledTools)
			{
				steps.Remove(uninstalledTool);
			}

			WorkspaceUpdateContext updateContext = new WorkspaceUpdateContext(
				context.ServiceProvider,
				context.GlobalSettings,
				context.UserWorkspaceSettings,
				context.UserProjectSettings,
				state.Current.CurrentChangeNumber,
				WorkspaceUpdateOptions.Build,
				BuildConfig.Development,
				null,
				projectSettings.BuildSteps, steps);

			WorkspaceUpdate update = new WorkspaceUpdate(updateContext);
			(WorkspaceUpdateResult result, string message) = await update.ExecuteAsync(perforceClient.Settings, projectInfo, state, logger, CancellationToken.None);

			if (result != WorkspaceUpdateResult.Success)
			{
				throw new UserErrorException("{Message}", message);
			}
		}

		private static async Task<ToolUpdateMonitor?> GetToolsMonitor(ILogger logger, CommandContext context, IPerforceConnection perforceClient)
		{
			ToolsCommandCommandOptions options = context.Arguments!.ApplyTo<ToolsCommandCommandOptions>(logger);
			context.Arguments.CheckAllArgumentsUsed();

			DirectoryReference dataFolder = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData)!, "UnrealGameSync");
			DirectoryReference.CreateDirectory(dataFolder);

			// create a temporary service provider for the tool update monitor
			ServiceCollection services = new ServiceCollection();
			services.AddSingleton<IAsyncDisposer, AsyncDisposer>();
			services.AddSingleton(sp => TokenStoreFactory.CreateTokenStore());
			services.AddSingleton<OidcTokenManager>();

			LauncherSettings launcherSettings = new LauncherSettings();
			launcherSettings.Read();

			if (launcherSettings.HordeServer != null)
			{
				services.AddHorde(options =>
				{
					options.ServerUrl = new Uri(launcherSettings.HordeServer);
					options.AllowAuthPrompt = false;
				});
			}

			ServiceProvider serviceProvider = services.BuildServiceProvider();

			ToolUpdateMonitor toolUpdateMonitor =
				new ToolUpdateMonitor(perforceClient.Settings, dataFolder, context.GlobalSettings!, logger, serviceProvider);

			// get the list of tools available
			logger.LogInformation("Retrieving tools information, please wait.");
			await toolUpdateMonitor.GetDataFromBackendAsync();

			return toolUpdateMonitor;
		}

		private static bool CanRunStep(BuildStep step, ToolUpdateMonitor toolsMonitor)
		{
			if (step.ToolId != Guid.Empty)
			{
				string? toolName = toolsMonitor.GetToolName(step.ToolId);
				if (toolName == null)
				{
					return false;
				}

				if (toolsMonitor.GetToolPath(toolName) == null)
				{
					return false;
				}
			}
			return true;
		}
	}
}
