// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;
using UnrealGameSync;

namespace UnrealGameSyncCmd.Commands
{
	internal class ChangesCommandOptions
	{
		[CommandLine("-Count=")]
		public int Count { get; set; } = 10;

		[CommandLine("-Lines=")]
		public int LineCount { get; set; } = 3;
	}

	internal class ChangesCommand : Command
	{
		public override async Task ExecuteAsync(CommandContext context)
		{
			ILogger logger = context.Logger!;
			IPerforceConnection perforceClient = context.PerforceClient!;
			UserWorkspaceSettings settings = context.UserWorkspaceSettings!;
			ChangesCommandOptions options = new ChangesCommandOptions();
			context.Arguments!.ApplyTo(options);
			context.Arguments!.CheckAllArgumentsUsed(logger);

			List<ChangesRecord> changes = await perforceClient.GetChangesAsync(EpicGames.Perforce.ChangesOptions.None, options.Count, ChangeStatus.Submitted, $"//{settings.ClientName}/...");
			foreach (IEnumerable<ChangesRecord> changesBatch in changes.Batch(10))
			{
				List<DescribeRecord> describeRecords = await perforceClient.DescribeAsync(changesBatch.Select(x => x.Number).ToArray());

				logger.LogInformation("  Change    Type     Author          Description");
				foreach (DescribeRecord describeRecord in describeRecords)
				{
					PerforceChangeDetails details = new PerforceChangeDetails(describeRecord);

					string type;
					if (details.ContainsCode)
					{
						if (details.ContainsContent)
						{
							type = "Both";
						}
						else
						{
							type = "Code";
						}
					}
					else
					{
						if (details.ContainsContent)
						{
							type = "Content";
						}
						else
						{
							type = "None";
						}
					}

					string author = StringUtils.Truncate(describeRecord.User, 15);

					List<string> lines = StringUtils.WordWrap(details.Description, Math.Max(ConsoleUtils.WindowWidth - 40, 10)).ToList();
					if (lines.Count == 0)
					{
						lines.Add(String.Empty);
					}

					int lineCount = Math.Min(options.LineCount, lines.Count);

					logger.LogInformation("  {Change,-9} {Type,-8} {Author,-15} {Description}", describeRecord.Number, type, author, lines[0]);
					for (int lineIndex = 1; lineIndex < lineCount; lineIndex++)
					{
						logger.LogInformation("                                     {Description}", lines[lineIndex]);
					}
				}
			}
		}
	}
}
