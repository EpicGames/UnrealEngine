// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;
using UnrealGameSyncCmd.Options;
using UnrealGameSyncCmd.Utils;

namespace UnrealGameSyncCmd.Commands
{
	internal class ClientsCommandOptions : ServerOptions
	{
	}

	internal class ClientsCommand : ICommand
	{
		public static async Task ExecuteAsync(ILogger logger, CommandLineArguments args, ILoggerFactory loggerFactory)
		{
			ClientsCommandOptions options = args.ApplyTo<ClientsCommandOptions>(logger);
			args.CheckAllArgumentsUsed();

			using IPerforceConnection perforceClient = await PerforceConnectionUtils.ConnectAsync(options.ServerAndPort, options.UserName, null, loggerFactory);

			InfoRecord info = await perforceClient.GetInfoAsync(InfoOptions.ShortOutput);

			List<ClientsRecord> clients = await perforceClient.GetClientsAsync(EpicGames.Perforce.ClientsOptions.None, perforceClient.Settings.UserName);
			foreach (ClientsRecord client in clients)
			{
				if (String.Equals(info.ClientHost, client.Host, StringComparison.OrdinalIgnoreCase))
				{
					logger.LogInformation("{Client,-50} {Root}", client.Name, client.Root);
				}
			}
		}
	}
}
