// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Linq;
using System.Threading.Tasks;

using Microsoft.Extensions.Logging;

using UnrealGameSync;
using UnrealGameSyncCmd.Options;

namespace UnrealGameSyncCmd.Commands
{
	internal class ConfigCommand : Command
	{
		public override async Task ExecuteAsync(CommandContext context)
		{
			ILogger logger = context.Logger!;
			UserWorkspaceSettings settings = context.UserWorkspaceSettings!;

			if (!context.Arguments!.GetUnusedArguments().Any())
			{
				ProcessStartInfo startInfo = new ProcessStartInfo();
				startInfo.FileName = settings.ConfigFile.FullName;
				startInfo.UseShellExecute = true;
				using (Process? editor = Process.Start(startInfo))
				{
					if (editor != null)
					{
						await editor.WaitForExitAsync();
					}
				}
			}
			else
			{
				ConfigCommandOptions options = new ConfigCommandOptions();
				context.Arguments.ApplyTo(options);
				context.Arguments.CheckAllArgumentsUsed(logger);

				options.ApplyTo(settings);
				settings.Save(logger);

				logger.LogInformation("Updated {ConfigFile}", settings.ConfigFile);
			}
		}
	}
}
