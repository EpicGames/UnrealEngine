// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;
using UnrealGameSyncCmd.Utils;

namespace UnrealGameSyncCmd.Commands
{
	internal class InstallCommand : Command
	{
		public override async Task ExecuteAsync(CommandContext context)
		{
			await InstallUtils.UpdateInstallAsync(true, context.Logger!);
		}
	}
}
