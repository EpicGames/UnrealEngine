// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Threading.Tasks;

using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.Logging;

using EpicGames.Core;
using EpicGames.OIDC;
using UnrealGameSync;

using UserErrorException = UnrealGameSyncCmd.Exceptions.UserErrorException;

namespace UnrealGameSyncCmd.Commands
{
	internal class LoginCommand : Command
	{
		public override async Task ExecuteAsync(CommandContext context)
		{
			ILogger logger = context.Logger!;

			// Get the positional argument indicating the file to look for
			if (!context.Arguments!.TryGetPositionalArgument(out string? providerIdentifier))
			{
				throw new UserErrorException("Missing provider identifier to login to.");
			}
			context.Arguments.CheckAllArgumentsUsed();

			UserWorkspaceSettings settings = context.UserWorkspaceSettings!;

			// Find the valid config file paths
			DirectoryInfo engineDir = DirectoryReference.Combine(settings.RootDir, "Engine").ToDirectoryInfo();
			DirectoryInfo gameDir = new DirectoryInfo(settings.ProjectPath);
			using ITokenStore tokenStore = TokenStoreFactory.CreateTokenStore();
			IConfiguration providerConfiguration = ProviderConfigurationFactory.ReadConfiguration(engineDir, gameDir);
			OidcTokenManager oidcTokenManager = OidcTokenManager.CreateTokenManager(providerConfiguration, tokenStore, new List<string>() { providerIdentifier });
			IOidcTokenInfo result = await oidcTokenManager.LoginAsync(providerIdentifier);

			logger.LogInformation("Logged in to provider {ProviderIdentifier}", providerIdentifier);
		}
	}
}
