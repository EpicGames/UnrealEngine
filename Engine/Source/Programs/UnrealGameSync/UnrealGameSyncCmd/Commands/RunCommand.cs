// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealGameSync;
using UnrealGameSyncCmd.Utils;

using UserErrorException = UnrealGameSyncCmd.Exceptions.UserErrorException;

namespace UnrealGameSyncCmd.Commands
{
	internal class RunCommandOptions
	{
		[CommandLine("-UseEditorArgsFromSettings")]
		public bool UseEditorArgsFromSettings { get; set; } = false;
	}

	internal class RunCommand : Command
	{
		public override async Task ExecuteAsync(CommandContext context)
		{
			await Task.Run(() => ExecuteInternal(context));
		}

		private static void ExecuteInternal(CommandContext context)
		{
			ILogger logger = context.Logger!;
			UserWorkspaceSettings settings = context.UserWorkspaceSettings!;
			ProjectInfo projectInfo = context.ProjectInfo!;
			ConfigFile projectConfig = context.ProjectConfig!;

			FileReference receiptFile = ConfigUtils.GetEditorReceiptFile(projectInfo, projectConfig, EditorConfig);
			logger.LogDebug("Receipt file: {Receipt}", receiptFile);

			if (!ConfigUtils.TryReadEditorReceipt(projectInfo, receiptFile, out TargetReceipt? receipt) || String.IsNullOrEmpty(receipt.Launch))
			{
				throw new UserErrorException("The editor needs to be built before you can run it. (Missing {ReceiptFile}).", receiptFile);
			}
			if (!File.Exists(receipt.Launch))
			{
				throw new UserErrorException("The editor needs to be built before you can run it. (Missing {LaunchFile}).", receipt.Launch);
			}

			List<string> launchArguments = new List<string>();

			if (settings.LocalProjectPath != null && settings.LocalProjectPath!.HasExtension(ProjectUtils.UProjectExtension))
			{
				launchArguments.Add($"\"{settings.LocalProjectPath}\"");
			}
			
			if (EditorConfig == BuildConfig.Debug || EditorConfig == BuildConfig.DebugGame)
			{
				launchArguments.Add("-debug");
			}

			RunCommandOptions options = context.Arguments!.ApplyTo<RunCommandOptions>(logger);
			for (int idx = 0; idx < context.Arguments!.Count; idx++)
			{
				if (!context.Arguments.HasBeenUsed(idx))
				{
					launchArguments.Add(context.Arguments[idx]);
				}
			}

			if (options.UseEditorArgsFromSettings)
			{
				foreach (LockableEditorArgument editorArgument in GetEditorArgumentsWithDefaults(context).Where(e => e.Enabled))
				{
					launchArguments.Add(editorArgument.Name);
				}
			}
			
			string commandLine = CommandLineArguments.Join(launchArguments);
			logger.LogInformation("Spawning: {LaunchFile} {CommandLine}", CommandLineArguments.Quote(receipt.Launch), commandLine);

			if (!Utility.SpawnProcess(receipt.Launch, commandLine))
			{
				logger.LogError("Unable to spawn {App} {Args}", receipt.Launch, launchArguments.ToString());
			}
		}

		private static List<LockableEditorArgument> GetDefaultEditorArguments(CommandContext context)
		{
			if(context.GlobalSettings == null || context.ProjectInfo == null)
			{
				return new List<LockableEditorArgument>();
			}

			List<string> defaultEditorArgumentDefinitions = new List<string>();
			ConfigUtils.GetProjectSettings(context.ProjectConfig!, context.ProjectInfo!.ProjectIdentifier, "DefaultEditorArgument", defaultEditorArgumentDefinitions);

			List<LockableEditorArgument> defaultEditorArguments = new List<LockableEditorArgument>();
			foreach (string editorArgumentDefinition in defaultEditorArgumentDefinitions.Distinct())
			{
				if (LockableEditorArgument.TryParseConfigEntry(editorArgumentDefinition, out LockableEditorArgument? editorArgument))
				{
					defaultEditorArguments.Add(editorArgument);
				}
			}

			return defaultEditorArguments;
		}

		private static List<LockableEditorArgument> GetEditorArgumentsWithDefaults(CommandContext context)
		{
			if (context.GlobalSettings == null)
			{
				return new List<LockableEditorArgument>();
			}

			List<LockableEditorArgument> defaultEditorArguments = GetDefaultEditorArguments(context);
			List<LockableEditorArgument> currentEditorArguments = context.GlobalSettings!.EditorArguments.ToList();

			foreach (LockableEditorArgument defaultEditorArgument in defaultEditorArguments)
			{
				// Check to see if the user already has this default argument in their list
				int index = currentEditorArguments.FindIndex(x => x.Name == defaultEditorArgument.Name);

				if (index != -1)
				{
					if (defaultEditorArgument.Locked)
					{
						currentEditorArguments[index] = new LockableEditorArgument(defaultEditorArgument.Name, defaultEditorArgument.Enabled, /* Locked = */ true);
					}
				}
				else
				{
					currentEditorArguments.Add(new LockableEditorArgument(defaultEditorArgument));
				}
			}

			return currentEditorArguments;
		}
	}
}
