// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;
using UnrealGameSync;
using UnrealGameSyncCmd.Utils;

namespace UnrealGameSyncCmd.Commands
{
	internal class SettingsCommandOptions
	{
		[CommandLine("-Global=")]
		public string Global { get; set; } = String.Empty;

		[CommandLine("-User=")]
		public string User { get; set; } = String.Empty;
	}

	internal class SettingsCommand : ICommand
	{
		public static async Task ExecuteAsync(ILogger logger, ILoggerFactory loggerFactory, CommandLineArguments args, UserSettings? globalSettings, GlobalSettingsFile? userSettings)
		{
			SettingsCommandOptions options = args.ApplyTo<SettingsCommandOptions>(logger);
			args.CheckAllArgumentsUsed();

			if (!String.IsNullOrWhiteSpace(options.Global))
			{
				if (globalSettings != null)
				{
					logger.LogInformation("Writing global settings to {Global}", options.Global);
					await WriteSettings(options.Global, globalSettings);
				}
				else if (userSettings != null)
				{
					logger.LogInformation("Writing global settings to {Global}", options.Global);
					await WriteSettings(options.Global, userSettings);
				}
			}

			if (!String.IsNullOrWhiteSpace(options.User))
			{
				logger.LogInformation("Writing user settings to {User}", options.User);
				UserWorkspaceSettings userWorkspaceSettings = UserSettingsUtils.ReadRequiredUserWorkspaceSettings();

				IPerforceConnection perforce = await PerforceConnectionUtils.ConnectAsync(userWorkspaceSettings, loggerFactory);

				// get the local path for the project
				List<WhereRecord> records = await perforce.WhereAsync($"//{userWorkspaceSettings.ClientName}{userWorkspaceSettings.ProjectPath}", CancellationToken.None).Where(x => !x.Unmap).ToListAsync();
				if (records.Count == 0)
				{
					throw new Exception($"Unable to find a local path for the project. Client: {userWorkspaceSettings.ClientName}, ProjectPath: {userWorkspaceSettings.ProjectPath}");
				}

				userWorkspaceSettings.LocalProjectPath = new FileReference(records[0].Path);

				await WriteSettings(options.User, userWorkspaceSettings);
			}
		}

		private static async Task WriteSettings(string path, object? settings)
		{
			if (String.IsNullOrWhiteSpace(path) || settings == null)
			{
				return;
			}

			string trimmed = path.Trim('\"')
				.Replace(Path.AltDirectorySeparatorChar, Path.DirectorySeparatorChar);
			if (!Path.IsPathFullyQualified(trimmed))
			{
				return;
			}

			string? directory = Path.GetDirectoryName(trimmed);
			if (!String.IsNullOrWhiteSpace(directory))
			{
				Directory.CreateDirectory(directory);
			}

			JsonSerializerOptions serializerOptions = new JsonSerializerOptions()
			{
				WriteIndented = true
			};
			string json = JsonSerializer.Serialize(settings, serializerOptions);

			await File.WriteAllTextAsync(trimmed, json);
		}
	}
}
