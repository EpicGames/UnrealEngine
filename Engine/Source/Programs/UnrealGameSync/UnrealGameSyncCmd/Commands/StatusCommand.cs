// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading;
using System.Threading.Tasks;

using Microsoft.Extensions.Logging;

using EpicGames.Perforce;

using UnrealGameSync;
using Microsoft.Extensions.Logging.Abstractions;

namespace UnrealGameSyncCmd.Commands
{
	internal class StatusCommand : Command
	{
		public override async Task ExecuteAsync(CommandContext context)
		{
			ILogger logger = context.Logger ?? NullLogger.Instance;
			if (context.PerforceClient == null || context.WorkspaceState == null || context.Arguments == null || context.UserWorkspaceSettings == null)
			{
				logger.LogWarning("Unable to determine status due to invalid CommandContext");
				return;
			}
			IPerforceConnection perforceClient = context.PerforceClient;
			WorkspaceStateWrapper state = context.WorkspaceState;

			bool update = context.Arguments.HasOption("-Update");
			context.Arguments.CheckAllArgumentsUsed();

			UserWorkspaceSettings settings = context.UserWorkspaceSettings;
			logger.LogInformation("User: {UserName}", settings.UserName);
			logger.LogInformation("Server: {ServerAndPort}", settings.ServerAndPort);
			logger.LogInformation("Project: {ClientProjectPath}", settings.ClientProjectPath);

			if (update)
			{
				ProjectInfo newProjectInfo = await ProjectInfo.CreateAsync(perforceClient, settings, CancellationToken.None);
				state.Modify(x => x.UpdateCachedProjectInfo(newProjectInfo, settings.LastModifiedTimeUtc));
			}

			string streamOrBranchName = state.Current.StreamName ?? settings.BranchPath.TrimStart('/');
			if (state.Current.LastSyncResultMessage == null)
			{
				logger.LogInformation("Not currently synced to {Stream}", streamOrBranchName);
			}
			else if (state.Current.LastSyncResult == WorkspaceUpdateResult.Success)
			{
				logger.LogInformation("Synced to {Stream} CL {Change}", streamOrBranchName, state.Current.LastSyncChangeNumber);
			}
			else
			{
				logger.LogWarning("Last sync to {Stream} CL {Change} failed: {Result}", streamOrBranchName, state.Current.LastSyncChangeNumber, state.Current.LastSyncResultMessage);
			}
		}
	}
}
