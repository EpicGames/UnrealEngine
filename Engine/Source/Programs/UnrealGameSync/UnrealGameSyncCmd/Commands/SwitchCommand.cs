// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.Threading.Tasks;

using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;
using UnrealGameSync;
using UnrealGameSyncCmd.Utils;

using UserErrorException = UnrealGameSyncCmd.Exceptions.UserErrorException;

namespace UnrealGameSyncCmd.Commands
{
	internal class SwitchCommandOptions
	{
		[CommandLine("-Force")]
		public bool Force { get; set; }
	}

	internal class SwitchCommand : Command
	{
		public override async Task ExecuteAsync(CommandContext context)
		{
			// Get the positional argument indicating the file to look for
			string? targetName;
			if (!context.Arguments!.TryGetPositionalArgument(out targetName))
			{
				throw new UserErrorException("Missing stream or project name to switch to.");
			}

			// Finish argument parsing
			SwitchCommandOptions options = new SwitchCommandOptions();
			context.Arguments.ApplyTo(options);
			context.Arguments.CheckAllArgumentsUsed();

			if (targetName.StartsWith("//", StringComparison.Ordinal))
			{
				options.Force = true;
			}

			// Check whether we're switching stream or project
			if (targetName.StartsWith("//", StringComparison.Ordinal))
			{
				await SwitchStreamAsync(context.PerforceClient!, targetName, options.Force, context.Logger!);
			}
			else
			{
				await SwitchProjectAsync(context.PerforceClient!, context.UserWorkspaceSettings!, targetName, context.Logger!);
			}
		}

		private static async Task SwitchStreamAsync(IPerforceConnection perforceClient, string streamName, bool force, ILogger logger)
		{
			if (!force && await perforceClient.OpenedAsync(OpenedOptions.None, FileSpecList.Any).AnyAsync())
			{
				throw new UserErrorException("Client {ClientName} has files opened. Use -Force to switch anyway.", perforceClient.Settings.ClientName!);
			}

			await perforceClient.SwitchClientToStreamAsync(streamName, SwitchClientOptions.IgnoreOpenFiles);

			logger.LogInformation("Switched to stream {StreamName}", streamName);
		}

		private static async Task SwitchProjectAsync(IPerforceConnection perforceClient, UserWorkspaceSettings settings, string projectName, ILogger logger)
		{
			settings.ProjectPath = await ProjectUtils.FindProjectPathAsync(perforceClient, settings.ClientName, settings.BranchPath, projectName);
			settings.Save(logger);
			logger.LogInformation("Switched to project {ProjectPath}", settings.ClientProjectPath);
		}
	}
}
