// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.InteropServices;
using System.Threading.Tasks;

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.OIDC;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

using UnrealGameSync;

namespace UnrealGameSyncCmd.Commands
{
	internal class ToolsCommandCommandOptions
	{
		[CommandLine("-Install=", ListSeparator = ',')]
		public List<string> Install { get; set; } = new List<string>();

		[CommandLine("-Uninstall=", ListSeparator = ',')]
		public List<string> Uninstall { get; set; } = new List<string>();

		[CommandLine("-List")]
		public bool List { get; set; }

		[CommandLine("-Update")]
		public bool Update { get; set; }
	}

	internal class ToolsCommand : ICommand
	{
		public static async Task ExecuteAsync(ILogger logger, CommandLineArguments args, UserSettings? globalSettings)
		{
			if (!RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				logger.LogInformation("This command is only available for Windows.");
				return;
			}

			args.TryGetPositionalArgument(out string? toolId);

			ToolsCommandCommandOptions options = args.ApplyTo<ToolsCommandCommandOptions>(logger);
			args.CheckAllArgumentsUsed();

			if (globalSettings == null)
			{
				logger.LogError("Could not retrieve user settings.");
				return;
			}

			using IPerforceConnection perforce = await PerforceConnection.CreateAsync(logger);

			DirectoryReference dataFolder = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData)!, "UnrealGameSync");
			DirectoryReference.CreateDirectory(dataFolder);

			// create a temporary service provider for the tool update monitor
			ServiceCollection services = new ServiceCollection();
			services.AddSingleton<IAsyncDisposer, AsyncDisposer>();
			services.AddSingleton(sp => TokenStoreFactory.CreateTokenStore());
			services.AddSingleton<OidcTokenManager>();

			LauncherSettings launcherSettings = new LauncherSettings();
			launcherSettings.Read();

			if (launcherSettings.HordeServer != null)
			{
				services.AddHorde(opt =>
				{
					opt.ServerUrl = new Uri(launcherSettings.HordeServer);
					opt.AllowAuthPrompt = false;
				});
			}

			ServiceProvider serviceProvider = services.BuildServiceProvider();

			using ToolUpdateMonitor toolUpdateMonitor =
				new ToolUpdateMonitor(perforce.Settings, dataFolder, globalSettings, logger, serviceProvider);

			// get the list of tools available
			logger.LogInformation("Retrieving tools information, please wait.");
			await toolUpdateMonitor.GetDataFromBackendAsync();

			// only list the tools
			if (options.List)
			{
				logger.LogInformation("Available Tools:");
				logger.LogInformation("");
				logger.LogInformation("  Id                                   | Name                                 | Enabled         | Description                                      ");
				logger.LogInformation("  -------------------------------------| -------------------------------------| ----------------| -------------------------------------------------");

				IReadOnlyList<ToolInfo> enabled = toolUpdateMonitor.GetEnabledTools();

				foreach (ToolInfo toolInfo in toolUpdateMonitor.GetTools().OrderBy(t => enabled.All(e => t.Id != e.Id)))
				{
					logger.LogInformation("  {Id,-36} | {Name,-36} | {Enabled,-16}| {Description,-48} ", toolInfo.Id, toolInfo.Name, enabled.Any(t => t.Id == toolInfo.Id), toolInfo.Description);
				}
			}

			// update the list of enabled tools and update (this will update all the tools)
			if (options.Install.Any() || options.Uninstall.Any())
			{
				IReadOnlyDictionary<Guid, ToolInfo> tools = toolUpdateMonitor.GetTools().ToDictionary(t => t.Id);
				IReadOnlyList<ToolInfo> enabled = toolUpdateMonitor.GetEnabledTools();
				HashSet<Guid> newEnabledTools = new HashSet<Guid>(enabled.Select(t => t.Id));

				foreach (string id in options.Install.Select(id => id.Trim()))
				{
					if (Guid.TryParse(id, out Guid guid))
					{
						if (tools.TryGetValue(guid, out ToolInfo? toolInfo))
						{
							logger.LogInformation("Adding tool '{Id}' - '{Name}' to the list of enabled tools", id, toolInfo.Name);
							newEnabledTools.Add(guid);

							foreach (Guid dependsOnToolId in toolInfo.DependsOnToolIds)
							{
								if (tools.TryGetValue(dependsOnToolId, out ToolInfo? dependsOnToolInfo))
								{
									logger.LogInformation("Tool '{Id}' - '{Name}' will be installed as it is a dependency of '{ToolName}'", dependsOnToolId, dependsOnToolInfo.Name, toolInfo.Name);
								}
								else
								{
									logger.LogError("Could not install tool '{Id}' dependency '{DependsOnToolId}'", id, dependsOnToolId);
								}
							}
						}
						else
						{
							logger.LogError("Could not install tool '{Id}'", id);
						}
					}
					else
					{
						logger.LogError("'{Id}' is not a valid guid", id);
					}
				}

				foreach (string id in options.Uninstall.Select(id => id.Trim()))
				{
					if (Guid.TryParse(id, out Guid guid))
					{
						foreach (ToolInfo toolInfo in toolUpdateMonitor.GetTools())
						{
							if (toolInfo.Id == guid)
							{
								logger.LogInformation("Removing tool '{Id}' - '{Name}' from the list of enabled tools", id, toolInfo.Name);
								newEnabledTools.Remove(guid);
								break;
							}
						}

						if (newEnabledTools.Contains(guid))
						{
							logger.LogError("Could not uninstall tool '{Id}'", id);
						}
					}
					else
					{
						logger.LogError("'{Id}' is not a valid guid", id);
					}
				}

				if (!newEnabledTools.SequenceEqual(globalSettings.EnabledTools))
				{
					globalSettings.EnabledTools.Clear();
					globalSettings.EnabledTools.UnionWith(newEnabledTools);
					globalSettings.Save(logger);

					logger.LogInformation("Updating tools");
					await toolUpdateMonitor.UpdateToolsAsync();
				}
			}

			// update the all the enabled tools
			if (options.Update)
			{
				await toolUpdateMonitor.UpdateToolsAsync();
			}
		}
	}
}
