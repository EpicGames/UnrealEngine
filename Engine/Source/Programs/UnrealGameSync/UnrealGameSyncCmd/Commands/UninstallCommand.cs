// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;
using UnrealGameSyncCmd.Utils;

namespace UnrealGameSyncCmd.Commands
{
	internal class UninstallCommand : Command
	{
		public override async Task ExecuteAsync(CommandContext context)
		{
			await InstallUtils.UpdateInstallAsync(false, context.Logger!);
		}
	}
}
