// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.IO.Compression;
using System.Net.Http;
using System.Reflection;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

using EpicGames.Core;

using UnrealGameSync;
using UnrealGameSyncCmd.Utils;

namespace UnrealGameSyncCmd.Commands
{
	internal class UpgradeCommandOptions
	{
		[CommandLine("-Check")]
		public bool Check { get; set; }

		[CommandLine("-Force")]
		public bool Force { get; set; }
	}

	internal class UpgradeCommand : ICommand
	{
		public static async Task ExecuteAsync(ILogger logger, CommandLineArguments args)
		{
			UpgradeCommandOptions options = new UpgradeCommandOptions();
			args.ApplyTo(options);
			string? targetDirStr = args.GetStringOrDefault("-TargetDir=", null);
			args.CheckAllArgumentsUsed(logger);

			string currentVersion = VersionUtils.GetVersion();

			string? latestVersion = await VersionUtils.GetLatestVersionAsync(logger, CancellationToken.None);
			if (latestVersion == null)
			{
				return;
			}
			if (latestVersion.Equals(currentVersion, StringComparison.OrdinalIgnoreCase) && !options.Force)
			{
				logger.LogInformation("You are running the latest version ({Version})", currentVersion);
				return;
			}
			if (options.Check)
			{
				logger.LogWarning("A newer version of UGS is available ({NewVersion})", latestVersion);
				return;
			}

			using (HttpClient httpClient = new HttpClient())
			{
				Uri baseUrl = new Uri(DeploymentSettings.Instance.HordeUrl ?? String.Empty);

				DirectoryReference currentDir = new FileReference(Assembly.GetExecutingAssembly().Location).Directory;

				DirectoryReference targetDir = (targetDirStr == null) ? currentDir : DirectoryReference.Combine(currentDir, targetDirStr);
				DirectoryReference.CreateDirectory(targetDir);

				FileReference tempFile = FileReference.Combine(targetDir, "update.zip");
				using (Stream requestStream = await httpClient.GetStreamAsync(new Uri(baseUrl, $"api/v1/tools/{VersionUtils.GetUpgradeToolName()}?action=download")))
				{
					using (Stream tempFileStream = FileReference.Open(tempFile, FileMode.Create, FileAccess.Write, FileShare.None))
					{
						await requestStream.CopyToAsync(tempFileStream);
					}
				}

				using (FileStream stream = FileReference.Open(tempFile, FileMode.Open, FileAccess.Read, FileShare.Read))
				{
					using (ZipArchive archive = new ZipArchive(stream, ZipArchiveMode.Read, true))
					{
						foreach (ZipArchiveEntry entry in archive.Entries)
						{
							FileReference targetFile = FileReference.Combine(targetDir, entry.Name);
							if (!targetFile.IsUnderDirectory(targetDir))
							{
								throw new InvalidDataException("Attempt to extract file outside source directory");
							}
							entry.ExtractToFile_CrossPlatform(targetFile.FullName, true);
						}
					}
				}
			}
		}
	}
}
