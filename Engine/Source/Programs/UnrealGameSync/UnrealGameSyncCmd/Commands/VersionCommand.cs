// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;
using UnrealGameSyncCmd.Utils;

namespace UnrealGameSyncCmd.Commands
{
	internal class VersionCommand : ICommand
	{
		public static void Execute(ILogger logger)
		{
			string version = VersionUtils.GetVersion();
			logger.LogInformation("UnrealGameSync {Version}", version);

		}
	}
}
