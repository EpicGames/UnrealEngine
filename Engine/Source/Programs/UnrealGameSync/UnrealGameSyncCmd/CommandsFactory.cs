// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.InteropServices;

using UnrealGameSyncCmd.Commands;
using UnrealGameSyncCmd.Options;

namespace UnrealGameSyncCmd
{
	internal static class CommandsFactory
	{
		internal static CommandInfo[] GetCommands()
		{
			List<CommandInfo> commandInfos = PortableCommands();

			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				commandInfos.AddRange(GetWinCommand());
			}

			return commandInfos
				.OrderBy(x => x.Name, StringComparer.OrdinalIgnoreCase)
				.ToArray();
		}

		private static List<CommandInfo> PortableCommands()
		{
			return new List<CommandInfo>
			{
				new CommandInfo("init", typeof(InitCommand), typeof(InitCommandOptions),
				"ugs init [stream-path]",
				"Create a client for the given stream, or initializes an existing client for use by UGS."
			),
				new CommandInfo("switch", typeof(SwitchCommand), typeof(SwitchCommandOptions),
				"ugs switch [project name|project path|stream]",
				"Changes the active project to the one in the workspace with the given name, or switches to a new stream."
			),
				new CommandInfo("changes", typeof(ChangesCommand), typeof(ChangesCommandOptions),
				"ugs changes",
				"List recently submitted changes to the current branch."
			),
				new CommandInfo("config", typeof(ConfigCommand), typeof(ConfigCommandOptions),
				"ugs config",
				"Updates the configuration for the current workspace."
			),
				new CommandInfo("filter", typeof(FilterCommand), typeof(FilterCommandOptions),
				"ugs filter",
				"Displays or updates the workspace or global sync filter"
			),
				new CommandInfo("sync", typeof(SyncCommand), typeof(SyncCommandOptions),
				"ugs sync [change|'latest']",
				"Syncs the current workspace to the given changelist, optionally removing all local state."
			),
				new CommandInfo("clients", typeof(ClientsCommand), typeof(ClientsCommandOptions),
				"ugs clients",
				"Lists all clients suitable for use on the current machine."
			),
				new CommandInfo("run", typeof(RunCommand), typeof(RunCommandOptions),
				"ugs run",
				"Runs the editor for the current branch."
			),
				new CommandInfo("build", typeof(BuildCommand), typeof(BuildCommandOptions),
				"ugs build [id]",
				"Runs the default build steps for the current project, or a particular step referenced by id."
			),
				new CommandInfo("status", typeof(StatusCommand), null,
				"ugs status [-update]",
				"Shows the status of the currently synced branch."
			),
				new CommandInfo("login", typeof(LoginCommand), null,
				"ugs login",
				"Starts a interactive login flow against the configured Identity Provider"
			),
				new CommandInfo("version", typeof(VersionCommand), null,
				"ugs version",
				"Prints the current application version"
			),
				new CommandInfo("install", typeof(InstallCommand), null,
				null,
				null
			),
				new CommandInfo("uninstall", typeof(UninstallCommand), null,
				null,
				null
			),
				new CommandInfo("upgrade", typeof(UpgradeCommand), typeof(UpgradeCommandOptions),
				"ugs upgrade",
				"Upgrades the current installation with the latest build of UGS."
			),
				new CommandInfo("settings", typeof(SettingsCommand), typeof(SettingsCommandOptions),
				"ugs settings",
				"Export the current ugs settings to a file"
			),
			};
		}

		private static List<CommandInfo> GetWinCommand()
		{
			return new List<CommandInfo>()
			{
				new CommandInfo("tools", typeof(ToolsCommand), typeof(ToolsCommandCommandOptions),
				"ugs tools",
				"Install a Custom Tool. Only available for windows."
				),
				new CommandInfo("archive", typeof(ArchiveCommand), typeof(ArchiveCommandOptions),
				"ugs archive",
				"Execute a command related to archives."
				),
				new CommandInfo("archive-override", typeof(ArchiveOverrideCommand), typeof(ArchiveOverrideCommandOptions),
					"ugs archive-override",
					"Execute a command related to archives overrides per project."
				),
			};
		}
	}
}
