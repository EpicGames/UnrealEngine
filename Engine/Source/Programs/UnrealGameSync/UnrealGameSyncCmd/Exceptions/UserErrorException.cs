// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealGameSyncCmd.Exceptions
{
	public sealed class UserErrorException : Exception
	{
		public LogEvent Event { get; }
		public int Code { get; }

		public UserErrorException(LogEvent evt)
			: base(evt.ToString())
		{
			Event = evt;
			Code = 1;
		}

		public UserErrorException(string message, params object[] args)
			: this(LogEvent.Create(LogLevel.Error, message, args))
		{
		}
	}
}
