// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealGameSync;

namespace UnrealGameSyncCmd.Options
{
	internal class ConfigCommandOptions : ServerOptions
	{
		public void ApplyTo(UserWorkspaceSettings settings)
		{
			if (ServerAndPort != null)
			{
				settings.ServerAndPort = (ServerAndPort.Length == 0) ? null : ServerAndPort;
			}
			if (UserName != null)
			{
				settings.UserName = (UserName.Length == 0) ? null : UserName;
			}
		}
	}
}
