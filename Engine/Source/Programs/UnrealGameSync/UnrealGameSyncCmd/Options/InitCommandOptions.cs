// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace UnrealGameSyncCmd.Options
{
	internal class InitCommandOptions : ConfigCommandOptions
	{
		[CommandLine("-Client=")]
		public string? ClientName { get; set; }

		[CommandLine("-Branch=")]
		public string? BranchPath { get; set; }

		[CommandLine("-Project=")]
		public string? ProjectName { get; set; }

		[CommandLine("-ClientRoot=")]
		public string? ClientRoot { get; set; }

		[CommandLine("-IgnoreExistingClients")]
		public bool IgnoreExistingClients { get; set; }
	}
}
