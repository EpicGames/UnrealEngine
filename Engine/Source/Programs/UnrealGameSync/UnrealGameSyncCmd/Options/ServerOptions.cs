// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace UnrealGameSyncCmd.Options
{
	internal class ServerOptions
	{
		[CommandLine("-Server=")]
		public string? ServerAndPort { get; set; }

		[CommandLine("-User=")]
		public string? UserName { get; set; }
	}
}
