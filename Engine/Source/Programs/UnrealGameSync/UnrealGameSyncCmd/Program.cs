// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Globalization;
using System.Linq;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading;
using System.Threading.Tasks;

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Perforce;

using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

using Serilog;
using UnrealGameSync;
using UnrealGameSyncCmd.Commands;
using UnrealGameSyncCmd.Utils;
using UserErrorException = UnrealGameSyncCmd.Exceptions.UserErrorException;

namespace UnrealGameSyncCmd
{
	using ILogger = Microsoft.Extensions.Logging.ILogger;

	public static class Program
	{
		static readonly CommandInfo[] _commands = CommandsFactory.GetCommands();

		public static async Task<int> Main(string[] rawArgs)
		{
			DirectoryReference globalConfigFolder;
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				globalConfigFolder = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData)!, "UnrealGameSync");
			}
			else
			{
				globalConfigFolder = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.UserProfile)!, ".config", "UnrealGameSync");
			}
			DirectoryReference.CreateDirectory(globalConfigFolder);

			string logName;
			DirectoryReference logFolder;
			if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				logFolder = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.UserProfile)!, "Library", "Logs", "Unreal Engine", "UnrealGameSync");
				logName = "UnrealGameSync-.log";
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				logFolder = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.UserProfile)!;
				logName = ".ugs-.log";
			}
			else
			{
				logFolder = globalConfigFolder;
				logName = "UnrealGameSyncCmd-.log";
			}

			Serilog.ILogger serilogLogger = new LoggerConfiguration()
				.Enrich.FromLogContext()
				.WriteTo.Console(Serilog.Events.LogEventLevel.Information, outputTemplate: "{Message:lj}{NewLine}")
				.WriteTo.File(FileReference.Combine(logFolder, logName).FullName, Serilog.Events.LogEventLevel.Debug, rollingInterval: RollingInterval.Day, rollOnFileSizeLimit: true, fileSizeLimitBytes: 20 * 1024 * 1024, retainedFileCountLimit: 10)
				.CreateLogger();

			using ILoggerFactory loggerFactory = new Serilog.Extensions.Logging.SerilogLoggerFactory(serilogLogger, true);
			ILogger logger = loggerFactory.CreateLogger("Main");
			try
			{
				LauncherSettings launcherSettings = new LauncherSettings();
				launcherSettings.Read();

				ServiceCollection services = new ServiceCollection();

				if (launcherSettings.HordeServer != null)
				{
					services.AddHorde(options => options.ServerUrl = new Uri(launcherSettings.HordeServer));
				}

				services.AddCloudStorage();

				ServiceProvider serviceProvider = services.BuildServiceProvider();
				IHordeClient? hordeClient = serviceProvider.GetService<IHordeClient>();
				ICloudStorage? cloudStorage = serviceProvider.GetService<ICloudStorage>();

				UserSettings? globalSettings = null;
				GlobalSettingsFile settings;
				if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
				{
					globalSettings = UserSettings.Create(globalConfigFolder, logger);
					settings = globalSettings as GlobalSettingsFile;
				}
				else
				{
					settings = GlobalSettingsFile.Create(FileReference.Combine(globalConfigFolder, "Global.json"));
				}

				CommandLineArguments args = new CommandLineArguments(rawArgs);

				string? commandName;
				if (!args.TryGetPositionalArgument(out commandName))
				{
					PrintHelp();
					return 0;
				}

				CommandInfo? command = _commands.FirstOrDefault(x => x.Name.Equals(commandName, StringComparison.OrdinalIgnoreCase));
				if (command == null)
				{
					logger.LogError("Unknown command '{Command}'", commandName);
					Console.WriteLine();
					PrintHelp();
					return 1;
				}

				// On Windows this is distributed with the GUI client, so we don't need to check for upgrades
				if (command.Type != typeof(InstallCommand) && command.Type != typeof(UpgradeCommand) && !RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
				{
					string version = VersionUtils.GetVersion();

					DateTime utcNow = DateTime.UtcNow;
					if (settings.Global.LastVersionCheck < utcNow - TimeSpan.FromDays(1.0) || IsUpgradeAvailable(settings, version))
					{
						using (CancellationTokenSource cancellationSource = new CancellationTokenSource(TimeSpan.FromSeconds(10.0)))
						{
							Task<string?> latestVersionTask = VersionUtils.GetLatestVersionAsync(null, cancellationSource.Token);

							Task delay = Task.Delay(TimeSpan.FromSeconds(2.0));
							await Task.WhenAny(latestVersionTask, delay);

							if (!latestVersionTask.IsCompleted)
							{
								logger.LogInformation("Checking for UGS updates...");
							}

							try
							{
								settings.Global.LatestVersion = await latestVersionTask;
							}
							catch (OperationCanceledException)
							{
								logger.LogInformation("Request timed out.");
							}
							catch (Exception ex)
							{
								logger.LogInformation(ex, "Upgrade check failed: {Message}", ex.Message);
							}
						}

						settings.Global.LastVersionCheck = utcNow;
						settings.Save(logger);
					}

					if (IsUpgradeAvailable(settings, version))
					{
						logger.LogWarning("A newer version of UGS is available ({LatestVersion} > {Version}). Run {Command} to update.", settings.Global.LatestVersion, version, "ugs upgrade");
						logger.LogInformation("");
					}
				}

				ICommand instance = (ICommand)Activator.CreateInstance(command.Type)!;

				switch (instance)
				{
					//////////////////////////////////////
					case ClientsCommand _:
						{
							await ClientsCommand.ExecuteAsync(logger, args, loggerFactory);
							break;
						}

					//////////////////////////////////////
					case InitCommand _:
						{
							await InitCommand.ExecuteAsync(logger, args, loggerFactory, globalSettings);
							break;
						}

					//////////////////////////////////////
					case SettingsCommand _:
						{
							await SettingsCommand.ExecuteAsync(logger, loggerFactory, args, globalSettings, settings);
							break;
						}

					//////////////////////////////////////
					case ToolsCommand _:
						{
							await ToolsCommand.ExecuteAsync(logger, args, globalSettings);
							break;
						}

					//////////////////////////////////////
					case UpgradeCommand _:
						{
							await UpgradeCommand.ExecuteAsync(logger, args);
							break;
						}

					//////////////////////////////////////
					case VersionCommand _:
						{
							VersionCommand.Execute(logger);
							break;
						}

					//////////////////////////////////////
					case Command contextCommand:
						{
							using CommandContext commandContext = await CommandContext.CreateAsync(args, serviceProvider,logger, loggerFactory, settings, globalSettings, hordeClient, cloudStorage);

							await contextCommand.ExecuteAsync(commandContext!);
							break;
						}
				}

				return 0;
			}
			catch (UserErrorException ex)
			{
				logger.Log(ex.Event.Level, "{Message}", ex.Event.ToString());
				return ex.Code;
			}
			catch (PerforceException ex)
			{
				logger.LogError(ex, "{Message}", ex.Message);
				return 1;
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Unhandled exception.\n{Str}", ex);
				return 1;
			}
		}

		private static bool IsUpgradeAvailable(GlobalSettingsFile settings, string version)
		{
			return settings.Global.LatestVersion != null && !settings.Global.LatestVersion.Equals(version, StringComparison.OrdinalIgnoreCase);
		}

		private static void PrintHelp()
		{
			string appName = "UnrealGameSync Command-Line Tool";

			string? productVersion = FileVersionInfo.GetVersionInfo(Assembly.GetExecutingAssembly().Location).ProductVersion;
			if (productVersion != null)
			{
				appName = $"{appName} ({productVersion})";
			}

			Console.WriteLine(appName);
			Console.WriteLine("");
			Console.WriteLine("Usage:");
			foreach (CommandInfo command in _commands)
			{
				if (command.Usage != null && command.Brief != null)
				{
					Console.WriteLine();
					ConsoleUtils.WriteLineWithWordWrap(GetUsage(command), 2, 8);
					ConsoleUtils.WriteLineWithWordWrap(command.Brief, 4, 4);
				}
			}
		}

		private static string GetUsage(CommandInfo commandInfo)
		{
			StringBuilder result = new StringBuilder(commandInfo.Usage);
			if (commandInfo.OptionsType != null)
			{
				foreach (PropertyInfo propertyInfo in commandInfo.OptionsType.GetProperties(BindingFlags.Instance | BindingFlags.Public))
				{
					List<string> names = new List<string>();
					foreach (CommandLineAttribute attribute in propertyInfo.GetCustomAttributes<CommandLineAttribute>())
					{
						string name = (attribute.Prefix ?? propertyInfo.Name).ToLower(CultureInfo.InvariantCulture);
						if (propertyInfo.PropertyType == typeof(bool) || propertyInfo.PropertyType == typeof(bool?))
						{
							names.Add(name);
						}
						else
						{
							names.Add($"{name}..");
						}
					}
					if (names.Count > 0)
					{
						result.Append($" [{String.Join('|', names)}]");
					}
				}
			}
			return result.ToString();
		}
	}
}
