{
    "installCommand": {
        "fileName": "ugs.exe",
        "arguments": [
            "install"
        ]
    },
    "uninstallCommand": {
        "fileName": "ugs.exe",
        "arguments": [
            "uninstall"
        ]
    }
}