<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <AssemblyName>ugs</AssemblyName>
  </PropertyGroup>

  <ItemGroup>
    <Content Include="Toolbox.json">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </Content>
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="Serilog" Version="2.10.0" />
    <PackageReference Include="Serilog.Sinks.Console" Version="4.0.1" />
    <PackageReference Include="Serilog.Sinks.File" Version="5.0.0" />
    <PackageReference Include="Serilog.Extensions.Logging" Version="3.1.0" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\UnrealGameSyncShared\UnrealGameSyncShared.csproj" />
  </ItemGroup>

  <ItemGroup>
    <None Update="setup-mac.sh">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </None>
  </ItemGroup>

  <ItemGroup Condition="Exists('..\..\..\..\Binaries\Win64\zen.exe')">
    <None Include="..\..\..\..\Binaries\Win64\zen.exe" TargetPath="Binaries\Win64\zen.exe" Link="Binaries\zen.exe">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Include="..\..\..\..\Binaries\Linux\zen" TargetPath="Binaries\Linux\zen" Link="Binaries\zen">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Include="..\..\..\..\Binaries\Mac\zen" TargetPath="Binaries\Mac\zen" Link="Binaries\zen">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
  </ItemGroup>

</Project>
