// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Reflection;
using System.Text.RegularExpressions;
using System.Threading.Tasks;

using Microsoft.Extensions.Logging;

using EpicGames.Core;

namespace UnrealGameSyncCmd.Utils
{
	internal static class InstallUtils
	{
		internal static async Task UpdateInstallAsync(bool install, ILogger logger)
		{
			DirectoryReference? installDir = GetInstallFolder();
			if (installDir != null)
			{
				UpdateInstalledFiles(install, installDir, logger);
			}
			else
			{
				installDir = new FileReference(Assembly.GetExecutingAssembly().GetOriginalLocation()).Directory;
			}

			if (OperatingSystem.IsWindows())
			{
				const string EnvVarName = "PATH";

				string? pathVar = Environment.GetEnvironmentVariable(EnvVarName, EnvironmentVariableTarget.User);
				pathVar ??= String.Empty;

				List<string> paths = new List<string>(pathVar.Split(Path.PathSeparator, StringSplitOptions.RemoveEmptyEntries));

				int changes = paths.RemoveAll(x => x.Equals(installDir.FullName, StringComparison.OrdinalIgnoreCase));
				if (install)
				{
					paths.Add(installDir.FullName);
					changes++;
				}
				if (changes > 0)
				{
					pathVar = String.Join(Path.PathSeparator, paths);
					Environment.SetEnvironmentVariable(EnvVarName, pathVar, EnvironmentVariableTarget.User);
				}

				if (install)
				{
					logger.LogInformation("Added {Path} to PATH environment variable", installDir);
				}
				else
				{
					logger.LogInformation("Removed {Path} from PATH environment variable", installDir);
				}
			}
			else if (OperatingSystem.IsMacOS())
			{
				DirectoryReference? userDir = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.UserProfile);
				if (userDir != null)
				{
					FileReference configFile = FileReference.Combine(userDir, ".zshrc");
					await UpdateAliasAsync(configFile, install, installDir, logger);
				}
			}
			else if (OperatingSystem.IsLinux())
			{
				DirectoryReference? userDir = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.UserProfile);
				if (userDir != null)
				{
					FileReference configFile = FileReference.Combine(userDir, ".bashrc");
					await UpdateAliasAsync(configFile, install, installDir, logger);
				}
			}
		}

		internal static DirectoryReference? GetInstallFolder()
		{
			if (OperatingSystem.IsWindows())
			{
				DirectoryReference? installDir = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData);
				if (installDir != null)
				{
					return DirectoryReference.Combine(installDir, "Epic Games", "UgsCmd");
				}
			}
			return null;
		}

		internal static void UpdateInstalledFiles(bool install, DirectoryReference installDir, ILogger logger)
		{
			FileReference assemblyFile = new FileReference(Assembly.GetExecutingAssembly().GetOriginalLocation());
			DirectoryReference sourceDir = assemblyFile.Directory;

			DirectoryReference TempDir = DirectoryReference.Combine(installDir.ParentDirectory!, "~" + installDir.GetDirectoryName());
			if (DirectoryReference.Exists(TempDir))
			{
				DirectoryReference.Delete(TempDir, true);
			}

			if (DirectoryReference.Exists(installDir))
			{
				logger.LogInformation("Removing application files from {Dir}", installDir);

				Directory.Move(installDir.FullName, TempDir.FullName);
				DirectoryReference.Delete(TempDir, true);
			}

			if (install)
			{
				logger.LogInformation("Copying application files to {Dir}", installDir);

				DirectoryReference.CreateDirectory(TempDir);
				foreach (FileReference SourceFile in DirectoryReference.EnumerateFiles(sourceDir, "*", SearchOption.AllDirectories))
				{
					FileReference TargetFile = FileReference.Combine(TempDir, SourceFile.MakeRelativeTo(sourceDir));
					DirectoryReference.CreateDirectory(TargetFile.Directory);
					FileReference.Copy(SourceFile, TargetFile, true);
				}
				Directory.Move(TempDir.FullName, installDir.FullName);
			}
		}

		internal static async Task UpdateAliasAsync(FileReference configFile, bool install, DirectoryReference installDir, ILogger logger)
		{
			List<string> lines = new List<string>();
			if (FileReference.Exists(configFile))
			{
				lines.AddRange(await FileReference.ReadAllLinesAsync(configFile));
				lines.RemoveAll(x => Regex.IsMatch(x, @"^\s*alias\s+ugs\s*="));
			}
			if (install)
			{
				lines.Add($"alias ugs={FileReference.Combine(installDir, "ugs")}");
			}

			await FileReference.WriteAllLinesAsync(configFile, lines);

			if (install)
			{
				logger.LogInformation("Added 'ugs' alias to {ConfigFile}", configFile);
			}
			else
			{
				logger.LogInformation("Removed 'ugs' alias from {ConfigFile}", configFile);
			}
		}
	}
}
