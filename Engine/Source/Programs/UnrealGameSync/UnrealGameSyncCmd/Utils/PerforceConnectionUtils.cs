// Copyright Epic Games, Inc. All Rights Reserved.

using System;

using System.Threading.Tasks;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;
using UnrealGameSync;

namespace UnrealGameSyncCmd.Utils
{
	public static class PerforceConnectionUtils
	{
		public static Task<IPerforceConnection> ConnectAsync(string? serverAndPort, string? userName, string? clientName, ILoggerFactory loggerFactory)
		{
			PerforceSettings settings = new PerforceSettings(PerforceSettings.Default);
			settings.ClientName = clientName;
			settings.PreferNativeClient = true;
			if (!String.IsNullOrEmpty(serverAndPort))
			{
				settings.ServerAndPort = serverAndPort;
			}
			if (!String.IsNullOrEmpty(userName))
			{
				settings.UserName = userName;
			}

			return PerforceConnection.CreateAsync(settings, loggerFactory.CreateLogger("Perforce"));
		}

		public static Task<IPerforceConnection> ConnectAsync(UserWorkspaceSettings settings, ILoggerFactory loggerFactory)
		{
			return ConnectAsync(settings.ServerAndPort, settings.UserName, settings.ClientName, loggerFactory);
		}
	}
}
