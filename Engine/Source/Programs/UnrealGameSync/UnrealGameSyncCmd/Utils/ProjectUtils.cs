// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;

using EpicGames.Perforce;
using UnrealGameSyncCmd.Exceptions;

namespace UnrealGameSyncCmd.Utils
{
	internal static class ProjectUtils
	{
		internal const string UProjectExtension = ".uproject";
		internal const string UProjectDirsExtension = ".uprojectdirs";

		internal static async Task<string> FindProjectPathAsync(IPerforceConnection perforce, string clientName, string branchPath, string? projectName)
		{
			using IPerforceConnection perforceClient = await PerforceConnection.CreateAsync(new PerforceSettings(perforce.Settings) { ClientName = clientName }, perforce.Logger);

			// Find or validate the selected project
			string searchPath;
			if (projectName == null)
			{
				searchPath = $"//{clientName}{branchPath}/*{UProjectDirsExtension}";
			}
			else if (projectName.Contains('.', StringComparison.Ordinal))
			{
				searchPath = $"//{clientName}{branchPath}/{projectName.TrimStart('/')}";
			}
			else
			{
				searchPath = $"//{clientName}{branchPath}/.../{projectName}{UProjectExtension}";
			}

			List<FStatRecord> projectFileRecords = await perforceClient.FStatAsync(FStatOptions.ClientFileInPerforceSyntax, searchPath).ToListAsync();
			projectFileRecords.RemoveAll(x => x.HeadAction == FileAction.Delete || x.HeadAction == FileAction.MoveDelete);
			projectFileRecords.RemoveAll(x => !x.IsMapped);

			List<string> paths = projectFileRecords.Select(x => PerforceUtils.GetClientRelativePath(x.ClientFile!)).Distinct(StringComparer.Ordinal).ToList();
			if (paths.Count == 0)
			{
				throw new UserErrorException("No project file found matching {SearchPath}", searchPath);
			}
			if (paths.Count > 1)
			{
				throw new UserErrorException("Multiple projects found matching {SearchPath}: {Paths}", searchPath, String.Join(", ", paths));
			}

			return "/" + paths[0];
		}
	}
}
