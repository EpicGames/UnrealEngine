// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using UnrealGameSync;

namespace UnrealGameSyncCmd.Utils
{
	internal static class SyncFilterUtils
	{
		internal static string[] ReadSyncFilter(
			UserWorkspaceSettings workspaceSettings, 
			UserProjectSettings contextUserProjectSettings,
			GlobalSettingsFile userSettings,
			ConfigFile projectConfig,
			string projectIdentifier)
		{
			Dictionary<Guid, WorkspaceSyncCategory> syncCategories = ConfigUtils.GetSyncCategories(projectConfig);
			
			// check if any category is from the role
			IDictionary<string, Preset> roles = ConfigUtils.GetPresets(projectConfig, projectIdentifier);
			if (roles.TryGetValue(contextUserProjectSettings.Preset, out Preset? role))
			{
				foreach (RoleCategory roleCategory in role.Categories.Values)
				{
					if (syncCategories.TryGetValue(roleCategory.Id, out WorkspaceSyncCategory? category))
					{
						category.Enable = roleCategory.Enabled;
					}
				}
			}

			ConfigSection? perforceSection = projectConfig.FindSection("Perforce");

			string[] combinedSyncFilter = GlobalSettingsFile.GetCombinedSyncFilter(syncCategories, contextUserProjectSettings.Preset, roles, userSettings.Global.Filter, workspaceSettings.Filter, perforceSection);

			return combinedSyncFilter;
		}
	}
}
