// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealGameSync;

namespace UnrealGameSyncCmd.Utils
{
	public static class UserSettingsUtils
	{
		public static UserWorkspaceSettings? ReadOptionalUserWorkspaceSettings()
		{
			DirectoryReference? dir = DirectoryReference.GetCurrentDirectory();
			for (; dir != null; dir = dir.ParentDirectory)
			{
				try
				{
					UserWorkspaceSettings? settings;
					if (UserWorkspaceSettings.TryLoad(dir, out settings))
					{
						return settings;
					}
				}
				catch
				{
					// Guard against directories we can't access, eg. /Users/.ugs
				}
			}
			return null;
		}

		public static UserWorkspaceSettings ReadRequiredUserWorkspaceSettings()
		{
			UserWorkspaceSettings? settings = ReadOptionalUserWorkspaceSettings();
			if (settings == null)
			{
				throw new UserErrorException("Unable to find UGS workspace in current directory.");
			}
			return settings;
		}
	}
}
