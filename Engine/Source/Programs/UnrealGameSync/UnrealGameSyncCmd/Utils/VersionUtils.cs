// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net.Http;
using System.Net.Http.Json;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;

using Microsoft.Extensions.Logging;
using UnrealGameSync;

namespace UnrealGameSyncCmd.Utils
{
	public class DeploymentInfo
	{
		public string Version { get; set; } = String.Empty;
	}

	public static class VersionUtils
	{
		public static string GetVersion()
		{
			AssemblyInformationalVersionAttribute? version = Assembly.GetExecutingAssembly().GetCustomAttribute<AssemblyInformationalVersionAttribute>();
			return version?.InformationalVersion ?? "Unknown";
		}

		public static async Task<string?> GetLatestVersionAsync(ILogger? logger, CancellationToken cancellationToken)
		{
			string? hordeUrl = DeploymentSettings.Instance.HordeUrl;
			if (hordeUrl == null)
			{
				logger?.LogError("Horde URL is not set in deployment config file. Cannot upgrade.");
				return null;
			}

			string? toolName = GetUpgradeToolName();
			if (toolName == null)
			{
				logger?.LogError("Command-line upgrades are not supported on this platform.");
				return null;
			}

			using (HttpClient httpClient = new HttpClient())
			{
				Uri baseUrl = new Uri(hordeUrl);

				DeploymentInfo? deploymentInfo;
				try
				{
					deploymentInfo = await httpClient.GetFromJsonAsync<DeploymentInfo>(new Uri(baseUrl, $"api/v1/tools/{toolName}/deployments"), cancellationToken);
				}
				catch (Exception ex)
				{
					logger?.LogError(ex, "Failed to query for deployment info: {Message}", ex.Message);
					return null;
				}
				if (deploymentInfo == null)
				{
					logger?.LogError("Failed to query for deployment info.");
					return null;
				}
				return deploymentInfo.Version;
			}
		}

		public static string? GetUpgradeToolName()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				if (RuntimeInformation.OSArchitecture == Architecture.Arm64)
				{
					return "ugs-mac-arm64";
				}
				else
				{
					return "ugs-mac";
				}
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				return "ugs-linux";
			}
			else
			{
				return null;
			}
		}
	}
}
