// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading;
using System.Threading.Tasks;

using EpicGames.Perforce;
using Microsoft.Extensions.Logging;
using UnrealGameSync;

namespace UnrealGameSyncCmd.Utils
{
	internal static class WorkspaceStateUtils
	{
		public static async Task<WorkspaceStateWrapper> ReadWorkspaceState(IPerforceConnection perforceClient, UserWorkspaceSettings settings, GlobalSettingsFile userSettings, ILogger logger)
		{
			WorkspaceStateWrapper state = userSettings.FindOrAddWorkspaceState(settings);
			if (state.Current.SettingsTimeUtc != settings.LastModifiedTimeUtc)
			{
				logger.LogDebug("Updating state due to modified settings timestamp");
				ProjectInfo info = await ProjectInfo.CreateAsync(perforceClient, settings, CancellationToken.None);
				state.Modify(x => x.UpdateCachedProjectInfo(info, settings.LastModifiedTimeUtc));
			}
			return state;
		}
	}
}
