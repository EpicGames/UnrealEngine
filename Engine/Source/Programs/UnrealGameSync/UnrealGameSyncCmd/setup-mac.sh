#!/bin/zsh

cd "$(dirname "$0")"
./ugs install
source ~/.zshrc
