// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Perforce;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	public interface IArchive
	{
		public string Key { get; }
		bool? RemoveOldBinaries { get; }

		Task<bool> DownloadAsync(IPerforceConnection perforce, DirectoryReference localRootPath, FileReference manifestFileName, ILogger logger, ProgressValue progress, CancellationToken cancellationToken);
	}

	public interface IArchiveChannel
	{
		public const string EditorArchiveType = "Editor";

		// Name to display in the UI
		string Name { get; }

		// Type key; only one item of each type may be enabled
		string Type { get; }

		// Tooltip when hovering over item in UI
		string ToolTip { get; }

		// Does this archive channel ignore required badges?  Default is false
		bool IgnoreRequiredBadges { get; }

		bool HasAny();
		IArchive? TryGetArchiveForChangeNumber(int changeNumber, int maxChangeNumber);
	}

	public abstract class BaseArchiveChannel : IArchiveChannel
	{
		public string Name { get; }
		public string Type { get; }

		public virtual string ToolTip { get; } = "";

		public bool IgnoreRequiredBadges { get; } = false;

		// TODO: executable/configuration?
		public SortedList<int, IArchive> ChangeNumberToArchive { get; } = new SortedList<int, IArchive>();

		protected BaseArchiveChannel(string name, string type, bool bIgnoreRequiredBadges = false)
		{
			Name = name;
			Type = type;
			IgnoreRequiredBadges = bIgnoreRequiredBadges;
		}

		public bool HasAny()
		{
			return ChangeNumberToArchive.Count > 0;
		}

		public IArchive? TryGetArchiveForChangeNumber(int changeNumber, int maxChangeNumber)
		{
			int idx = ChangeNumberToArchive.Keys.AsReadOnly().BinarySearch(changeNumber);
			if (idx >= 0)
			{
				return ChangeNumberToArchive.Values[idx];
			}

			int nextIdx = ~idx;
			if (nextIdx < ChangeNumberToArchive.Count && ChangeNumberToArchive.Keys[nextIdx] <= maxChangeNumber)
			{
				return ChangeNumberToArchive.Values[nextIdx];
			}

			return null;
		}

		public override string ToString()
		{
			return Name;
		}
	}

	public class PerforceArchiveChannel : BaseArchiveChannel
	{
		public string DepotPath { get; set; }
		public string? Target { get; }

		public override string ToolTip
			=> HasAny() ? "" : $"No valid archives found at {DepotPath}";

		public PerforceArchiveChannel(string name, string type, string depotPath, string? target, bool bIgnoreRequiredBadges = false)
			: base(name, type, bIgnoreRequiredBadges)
		{
			Target = target;
			DepotPath = depotPath;
			Target = target;
		}

		public override bool Equals(object? other)
		{
			PerforceArchiveChannel? otherArchive = other as PerforceArchiveChannel;
			return otherArchive != null && Name == otherArchive.Name && Type == otherArchive.Type && DepotPath == otherArchive.DepotPath && Target == otherArchive.Target
				&& Enumerable.SequenceEqual(ChangeNumberToArchive.Select(x => (x.Key, x.Value)), otherArchive.ChangeNumberToArchive.Select(x => (x.Key, x.Value)));
		}

		public override int GetHashCode()
		{
			throw new NotSupportedException();
		}

		class PerforceArchive : IArchive
		{
			public string Key { get; }
			public bool? RemoveOldBinaries { get; } = true;

			public PerforceArchive(string key)
				=> Key = key;

			public async Task<bool> DownloadAsync(IPerforceConnection perforce, DirectoryReference localRootPath, FileReference manifestFileName, ILogger logger, ProgressValue progress, CancellationToken cancellationToken)
			{
				DirectoryReference configDir = UserSettings.GetConfigDir(localRootPath);
				UserSettings.CreateConfigDir(configDir);

				FileReference tempZipFileName = FileReference.Combine(configDir, "archive.zip");
				try
				{
					PrintRecord record = await perforce.PrintAsync(tempZipFileName.FullName, Key, cancellationToken);

					if (tempZipFileName.ToFileInfo().Length == 0)
					{
						return false;
					}
					ArchiveUtils.ExtractFiles(tempZipFileName, localRootPath, manifestFileName, progress, logger);
				}
				finally
				{
					FileReference.SetAttributes(tempZipFileName, FileAttributes.Normal);
					FileReference.Delete(tempZipFileName);
				}

				return true;
			}
		}

		public static bool TryParseConfigEntryAsync(string text, [NotNullWhen(true)] out PerforceArchiveChannel? channel)
		{
			ConfigObject obj = new ConfigObject(text);

			string? name = obj.GetValue("Name", null);
			if (name == null)
			{
				channel = null;
				return false;
			}

			// Where to find archives, you'll have either Perforce (DepotPath) or Horde (ArchiveType)
			string? depotPath = obj.GetValue("DepotPath", null);
			if (depotPath == null)
			{
				channel = null;
				return false;
			}

			string? target = obj.GetValue("Target", null);

			string type = obj.GetValue("Type", null) ?? name;

			bool bIgnoreRequiredBadges = obj.GetValue("bIgnoreRequiredBadges", false);

			// Build a new list of zipped binaries
			channel = new PerforceArchiveChannel(name, type, depotPath, target, bIgnoreRequiredBadges);
			return true;
		}

		public async Task FindArtifactsAsync(IPerforceConnection perforce, CancellationToken cancellationToken)
		{
			PerforceResponseList<FileLogRecord> response = await perforce.TryFileLogAsync(128, FileLogOptions.FullDescriptions, DepotPath, cancellationToken);
			if (response.Succeeded)
			{
				// Build a new list of zipped binaries
				foreach (FileLogRecord file in response.Data)
				{
					foreach (RevisionRecord revision in file.Revisions)
					{
						if (revision.Action != FileAction.Purge)
						{
							string[] tokens = revision.Description.Split(' ');
							if (tokens[0].StartsWith("[CL", StringComparison.Ordinal) && tokens[1].EndsWith("]", StringComparison.Ordinal))
							{
								int originalChangeNumber;
								if (Int32.TryParse(tokens[1].Substring(0, tokens[1].Length - 1), out originalChangeNumber) && !ChangeNumberToArchive.ContainsKey(originalChangeNumber))
								{
									PerforceArchive archive = new PerforceArchive($"{DepotPath}#{revision.RevisionNumber}");
									ChangeNumberToArchive[originalChangeNumber] = archive;
								}
							}
						}
					}
				}
			}
		}

		public static async Task<List<PerforceArchiveChannel>> GetChannelsAsync(IPerforceConnection perforce, ConfigFile latestProjectConfigFile, string projectIdentifier, CancellationToken cancellationToken)
		{
			List<PerforceArchiveChannel> channels = new List<PerforceArchiveChannel>();

			// Find all the zipped binaries under this stream
			ConfigSection? projectConfigSection = latestProjectConfigFile.FindSection(projectIdentifier);
			if (projectConfigSection != null)
			{
				// Legacy
				string? legacyEditorArchivePath = projectConfigSection.GetValue("ZippedBinariesPath", null);
				if (legacyEditorArchivePath != null)
				{
					// Only Perforce uses the legacy method
					PerforceArchiveChannel legacyChannel = new PerforceArchiveChannel("Editor", "Editor", legacyEditorArchivePath, null);
					await legacyChannel.FindArtifactsAsync(perforce, cancellationToken);
					channels.Add(legacyChannel);
				}

				// New style
				foreach (string archiveValue in projectConfigSection.GetValues("Archives", Array.Empty<string>()))
				{
					PerforceArchiveChannel? channel;
					if (PerforceArchiveChannel.TryParseConfigEntryAsync(archiveValue, out channel))
					{
						await channel.FindArtifactsAsync(perforce, cancellationToken);
						channels.Add(channel!);
					}
				}
			}

			return channels;
		}
	}

	public class HordeArchiveChannel : BaseArchiveChannel
	{
		public HordeArchiveChannel(string name, string type, bool bIgnoreRequiredBadges = false)
			: base(name, type, bIgnoreRequiredBadges)
		{
		}

		public override bool Equals(object? other)
		{
			HordeArchiveChannel? otherArchive = other as HordeArchiveChannel;
			return otherArchive != null && Name == otherArchive.Name && Type == otherArchive.Type
				&& Enumerable.SequenceEqual(ChangeNumberToArchive.Select(x => (x.Key, x.Value)), otherArchive.ChangeNumberToArchive.Select(x => (x.Key, x.Value)));
		}

		public override int GetHashCode()
		{
			throw new NotSupportedException();
		}

		class HordeArchive : IArchive
		{
			readonly IHordeClient _hordeClient;
			readonly ArtifactId _artifactId;

			public string Key { get; }
			public bool? RemoveOldBinaries { get; } = true;

			public HordeArchive(IHordeClient hordeClient, ArtifactId artifactId)
			{
				_hordeClient = hordeClient;
				_artifactId = artifactId;
				Key = artifactId.ToString();
			}

			public async Task<bool> DownloadAsync(IPerforceConnection perforce, DirectoryReference localRootPath, FileReference manifestFileName, ILogger logger, ProgressValue progress, CancellationToken cancellationToken)
			{
				IArtifact? artifact = await _hordeClient.Artifacts.GetAsync(_artifactId, cancellationToken);
				if (artifact == null)
				{
					return false;
				}

				ExtractOptions options = new() { Progress = new ExtractStatsLogger(logger) };
				await artifact.Content.ExtractAsync(localRootPath.ToDirectoryInfo(), options, logger, cancellationToken);

				FileReference.Delete(manifestFileName);

				ArchiveManifest manifest = new();
				await GatherFilesForManifestAsync(String.Empty, artifact.Content, manifest, DateTime.UtcNow, cancellationToken);

				await using (FileStream manifestStream = FileReference.Open(manifestFileName, FileMode.Create, FileAccess.Write))
				{
					manifest.Write(manifestStream);
				}

				return true;
			}

			private static async Task GatherFilesForManifestAsync(string directoryPath, IBlobRef<DirectoryNode> directoryRef, ArchiveManifest manifest, DateTime timeStamp, CancellationToken cancellationToken)
			{
				DirectoryNode directoryNode = await directoryRef.ReadBlobAsync(cancellationToken);

				foreach (FileEntry fileEntry in directoryNode.Files)
				{
					manifest.Files.Add(new ArchiveManifestFile(Path.Combine(directoryPath, fileEntry.Name), fileEntry.Length, fileEntry.ModTime));
				}

				foreach (DirectoryEntry directoryEntry in directoryNode.Directories)
				{
					await GatherFilesForManifestAsync(Path.Combine(directoryPath, directoryEntry.Name), directoryEntry.Handle, manifest, timeStamp, cancellationToken);
				}
			}
		}

		public static async Task<List<HordeArchiveChannel>> GetChannelsAsync(IHordeClient hordeClient, string projectIdentifier, CancellationToken cancellationToken)
		{
			List<HordeArchiveChannel> channels = new List<HordeArchiveChannel>();
			try
			{
				HordeHttpClient hordeHttpClient = hordeClient.CreateHttpClient();

				ArtifactType artifactType = new ArtifactType("ugs-pcb");
				string[] artifactKeys = new[] { $"ugs-project={projectIdentifier}" };

				List<GetArtifactResponse> artifactResponses = await hordeHttpClient.FindArtifactsAsync(type: artifactType, keys: artifactKeys, cancellationToken: cancellationToken);
				foreach (IGrouping<ArtifactName, GetArtifactResponse> group in artifactResponses.GroupBy(x => x.Name))
				{
					GetArtifactResponse? first = group.FirstOrDefault();
					if (first != null)
					{
						string name = first.Description ?? first.Name.ToString();

						const string ArchiveTypePrefix = "ArchiveType=";
						string? archiveType = first.Metadata?.FirstOrDefault(x => x.StartsWith(ArchiveTypePrefix, StringComparison.OrdinalIgnoreCase));

						string type = IArchiveChannel.EditorArchiveType;
						if (archiveType != null)
						{
							type = archiveType.Substring(ArchiveTypePrefix.Length);
						}

						bool ignoreRequiredBadges = !String.Equals(type, IArchiveChannel.EditorArchiveType, StringComparison.OrdinalIgnoreCase);

						HordeArchiveChannel channel = new HordeArchiveChannel(name, type, ignoreRequiredBadges);
						foreach (GetArtifactResponse response in group)
						{
							HordeArchive archive = new HordeArchive(hordeClient, response.Id);
							channel.ChangeNumberToArchive[response.CommitId!.GetPerforceChange()] = archive;
						}
						channels.Add(channel);
					}
				}
			}
			catch (Exception)
			{
			}
			return channels;
		}
	}
	
	public class CloudArchiveChannel : BaseArchiveChannel
	{
		private readonly string _archiveType;
		private readonly string _streamName;

		public CloudArchiveChannel(string name, string type, string archiveType, string streamName, bool bIgnoreRequiredBadges = false)
			: base(name, type, bIgnoreRequiredBadges)
		{
			_archiveType = archiveType;
			_streamName = streamName;
		}

		public override bool Equals(object? other)
		{
			CloudArchiveChannel? otherArchive = other as CloudArchiveChannel;
			return otherArchive != null && Name == otherArchive.Name && Type == otherArchive.Type
				&& Enumerable.SequenceEqual(ChangeNumberToArchive.Select(x => (x.Key, x.Value)), otherArchive.ChangeNumberToArchive.Select(x => (x.Key, x.Value)));
		}

		public override int GetHashCode()
		{
			throw new NotSupportedException();
		}

		class CloudArchive : IArchive
		{
			readonly ICloudStorage _cloudStorageClient;
			private readonly string _host;
			private readonly string _namespaceId;
			private readonly string _bucketId;

			public string Key { get; }
			public bool? RemoveOldBinaries { get; } = false;

			public CloudArchive(ICloudStorage cloudStorageClient, string host, string namespaceId, string bucketId, string buildId)
			{
				_cloudStorageClient = cloudStorageClient;
				_host = host;
				_namespaceId = namespaceId;
				_bucketId = bucketId;
				Key = buildId;
			}

			public async Task<bool> DownloadAsync(IPerforceConnection perforce, DirectoryReference localRootPath, FileReference manifestFileName, ILogger logger, ProgressValue progress, CancellationToken cancellationToken)
			{
				DirectoryReference configDir = UserSettings.GetConfigDir(localRootPath);
				UserSettings.CreateConfigDir(configDir);

				DirectoryReference zenStateFolder = DirectoryReference.Combine(configDir, ".zen");

				Progress<string> progressCallback = new Progress<string>((s =>
				{
#pragma warning disable CA2254
					logger.LogInformation(s);
#pragma warning restore CA2254
				}));

				await _cloudStorageClient.DownloadBuildAsync(_host, _namespaceId, _bucketId,Key, localRootPath, zenStateDirectory: zenStateFolder, progressCallback, cancellationToken);

				FileReference.Delete(manifestFileName);

				ArchiveManifest manifest = new();
				
				// we do not output a list of files we wrote as zencli keeps this state internally and will remove any old files when it updates next
				// and we explicitly do not want UGS to be removing files that we could use to patch the new build
				//manifest.Files.Add();

				await using (FileStream manifestStream = FileReference.Open(manifestFileName, FileMode.Create, FileAccess.Write))
				{
					manifest.Write(manifestStream);
				}

				return true;
			}
		}

		public static async Task<List<CloudArchiveChannel>> GetChannelsAsync(ICloudStorage cloudStorage, ConfigFile latestProjectConfigFile, string projectIdentifier, CancellationToken cancellationToken)
		{
			List<CloudArchiveChannel> channels = new List<CloudArchiveChannel>();
			try
			{
				string? host = null;
				string? namespaceId = null;
				ConfigSection? cloudStorageSection = latestProjectConfigFile.FindSection("CloudStorage");
				if (cloudStorageSection != null)
				{
					namespaceId = cloudStorageSection.GetValue("Namespace", null);
					host = cloudStorageSection.GetValue("Host", null);
				}

				if (namespaceId == null || host == null)
				{
					return channels;
				}

				ConfigSection? projectConfigSection = latestProjectConfigFile.FindSection(projectIdentifier);
				if (projectConfigSection != null)
				{
					string? projectOverride = projectConfigSection.GetValue("CloudArchivesProjectOverride", String.Empty);
					if(!String.IsNullOrWhiteSpace(projectOverride))
					{
						projectIdentifier = projectOverride;
					}

					foreach (string archiveValue in projectConfigSection.GetValues("Archives", Array.Empty<string>()))
					{
						CloudArchiveChannel? channel;
						if (CloudArchiveChannel.TryParseConfigEntryAsync(archiveValue, out channel))
						{
							await channel.FindArtifactsAsync(cloudStorage, projectIdentifier, host, namespaceId, cancellationToken);
							channels.Add(channel!);
						}
					}
				}
			}
			catch (Exception)
			{
			}
			return channels;
		}

		private async Task FindArtifactsAsync(ICloudStorage cloudStorage, string projectIdentifier, string host, string namespaceId, CancellationToken cancellationToken)
		{
			CbWriter queryWriter = new CbWriter();
			queryWriter.BeginObject();
			queryWriter.BeginObject("ugs-project");
			queryWriter.WriteString("$eq", projectIdentifier);
			queryWriter.EndObject(); // ugs-project
			queryWriter.BeginObject("branch");
			queryWriter.WriteString("$eq", SanitizeBucketComponent(_streamName));
			queryWriter.EndObject(); // branch
			queryWriter.BeginObject("platform");
			queryWriter.WriteString("$eq", "windows");
			queryWriter.EndObject(); // platform
			queryWriter.EndObject();

			string SanitizeBucketComponent(string s)
			{
				// . is reserved for bucket component separation
				return s.Replace(".", "-", StringComparison.OrdinalIgnoreCase).ToLowerInvariant();
			}
			string projectName = Path.GetFileNameWithoutExtension(projectIdentifier);
			Regex bucketRegex = new Regex($"{SanitizeBucketComponent(projectName)}.{SanitizeBucketComponent(_archiveType)}\\..*");

			CbObject query = queryWriter.ToObject();

			IAsyncEnumerable<FoundBuildResponse> artifactResponses = cloudStorage.FindBuildAsync(host, namespaceId, bucketRegex, query, cancellationToken: cancellationToken);
			await foreach (FoundBuildResponse response in artifactResponses)
			{
				if (Int32.TryParse(response.Commit, out int commit))
				{
					CloudArchive archive = new CloudArchive(cloudStorage, host, namespaceId, response.BucketId, response.BuildId);
					ChangeNumberToArchive[commit] = archive;
				}
				// not a integer commit, unable to map to perforce
			}
		}

		public static bool TryParseConfigEntryAsync(string text, [NotNullWhen(true)] out CloudArchiveChannel? channel)
		{
			ConfigObject obj = new ConfigObject(text);

			string? name = obj.GetValue("Name", null);
			if (name == null)
			{
				channel = null;
				return false;
			}

			string? streamName = obj.GetValue("StreamName", null);
			if (streamName == null)
			{
				channel = null;
				return false;
			}

			string type = obj.GetValue("Type", IArchiveChannel.EditorArchiveType);
			string archiveType = obj.GetValue("ArchiveType", "ugs-pcb");
			bool bIgnoreRequiredBadges = obj.GetValue("bIgnoreRequiredBadges", false);

			channel = new CloudArchiveChannel(name, type, archiveType, streamName, bIgnoreRequiredBadges);
			return true;
		}
	}

	public static class BaseArchive
	{
		public static readonly HashSet<string> DefaultArchiveSources = new HashSet<string>(StringComparer.OrdinalIgnoreCase) { "perforce", "cloud", "horde" };

		public static async Task<List<BaseArchiveChannel>> EnumerateChannelsAsync(
			IPerforceConnection perforce, 
			IHordeClient? hordeClient, 
			ICloudStorage? cloudStorage, 
			ConfigFile latestProjectConfigFile, 
			string projectIdentifier,
			HashSet<string> archiveSources,
			CancellationToken cancellationToken)
		{
			List<BaseArchiveChannel> newArchives = new List<BaseArchiveChannel>();

			if (archiveSources.Contains("perforce"))
			{
				newArchives.AddRange(await PerforceArchiveChannel.GetChannelsAsync(perforce, latestProjectConfigFile, projectIdentifier, cancellationToken));
			}

			// prefer Cloud Storage pcbs over horde artifacts
			if (archiveSources.Contains("cloud"))
			{
				if (cloudStorage != null && cloudStorage.IsEnabled(latestProjectConfigFile))
				{
					newArchives.AddRange(await CloudArchiveChannel.GetChannelsAsync(cloudStorage, latestProjectConfigFile, projectIdentifier, cancellationToken));
				}
			}

			if (archiveSources.Contains("horde"))
			{
				if (hordeClient != null)
				{
					newArchives.AddRange(await HordeArchiveChannel.GetChannelsAsync(hordeClient, projectIdentifier, cancellationToken));
				}
			}

			return newArchives;
		}
	}
}
