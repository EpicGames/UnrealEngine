// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.IO.Compression;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	[DebuggerDisplay("{FileName}")]
	public class ArchiveManifestFile
	{
		public string FileName { get; }
		public long Length { get; }
		public DateTime LastWriteTimeUtc { get; }

		public ArchiveManifestFile(BinaryReader reader)
		{
			FileName = reader.ReadString();
			Length = reader.ReadInt64();
			LastWriteTimeUtc = new DateTime(reader.ReadInt64());
		}

		public ArchiveManifestFile(string fileName, long length, DateTime lastWriteTimeUtc)
		{
			FileName = fileName;
			Length = length;
			LastWriteTimeUtc = lastWriteTimeUtc;
		}

		public void Write(BinaryWriter writer)
		{
			writer.Write(FileName);
			writer.Write(Length);
			writer.Write(LastWriteTimeUtc.Ticks);
		}
	}

	public class ArchiveManifest
	{
		const int Signature = ((int)'U' << 24) | ((int)'A' << 16) | ((int)'M' << 8) | 1;

		public List<ArchiveManifestFile> Files { get; } = new List<ArchiveManifestFile>();

		public ArchiveManifest()
		{
		}

		public ArchiveManifest(FileStream inputStream)
		{
			using BinaryReader reader = new BinaryReader(inputStream, Encoding.UTF8, true);
			if (reader.ReadInt32() != Signature)
			{
				throw new Exception("Archive manifest signature does not match");
			}

			int numFiles = reader.ReadInt32();
			for (int idx = 0; idx < numFiles; idx++)
			{
				Files.Add(new ArchiveManifestFile(reader));
			}
		}

		public void Write(FileStream outputStream)
		{
			using BinaryWriter writer = new BinaryWriter(outputStream, Encoding.UTF8, true);
			writer.Write(Signature);
			writer.Write(Files.Count);
			foreach (ArchiveManifestFile file in Files)
			{
				file.Write(writer);
			}
		}
	}

	public static class ArchiveUtils
	{
		public static void ExtractFiles(FileReference archiveFileName, DirectoryReference baseDirectoryName, FileReference? manifestFileName, ProgressValue progress, ILogger logger)
		{
			DateTime timeStamp = DateTime.UtcNow;
			using (ZipArchive zip = new ZipArchive(File.OpenRead(archiveFileName.FullName)))
			{
				if (manifestFileName != null)
				{
					FileReference.Delete(manifestFileName);

					// Create the manifest
					ArchiveManifest manifest = new ArchiveManifest();
					foreach (ZipArchiveEntry entry in zip.Entries)
					{
						if (!entry.FullName.EndsWith("/", StringComparison.Ordinal) && !entry.FullName.EndsWith("\\", StringComparison.Ordinal))
						{
							manifest.Files.Add(new ArchiveManifestFile(entry.FullName, entry.Length, timeStamp));
						}
					}

					// Write it out to a temporary file, then move it into place
					FileReference tempManifestFileName = manifestFileName + ".tmp";
					using (FileStream outputStream = FileReference.Open(tempManifestFileName, FileMode.Create, FileAccess.Write))
					{
						manifest.Write(outputStream);
					}
					FileReference.Move(tempManifestFileName, manifestFileName);
				}

				// Extract all the files
				int entryIdx = 0;
				foreach (ZipArchiveEntry entry in zip.Entries)
				{
					if (!entry.FullName.EndsWith("/", StringComparison.Ordinal) && !entry.FullName.EndsWith("\\", StringComparison.Ordinal))
					{
						FileReference fileName = FileReference.Combine(baseDirectoryName, entry.FullName);
						DirectoryReference.CreateDirectory(fileName.Directory);
						logger.LogInformation("Writing {FileName}", fileName);

						entry.ExtractToFile(fileName.FullName, true);
						FileReference.SetLastWriteTimeUtc(fileName, timeStamp);
					}
					progress.Set((float)++entryIdx / (float)zip.Entries.Count);
				}
			}
		}

		public static void RemoveExtractedFiles(DirectoryReference baseDirectoryName, FileReference manifestFileName, ProgressValue progress, ILogger logWriter)
		{
			// Read the manifest in
			ArchiveManifest manifest;
			using (FileStream inputStream = FileReference.Open(manifestFileName, FileMode.Open, FileAccess.Read))
			{
				manifest = new ArchiveManifest(inputStream);
			}

			// Remove all the files that haven't been modified match
			for (int idx = 0; idx < manifest.Files.Count; idx++)
			{
				FileInfo file = FileReference.Combine(baseDirectoryName, manifest.Files[idx].FileName).ToFileInfo();
				if (file.Exists)
				{
					if (file.Length != manifest.Files[idx].Length)
					{
						logWriter.LogInformation("Skipping {FileName} due to modified length", file.FullName);
					}
					else if (Math.Abs((file.LastWriteTimeUtc - manifest.Files[idx].LastWriteTimeUtc).TotalSeconds) > 2.0)
					{
						logWriter.LogInformation("Skipping {FileName} due to modified timestamp", file.FullName);
					}
					else
					{
						logWriter.LogInformation("Removing {FileName}", file.FullName);
						file.Delete();
					}
				}
				progress.Set((float)(idx + 1) / (float)manifest.Files.Count);
			}
		}
	}
}
