// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.RegularExpressions;
using System.Threading.Tasks;

using EpicGames.Perforce;
using Microsoft.Extensions.Logging;
using ILogger = Microsoft.Extensions.Logging.ILogger;

namespace UnrealGameSync
{
	/// <summary>
	/// Defines a contract for retrieving an archive changelist number that may override the provided changelist.
	/// </summary>
	/// <remarks>Implementations of this interface can be used to determine if a different changelist should be used
	/// in place of the specified one, such as for custom archiving or versioning scenarios. The returned changelist may be
	/// the same as the input or a different value, depending on the override logic.</remarks>
	public interface IArchiveOverride
	{
		Task<int> GetChangelistAsync(int changelist);
	}

	/// <summary>
	/// Provides a default implementation of the IArchiveOverride interface that returns the specified changelist without
	/// modification.
	/// </summary>
	public class DefaultArchiveOverride : IArchiveOverride
	{
		public Task<int> GetChangelistAsync(int changelist)
		{
			return Task.FromResult<int>(changelist);
		}
	}

	/// <summary>
	/// Provides an implementation of the IArchiveOverride interface that determines the changelist override by parsing the
	/// Perforce stream description.
	/// </summary>
	/// <remarks>This class is typically used to extract a changelist number from the description of a Perforce
	/// stream, allowing for dynamic override of the changelist based on stream metadata. It requires valid Perforce
	/// settings and a logger to function. Thread safety depends on the usage of the provided logger and Perforce
	/// connection objects.</remarks>
	public partial class ArchiveOverrideFromStreamDescription : IArchiveOverride
	{
		private readonly IPerforceSettings _perforceSettings;
		private readonly ILogger _logger;

		[GeneratedRegex("^\\s*import\\s+\\.\\.\\.")]
		private static partial Regex ImportRegex();

		[GeneratedRegex("(//[^@]+)@(\\d+)")]
		private static partial Regex ImportCaptureRegex();

		public ArchiveOverrideFromStreamDescription(IPerforceSettings perforceSettings, ILogger logger)
		{
			_perforceSettings = perforceSettings;
			_logger = logger;
		}

		public async Task<int> GetChangelistAsync(int changelist)
		{
			using IPerforceConnection perforce = await PerforceConnection.CreateAsync(_perforceSettings, _logger);

			// Get the stream name for the current workspace
			string? streamName = await perforce.GetCurrentStreamAsync();

			if (String.IsNullOrWhiteSpace(streamName))
			{
				_logger.LogWarning("No stream specified for ArchiveOverrideFromStreamDescription");
				return changelist;
			}

			string changelistFromStreamDescription = String.Empty;
			string streamPath = $"{streamName}@{changelist}";
			PerforceResponse<StreamRecord> streamRecordResponse = await perforce.TryGetStreamAsync(streamPath, true);

			if (streamRecordResponse.Succeeded)
			{
				foreach (string path in streamRecordResponse.Data.Paths)
				{
					if (ImportRegex().IsMatch(path))
					{
						Match match = ImportCaptureRegex().Match(path);
						if (match.Success)
						{
							changelistFromStreamDescription = match.Groups[2].Value;
							break;
						}
					}
				}

				if (String.IsNullOrWhiteSpace(changelistFromStreamDescription) || !Int32.TryParse(changelistFromStreamDescription, out int changelistNumberFromStreamDescription))
				{
					_logger.LogWarning("No valid changelist override found in stream description for ArchiveOverrideFromStreamDescription");
					return changelist;
				}

				return changelistNumberFromStreamDescription;
			}

			return changelist;
		}
	}

	/// <summary>
	/// Provides factory methods for creating instances of archive override implementations.
	/// </summary>
	public static class ArchiveOverrideFactory
	{
		public static IArchiveOverride Create(string archiveOverride, IPerforceSettings perforceSettings, Microsoft.Extensions.Logging.ILogger logger)
		{
			switch (archiveOverride)
			{
				case "FromStreamDescription":
					return new ArchiveOverrideFromStreamDescription(perforceSettings, logger);
				default:
					return new DefaultArchiveOverride();

			}
		}
	}
}
