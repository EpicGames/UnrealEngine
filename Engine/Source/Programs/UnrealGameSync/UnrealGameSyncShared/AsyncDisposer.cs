// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	public interface IAsyncDisposer
	{
		void Add(Task task);
	}

	public class AsyncDisposer : IAsyncDisposer, IAsyncDisposable
	{
		readonly object _lockObject = new object();
		readonly List<Task> _tasks = new List<Task>();
		readonly ILogger _logger;

		public AsyncDisposer(ILogger<AsyncDisposer> logger)
		{
			_logger = logger;
		}

		public void Add(Task task)
		{
			Task continuationTask = task.ContinueWith(Remove, TaskScheduler.Default);
			lock (_lockObject)
			{
				_tasks.Add(continuationTask);
			}
		}

		private void Remove(Task task)
		{
			if (task.IsFaulted)
			{
				_logger.LogError(task.Exception, "Exception while disposing task");
			}
		}

		public async ValueTask DisposeAsync()
		{
			List<Task> tasksCopy;
			lock (_lockObject)
			{
				tasksCopy = new List<Task>(_tasks);
			}

			Task waitTask = Task.WhenAll(tasksCopy);
			while (!waitTask.IsCompleted)
			{
				_logger.LogInformation("Waiting for {NumTasks} tasks to complete", tasksCopy.Count);
				await Task.WhenAny(waitTask, Task.Delay(TimeSpan.FromSeconds(5.0)));
			}
			
			GC.SuppressFinalize(this);
		}
	}
}