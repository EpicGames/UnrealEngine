// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;

namespace UnrealGameSync
{
	public enum BuildStepType
	{
		Compile,
		Cook,
		Other,
	}

	[DebuggerDisplay("{Description}")]
	public class BuildStep
	{
		public const string UniqueIdKey = "UniqueId";

		public Guid UniqueId { get; set; }
		public int OrderIndex { get; set; }
		public string? Description { get; set; }
		public string? StatusText { get; set; }
		public int EstimatedDuration { get; set; }
		public BuildStepType Type { get; set; }
		public string? Target { get; set; }
		public string? Platform { get; set; }
		public string? Configuration { get; set; }
		public string? FileName { get; set; }
		public string? WorkingDir { get; set; }
		public string? Arguments { get; set; }
		public bool UseLogWindow { get; set; }
		public bool NormalSync { get; set; }
		public bool ScheduledSync { get; set; }
		public string? StatusPanelLink { get; set; }
		public List<Guid> Requires { get; } = new List<Guid>();
		public List<Guid> LocalCompileRequires { get; } = new List<Guid>();
		public Guid ToolId { get; set; }

		public BuildStep(Guid inUniqueId, int inOrderIndex, string? inDescription, string? inStatusText, int inEstimatedDuration, string? inFileName, string? inArguments, string? inWorkingDir, bool inUseLogWindow)
		{
			UniqueId = inUniqueId;
			OrderIndex = inOrderIndex;
			Description = inDescription;
			StatusText = inStatusText;
			EstimatedDuration = inEstimatedDuration;
			Type = BuildStepType.Other;
			FileName = inFileName;
			Arguments = inArguments;
			WorkingDir = inWorkingDir;
			UseLogWindow = inUseLogWindow;
		}

		public BuildStep(Guid inUniqueId, int inOrderIndex, string? inDescription, string? inStatusText, int inEstimatedDuration, string? inTarget, string? inPlatform, string? inConfiguration, string? inArguments, bool inSyncDefault)
		{
			UniqueId = inUniqueId;
			OrderIndex = inOrderIndex;
			Description = inDescription;
			StatusText = inStatusText;
			EstimatedDuration = inEstimatedDuration;
			Type = BuildStepType.Compile;
			Target = inTarget;
			Platform = inPlatform;
			Configuration = inConfiguration;
			Arguments = inArguments;
			UseLogWindow = true;
			NormalSync = inSyncDefault;
			ScheduledSync = inSyncDefault;
		}

		public BuildStep(ConfigObject obj)
		{
			UniqueId = obj.GetValue(UniqueIdKey, Guid.NewGuid());
			OrderIndex = obj.GetValue("OrderIndex", -1);
			Description = obj.GetValue("Description", "Untitled");
			StatusText = obj.GetValue("StatusText", "Untitled");
			EstimatedDuration = Math.Max(obj.GetValue("EstimatedDuration", 1), 1);

			BuildStepType stepType;
			if (!Enum.TryParse(obj.GetValue("Type", ""), true, out stepType))
			{
				stepType = BuildStepType.Other;
			}
			Type = stepType;

			Target = obj.GetValue("Target");
			Platform = obj.GetValue("Platform");
			Configuration = obj.GetValue("Configuration");
			FileName = obj.GetValue("FileName");
			WorkingDir = obj.GetValue("WorkingDir");
			Arguments = obj.GetValue("Arguments");

			UseLogWindow = obj.GetValue("bUseLogWindow", true);
			NormalSync = obj.GetValue("bNormalSync", true);
			ScheduledSync = obj.GetValue("bScheduledSync", NormalSync);

			StatusPanelLink = obj.GetValue("Link", null);
			if (String.IsNullOrEmpty(StatusPanelLink))
			{
				bool showAsTool;
				if (Boolean.TryParse(obj.GetValue("bShowAsTool", ""), out showAsTool) && showAsTool)
				{
					StatusPanelLink = $"More...|{Description}";
				}
			}

			List<Guid> requires = new List<Guid>();
			foreach (string requireString in obj.GetValue("Requires", String.Empty).Split(';', StringSplitOptions.RemoveEmptyEntries))
			{
				if (Guid.TryParse(requireString, out Guid require))
				{
					requires.Add(require);
				}
			}
			Requires = requires;

			List<Guid> localCompileRequires = new List<Guid>();
			foreach (string localCompileRequireString in obj.GetValue("LocalCompileRequires", String.Empty).Split(';', StringSplitOptions.RemoveEmptyEntries))
			{
				if (Guid.TryParse(localCompileRequireString, out Guid localCompileRequire))
				{
					localCompileRequires.Add(localCompileRequire);
				}
			}
			LocalCompileRequires = localCompileRequires;

			ToolId = obj.GetValue("Tool", Guid.Empty);
		}

		public bool IsValid()
		{
			switch (Type)
			{
				case BuildStepType.Compile:
					return Target != null && Platform != null && Configuration != null;
				case BuildStepType.Cook:
					return FileName != null;
				case BuildStepType.Other:
					return FileName != null;
				default:
					return false;
			}
		}

		public static void MergeBuildStepObjects(Dictionary<Guid, ConfigObject> buildStepObjects, IEnumerable<ConfigObject> modifyObjects)
		{
			foreach (ConfigObject modifyObject in modifyObjects)
			{
				Guid uniqueId;
				if (Guid.TryParse(modifyObject.GetValue(UniqueIdKey, ""), out uniqueId))
				{
					ConfigObject? defaultObject;
					if (buildStepObjects.TryGetValue(uniqueId, out defaultObject))
					{
						modifyObject.SetDefaults(defaultObject);
					}
					buildStepObjects[uniqueId] = modifyObject;
				}
			}
		}

		public ConfigObject ToConfigObject()
		{
			ConfigObject result = new ConfigObject();
			result["UniqueId"] = UniqueId.ToString();
			result["Description"] = Description;
			result["StatusText"] = StatusText;
			result["EstimatedDuration"] = EstimatedDuration.ToString();
			result["Type"] = Type.ToString();
			switch (Type)
			{
				case BuildStepType.Compile:
					result["Target"] = Target;
					result["Platform"] = Platform;
					result["Configuration"] = Configuration;
					result["Arguments"] = Arguments;
					break;
				case BuildStepType.Cook:
					result["FileName"] = FileName;
					break;
				case BuildStepType.Other:
					result["FileName"] = FileName;
					result["WorkingDir"] = WorkingDir;
					result["Arguments"] = Arguments;
					result["bUseLogWindow"] = UseLogWindow.ToString();
					break;
			}
			result["OrderIndex"] = OrderIndex.ToString();
			result["bNormalSync"] = NormalSync.ToString();
			result["bScheduledSync"] = ScheduledSync.ToString();
			if (!String.IsNullOrEmpty(StatusPanelLink))
			{
				result["Link"] = StatusPanelLink;
			}
			if (ToolId != Guid.Empty)
			{
				result["Tool"] = ToolId.ToString();
			}
			return result;
		}

		public ConfigObject? ToConfigObject(ConfigObject? defaultObject)
		{
			ConfigObject result = new ConfigObject();
			result[UniqueIdKey] = UniqueId.ToString();
			result.AddOverrides(ToConfigObject(), defaultObject);
			return (result.Pairs.Count <= 1) ? null : result;
		}
	}
}
