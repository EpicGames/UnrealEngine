// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.IO;
using System.Reflection;
using System.Runtime.CompilerServices;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.OIDC;
using EpicGames.Serialization;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	/// <summary>
	/// A build found during search
	/// </summary>
	/// <param name="buildName"></param>
	/// <param name="buildId"></param>
	/// <param name="bucketId"></param>
	/// <param name="commit"></param>
	public class FoundBuildResponse(string buildName, string buildId, string bucketId, string commit)
	{
		public string BuildId { get; init; } = buildId;
		public string BucketId { get; init; } = bucketId;

		public string Commit { get; init; } = commit;
		public string Name { get; set; } = buildName;
	}

	/// <summary>
	/// Interface for interacting with UE Cloud Storage
	/// </summary>
	public interface ICloudStorage
	{
		/// <summary>
		/// True if the config file enables cloud storage
		/// </summary>
		/// <param name="configFile">Config file to lookup cloud storage settings in</param>
		/// <returns></returns>
		bool IsEnabled(ConfigFile configFile);

		/// <summary>
		/// Download a build from UE Cloud Storage based on an uri
		/// </summary>
		/// <param name="uri">The uri</param>
		/// <param name="outputDir">The directory to write the build to</param>
		/// <param name="progress">Progress reporting</param>
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		Task<bool> DownloadBuildFromUriAsync(Uri uri, DirectoryReference outputDir, IProgress<string> progress, CancellationToken cancellationToken);

		/// <summary>
		/// Download a build using all the required fields
		/// </summary>
		/// <param name="host">The host url</param>
		/// <param name="namespaceId">The namespace</param>
		/// <param name="bucketId">The bucket</param>
		/// <param name="buildId">The id of the build</param>
		/// <param name="outputDir">The directory to write the build to</param>
		/// <param name="zenStateDirectory">A optional directory were zen cli should create its state. In the output dir by default</param>
		/// <param name="progress">Progress reporting</param>
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		Task<bool> DownloadBuildAsync(string host, string namespaceId, string bucketId, string buildId, DirectoryReference outputDir, DirectoryReference? zenStateDirectory, IProgress<string> progress, CancellationToken cancellationToken);

		/// <summary>
		/// Search for builds in UE Cloud Storage using a regular expression to describe which buckets the build might be in
		/// </summary>
		/// <param name="host">The host url</param>
		/// <param name="namespaceId">The namespace</param>
		/// <param name="bucketRegex">A regex that describe which buckets the build might be in</param>
		/// <param name="query">CbObject that describes the query, see UE Cloud Storage documentation</param>
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		IAsyncEnumerable<FoundBuildResponse> FindBuildAsync(string host, string namespaceId, Regex bucketRegex, CbObject query, CancellationToken cancellationToken);
	}

	class CloudStorage(ILogger<CloudStorage> logger, ITokenStore tokenStore) : ICloudStorage, IDisposable
	{
		private OidcTokenManager? _tokenManager;
		private string? _providerName;

		private readonly SemaphoreSlim _createOidcTokenSemaphore = new SemaphoreSlim(1, 1);
		public bool IsEnabled(ConfigFile configFile)
		{
			ConfigSection? section = configFile.FindSection("CloudStorage");
			if (section == null)
			{
				return false;
			}

			return section.GetValue("Enabled", false);
		}

		public async Task CreateOidcTokenManagerAsync(Uri uri)
		{
			ClientAuthConfigurationV1? remoteAuthConfig = await ProviderConfigurationFactory.ReadRemoteAuthConfigurationAsync(uri, ProviderConfigurationFactory.DefaultEncryptionKey);
			if (remoteAuthConfig == null)
			{
				throw new NotImplementedException("Unable to fetch remote auth configuration");
			}

			IConfiguration config = ProviderConfigurationFactory.BindOptions(remoteAuthConfig);

			_tokenManager = OidcTokenManager.CreateTokenManager(config, tokenStore);
			_providerName = remoteAuthConfig.DefaultProvider;
		}

		public async Task<bool> DownloadBuildFromUriAsync(Uri uri, DirectoryReference outputDir, IProgress<string> progress, CancellationToken cancellationToken)
		{
			string host = $"{uri.Scheme}://{uri.Host}";
			string path = uri.AbsolutePath;
			// remove api path if specified
			path = path.Replace("api/v2/builds/", "", StringComparison.OrdinalIgnoreCase);
			string[] components = path.Split('/', StringSplitOptions.RemoveEmptyEntries);
			if (components.Length != 3)
			{
				logger.LogError("Failed to base uri path {Uri}, did not match expected format of namespace/bucket/buildId", path);
				return true;
			}

			string namespaceId = components[0];
			string bucketId = components[1];
			string buildId = components[2];	

			return await DownloadBuildAsync(host, namespaceId, bucketId, buildId, outputDir, zenStateDirectory: null, progress, cancellationToken);
		}

		public async Task<bool> DownloadBuildAsync(string host, string namespaceId, string bucketId, string buildId, DirectoryReference outputDir, DirectoryReference? zenStateDirectory, IProgress<string> progress, CancellationToken cancellationToken)
		{
			string? accessToken = await GetAccessToken(new Uri(host));

			if (accessToken == null)
			{
				logger.LogError("Unable to determine ugs root directory so unable to find zen executable.");
				return true;
			}

			string zenStateArg = zenStateDirectory == null ? "" : $" --zen-folder-path {zenStateDirectory} ";
			string cmdline = $"builds download --host {host} --namespace {namespaceId} --bucket {bucketId} {zenStateArg} --plain-progress \"{outputDir}\" {buildId}";

			return await RunZenCli(cmdline, accessToken, progress, cancellationToken);
		}

		public async Task<bool> RunZenCli(string cmdline, string accessToken, IProgress<string> progress, CancellationToken cancellationToken)
		{
			string accessTokenEnvVar = "UE-CloudDataCacheAccessToken";
			// pass the access token via environment variable to avoid showing it on the cli
			cmdline += $" --access-token-env {accessTokenEnvVar} ";

			progress.Report("Connecting to server...");

			string? ugsDir = Path.GetDirectoryName(Assembly.GetEntryAssembly()?.Location);
			if (ugsDir == null)
			{
				logger.LogError("Unable to determine ugs root directory so unable to find zen executable.");
				return true;
			}
			FileInfo zenExe;
			switch (RuntimePlatform.Current)
			{
				case RuntimePlatform.Type.Windows:
					zenExe = new FileInfo(Path.Combine(ugsDir, "Binaries/Win64/zen.exe"));
					break;
				case RuntimePlatform.Type.Linux:
					zenExe = new FileInfo(Path.Combine(ugsDir, "Binaries/Linux/zen"));
					break;
				case RuntimePlatform.Type.Mac:
					zenExe = new FileInfo(Path.Combine(ugsDir, "Binaries/Mac/zen"));
					break;
				default:
					throw new ArgumentOutOfRangeException($"Unknown runtime platform {RuntimePlatform.Current}");
			}

			if (!zenExe.Exists)
			{
				logger.LogError("Unable to locate zen executable at {Path} unable to run it.", zenExe);
				return true;
			}

			// we need to propagate the environment while also adding the access token we want to override
			Dictionary<string, string> processEnv = new Dictionary<string, string>();
			foreach (DictionaryEntry entry in Environment.GetEnvironmentVariables())
			{
				if (entry is { Key: string k, Value: string v })
				{
					processEnv[k] = v;
				}
			}

			processEnv[accessTokenEnvVar] = accessToken;

			logger.LogInformation("Running zen cli '{ZenPath} {ZenArgs}' to download build.", zenExe.FullName, cmdline);

			int exitCode = await Utility.ExecuteProcessAsync(zenExe.FullName, zenExe.DirectoryName, cmdline, s =>
			{
				progress.Report(s);
				logger.LogInformation("{Line}", s);
			}, processEnv, cancellationToken);
			if (exitCode != 0)
			{
				logger.LogError("Error running {App} {Args} shutdown with exitcode {ExitCode}", zenExe, cmdline, exitCode);
			}

			return exitCode != 0;
		}

		public async IAsyncEnumerable<FoundBuildResponse> FindBuildAsync(string host, string namespaceId, Regex bucketRegex, CbObject query, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			string? accessToken = await GetAccessToken(new Uri(host), allowInteractiveLogin: false);

			if (accessToken == null)
			{
				throw new Exception("Unable to list builds as no access token was found");
			}

			string tempInputFile = Path.GetTempFileName();
			string tempOutputFile = Path.GetTempFileName();
			tempInputFile = Path.ChangeExtension(tempInputFile, "cbo");
			tempOutputFile = Path.ChangeExtension(tempOutputFile, "cbo");
			try
			{
				// copy the query and append the bucket regex filter
				CbWriter writer = new CbWriter();
				writer.BeginObject();
				writer.WriteObject("query", query);
				writer.WriteString("bucketRegex", bucketRegex.ToString()!);
				writer.EndObject();

				await File.WriteAllBytesAsync(tempInputFile, writer.ToByteArray(), cancellationToken);
				string cmdline = $"builds list --host {host} --namespace {namespaceId} --query-path \"{tempInputFile}\" --result-path \"{tempOutputFile}\"";

				Progress<string> progress = new Progress<string>();
				bool failed = await RunZenCli(cmdline, accessToken, progress, cancellationToken);

				if (failed)
				{
					logger.LogError("Failed to list builds, zen non-zero exit code");
					yield break;
				}
				byte[] b = await File.ReadAllBytesAsync(tempOutputFile, cancellationToken);
				CbObject o = new CbObject(b);

				foreach (CbField result in o["results"].AsArray())
				{
					CbObject metadata = result["metadata"].AsObject();
					if (metadata.Equals(CbObject.Empty))
					{
						// we require metadata to be able to know which commit this build belongs to
						continue;
					}

					string buildName = metadata["name"].AsString();
					string bucketId = result["bucketId"].AsString();
					string buildId = result["buildId"].AsString();
					CbField commitField = metadata["commit"];
					if (commitField.Equals(CbField.Empty))
					{
						// we require metadata to be able to know which commit this build belongs to
						continue;
					}

					string commit = "";
					if (commitField.IsInteger())
					{
						commit = commitField.AsInt64().ToString();

					}
					if (commitField.IsFloat())
					{
						// due to jsons problematic number types we sometimes get doubles for integers, these are likely changelists and thus whole numbers so we just cast them
						commit = ((long)commitField.AsDouble()).ToString();
					}
					else if (commitField.IsString())
					{
						commit = commitField.AsString();
					}
					
					if (String.IsNullOrEmpty(commit) )
					{
						// if there is no commit specified we can not know which changelist this PCB belong to
						continue;
					}
					yield return new FoundBuildResponse(buildName, buildId, bucketId, commit);
				}
			}
			finally
			{
				if (File.Exists(tempInputFile))
				{
					File.Delete(tempInputFile);
				}

				if (File.Exists(tempOutputFile))
				{
					File.Delete(tempOutputFile);
				}
			}
		}

		private async Task<string?> GetAccessToken(Uri uri, bool allowInteractiveLogin = true)
		{
			if (_tokenManager == null)
			{
				await _createOidcTokenSemaphore.WaitAsync(); 
				try
				{
					// check to see if it wasn't created while we waited for the semaphore
#pragma warning disable CA1508
					if (_tokenManager == null)
#pragma warning restore CA1508
					{
						await CreateOidcTokenManagerAsync(uri); 
					}
				}
				finally
				{
					_createOidcTokenSemaphore.Release();
				}
			}

			IOidcTokenInfo tokenInfo;
			try
			{
				tokenInfo = await _tokenManager!.GetAccessToken(_providerName!, CancellationToken.None);
			}
			catch (NotLoggedInException)
			{
				if (allowInteractiveLogin)
				{
					tokenInfo = await _tokenManager!.LoginAsync(_providerName!, CancellationToken.None);
				}
				else
				{
					logger.LogError("Failed to get a access token");
					return null;
				}
			}
			return tokenInfo.AccessToken;
		}

		private void Dispose(bool disposing)
		{
			if (disposing)
			{
				_createOidcTokenSemaphore.Dispose();
			}
		}

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		~CloudStorage()
		{
			Dispose(false);
		}
	}

	/// <summary>
	/// Extension methods for Cloud Storage
	/// </summary>
	public static class CloudStorageExtensions
	{
		/// <summary>
		/// Adds Horde-related services with the default settings
		/// </summary>
		/// <param name="serviceCollection">Collection to register services with</param>
		public static void AddCloudStorage(this IServiceCollection serviceCollection)
		{
			serviceCollection.AddLogging();

			serviceCollection.AddSingleton<ITokenStore>(sp => TokenStoreFactory.CreateTokenStore());
			serviceCollection.AddSingleton<ICloudStorage, CloudStorage>();
		}

		/// <summary>
		/// Adds Horde-related services
		/// </summary>
		/// <param name="serviceCollection">Collection to register services with</param>
		/// <param name="configureHorde">Callback to configure options</param>
		public static void AddCloudStorage(this IServiceCollection serviceCollection, Action<CloudStorageOptions> configureHorde)
		{
			serviceCollection.Configure(configureHorde);
			AddCloudStorage(serviceCollection);
		}
	}

	public class CloudStorageOptions
	{

	}
}
