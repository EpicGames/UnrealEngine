// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.IO.Enumeration;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	[JsonConverter(typeof(ConfigObjectJsonConverter))]
	public class ConfigObject
	{
		const string ConfigSeparatorCharacters = "(),= \t\"";

		public List<KeyValuePair<string, string>> Pairs { get; }

		public ConfigObject()
		{
			Pairs = new List<KeyValuePair<string, string>>();
		}

		public ConfigObject(ConfigObject other)
		{
			Pairs = new List<KeyValuePair<string, string>>(other.Pairs);
		}

		public ConfigObject(string text)
		{
			Pairs = new List<KeyValuePair<string, string>>();
			ParseConfigString(text);
		}

		public ConfigObject(ConfigObject baseObject, string text)
		{
			Pairs = new List<KeyValuePair<string, string>>(baseObject.Pairs);
			ParseConfigString(text);
		}

		void ParseConfigString(string text)
		{
			int idx = 0;
			if (ParseConfigToken(text, ref idx) == "(")
			{
				while (idx < text.Length)
				{
					// Read the next key/value pair
					string? key = ParseConfigToken(text, ref idx);
					if (key == null)
					{
						return;
					}
					if (ParseConfigToken(text, ref idx) == "=")
					{
						string? value = ParseConfigValueToken(text, ref idx);
						if (value == null)
						{
							return;
						}
						SetValue(key, value);
					}

					// Check for the end of the list, or a comma before the next pair
					for (; ; )
					{
						string? token = ParseConfigValueToken(text, ref idx);
						if (token == ",")
						{
							break;
						}
						if (token == ")" || token == null)
						{
							return;
						}
					}
				}
			}
		}

		static string? ParseConfigToken(string text, ref int idx)
		{
			// Skip whitespace
			while (idx < text.Length && Char.IsWhiteSpace(text[idx]))
			{
				idx++;
			}
			if (idx == text.Length)
			{
				return null;
			}

			// Read the token
			if (text[idx] == '\"')
			{
				StringBuilder token = new StringBuilder();
				while (++idx < text.Length)
				{
					if (text[idx] == '\"')
					{
						idx++;
						break;
					}
					if (text[idx] == '\\' && idx + 1 < text.Length)
					{
						idx++;
					}
					token.Append(text[idx]);
				}
				return token.ToString();
			}
			else if (ConfigSeparatorCharacters.Contains(text[idx], StringComparison.Ordinal))
			{
				return text[idx++].ToString();
			}
			else
			{
				int startIdx = idx;
				while (idx < text.Length && !ConfigSeparatorCharacters.Contains(text[idx], StringComparison.Ordinal))
				{
					idx++;
				}
				return text.Substring(startIdx, idx - startIdx);
			}
		}

		static string? ParseConfigValueToken(string text, ref int idx)
		{
			string? token = ParseConfigToken(text, ref idx);
			if (token == "(")
			{
				int startIdx = idx - 1;
				for (; ; )
				{
					string? nextToken = ParseConfigValueToken(text, ref idx);
					if (nextToken == null || nextToken == ")")
					{
						break;
					}
				}
				token = text.Substring(startIdx, idx - startIdx);
			}
			return token;
		}

		public IEnumerable<string> GetKeys()
		{
			for (int idx = 0; idx < Pairs.Count; idx++)
			{
				yield return Pairs[idx].Key;
			}
		}

		[return: NotNullIfNotNull("defaultValue")]
		public string? GetValue(string key, string? defaultValue = null)
		{
			for (int idx = 0; idx < Pairs.Count; idx++)
			{
				if (Pairs[idx].Key.Equals(key, StringComparison.OrdinalIgnoreCase))
				{
					return Pairs[idx].Value;
				}
			}
			return defaultValue;
		}

		public Guid GetValue(string key, Guid defaultValue)
		{
			string? stringValue = GetValue(key);
			if (stringValue != null)
			{
				Guid value;
				if (Guid.TryParse(stringValue, out value))
				{
					return value;
				}
			}
			return defaultValue;
		}

		public int GetValue(string key, int defaultValue)
		{
			string? stringValue = GetValue(key);
			if (stringValue != null)
			{
				int value;
				if (Int32.TryParse(stringValue, out value))
				{
					return value;
				}
			}
			return defaultValue;
		}

		public bool GetValue(string key, bool defaultValue)
		{
			string? stringValue = GetValue(key);
			if (stringValue != null)
			{
				bool value;
				if (Boolean.TryParse(stringValue, out value))
				{
					return value;
				}
			}
			return defaultValue;
		}

		public void SetValue(string key, string? value)
		{
			if (value == null)
			{
				Pairs.RemoveAll(x => x.Key.Equals(key, StringComparison.OrdinalIgnoreCase));
			}
			else
			{
				for (int idx = 0; idx < Pairs.Count; idx++)
				{
					if (Pairs[idx].Key.Equals(key, StringComparison.OrdinalIgnoreCase))
					{
						Pairs[idx] = new KeyValuePair<string, string>(key, value);
						return;
					}
				}
				Pairs.Add(new KeyValuePair<string, string>(key, value));
			}
		}

		public void SetValue(string key, Guid value)
		{
			SetValue(key, value.ToString());
		}

		public void SetValue(string key, int value)
		{
			SetValue(key, value.ToString());
		}

		public void SetValue(string key, bool value)
		{
			SetValue(key, value.ToString());
		}

		public string? this[string key]
		{
			get => GetValue(key);
			set => SetValue(key, value);
		}

		public void SetDefaults(ConfigObject other)
		{
			foreach (KeyValuePair<string, string> pair in other.Pairs)
			{
				if (GetValue(pair.Key) == null)
				{
					SetValue(pair.Key, pair.Value);
				}
			}
		}

		public void AddOverrides(ConfigObject obj, ConfigObject? defaultObject)
		{
			foreach (KeyValuePair<string, string> pair in obj.Pairs)
			{
				if (defaultObject == null || defaultObject.GetValue(pair.Key) != pair.Value)
				{
					SetValue(pair.Key, pair.Value);
				}
			}
		}

		public string ToString(ConfigObject? baseObject)
		{
			StringBuilder result = new StringBuilder();
			result.Append('(');
			foreach (KeyValuePair<string, string> pair in Pairs)
			{
				if (baseObject == null || baseObject.GetValue(pair.Key) != pair.Value)
				{
					if (result.Length > 1)
					{
						result.Append(", ");
					}
					result.Append(pair.Key);
					result.Append('=');
					if (pair.Value == null)
					{
						result.Append("\"\"");
					}
					else
					{
						result.AppendFormat("\"{0}\"", pair.Value.Replace("\\", "\\\\", StringComparison.Ordinal).Replace("\"", "\\\"", StringComparison.Ordinal));
					}
				}
			}
			result.Append(')');
			return result.ToString();
		}

		public override string ToString()
		{
			return ToString(null);
		}
	}

	public class ConfigObjectJsonConverter : JsonConverter<ConfigObject>
	{
		public override ConfigObject Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			ConfigObject obj = new ConfigObject();
			if (reader.TokenType == JsonTokenType.StartObject)
			{
				while (reader.Read() && reader.TokenType == JsonTokenType.PropertyName)
				{
					string name = reader.GetString()!;
					reader.Read();
					string value = reader.GetString()!;
					obj.Pairs.Add(KeyValuePair.Create(name, value));
				}
			}
			else
			{
				reader.Skip();
			}
			return obj;
		}

		public override void Write(Utf8JsonWriter writer, ConfigObject value, JsonSerializerOptions options)
		{
			writer.WriteStartObject();
			foreach (KeyValuePair<string, string> pair in value.Pairs)
			{
				writer.WriteString(pair.Key, pair.Value);
			}
			writer.WriteEndObject();
		}
	}

	[DebuggerDisplay("{Name}")]
	public class ConfigSection
	{
		public string Name { get; set; }
		public Dictionary<string, string> Pairs { get; } = new Dictionary<string, string>();

		private readonly HashSet<string> _wildCardKeys = new HashSet<string>();

		public ConfigSection(string inName)
		{
			Name = inName;
		}

		public void Clear()
		{
			Pairs.Clear();
			_wildCardKeys.Clear();
		}

		public void SetValue(string key, int value)
		{
			Pairs[key] = value.ToString();
		}

		public void SetValue(string key, long value)
		{
			Pairs[key] = value.ToString();
		}

		public void SetValue(string key, bool value)
		{
			Pairs[key] = value ? "1" : "0";
		}

		public void SetValue(string key, string value, bool fromWildCard = false)
		{
			if (value == null)
			{
				RemoveValue(key);
			}
			else
			{
				Pairs[key] = value;
			}

			if (fromWildCard)
			{
				_wildCardKeys.Add(key);
			}
			else
			{
				_wildCardKeys.Remove(key);
			}
		}

		public bool HasWildCardValue(string key)
		{
			return _wildCardKeys.Contains(key);
		}

		public void SetValues(string key, string[] values)
		{
			if (values == null || values.Length == 0)
			{
				RemoveValue(key);
			}
			else
			{
				Pairs[key] = String.Join("\n", values);
			}
		}

		public void SetValues(string key, Guid[] values)
		{
			if (values == null)
			{
				RemoveValue(key);
			}
			else
			{
				Pairs[key] = String.Join("\n", values.Select(x => x.ToString()));
			}
		}

		public void AppendValue(string key, string value)
		{
			string? currentValue;
			if (Pairs.TryGetValue(key, out currentValue))
			{
				Pairs[key] = currentValue + "\n" + value;
			}
			else
			{
				Pairs[key] = value;
			}
		}

		public void RemoveValue(string key)
		{
			Pairs.Remove(key);
		}

		public int GetValue(string key, int defaultValue) => GetOptionalIntValue(key, defaultValue) ?? defaultValue;

		public int? GetOptionalIntValue(string key, int? defaultValue)
		{
			string? valueString = GetValue(key);
			if (valueString != null)
			{
				int value;
				if (Int32.TryParse(valueString, out value))
				{
					return value;
				}
			}
			return defaultValue;
		}

		public long GetValue(string key, long defaultValue)
		{
			string? valueString = GetValue(key);
			if (valueString != null)
			{
				long value;
				if (Int64.TryParse(valueString, out value))
				{
					return value;
				}
			}
			return defaultValue;
		}

		public bool GetValue(string key, bool defaultValue)
		{
			return GetValue(key, defaultValue ? 1 : 0) != 0;
		}

		[return: NotNullIfNotNull("defaultValue")]
		public string? GetValue(string key, string? defaultValue = null)
		{
			string? value;
			if (!Pairs.TryGetValue(key, out value))
			{
				value = defaultValue;
			}
			return value;
		}

		public TEnum GetEnumValue<TEnum>(string key, TEnum defaultValue) where TEnum : struct
		{
			string? str = GetValue(key, null);
			if (str != null && Enum.TryParse(str, true, out TEnum value))
			{
				return value;
			}
			return defaultValue;
		}

		[return: NotNullIfNotNull("defaultValue")]
		public string[]? GetValues(string key, string[]? defaultValue = null)
		{
			string? value = GetValue(key, null);
			if (value == null)
			{
				return defaultValue;
			}
			else
			{
				return value.Split('\n');
			}
		}

		[return: NotNullIfNotNull("defaultValue")]
		public Guid[]? GetValues(string key, Guid[]? defaultValue = null)
		{
			string[]? stringValues = GetValues(key, (string[]?)null);
			if (stringValues == null)
			{
				return defaultValue;
			}

			List<Guid> guidValues = new List<Guid>();
			foreach (string stringValue in stringValues)
			{
				Guid guidValue;
				if (Guid.TryParse(stringValue, out guidValue))
				{
					guidValues.Add(guidValue);
				}
			}

			return guidValues.ToArray();
		}
	}

	public class ConfigFile
	{
		private readonly List<ConfigSection> _sections = new List<ConfigSection>();

		public ConfigFile()
		{
		}

		static FileReference GetTempFileName(FileReference fileName)
		{
			return fileName + ".tmp";
		}

		public void Load(FileReference fileName)
		{
			Parse(FileReference.ReadAllLines(fileName));
		}

		public bool TryLoad(FileReference fileName, ILogger logger)
		{
			FileInfo fileInfo = fileName.ToFileInfo();
			if (fileInfo.Exists)
			{
				logger.LogDebug("Loading config file from {File} ({Size} bytes)", fileInfo.FullName, fileInfo.Length);
				Load(fileName);
				return true;
			}

			FileReference tempFileName = GetTempFileName(fileName);

			FileInfo tempFileInfo = tempFileName.ToFileInfo();
			if (tempFileInfo.Exists)
			{
				logger.LogDebug("Loading temporary config file from {File} ({Size} bytes)", tempFileInfo.FullName, tempFileInfo.Length);
				Load(tempFileName);
				return true;
			}

			logger.LogDebug("No existing config file at {File}", fileName);
			return false;
		}

		public void Parse(string[] lines, string? project = null)
		{
			ConfigSection? currentSection = null;
			bool wildCardSection = false;

			foreach (string line in lines)
			{
				string trimLine = line.Trim();
				if (!trimLine.StartsWith(";", StringComparison.Ordinal))
				{
					if (trimLine.StartsWith("[", StringComparison.Ordinal) && trimLine.EndsWith("]", StringComparison.Ordinal))
					{
						string sectionName = trimLine.Substring(1, trimLine.Length - 2).Trim();
						wildCardSection = false;

						if (String.IsNullOrEmpty(project) == false &&
							sectionName.StartsWith("//", StringComparison.Ordinal) && sectionName.Contains('*', StringComparison.Ordinal))
						{
							if (FileSystemName.MatchesSimpleExpression(sectionName, project))
							{
								sectionName = project;
								wildCardSection = true;
							}
						}

						currentSection = FindOrAddSection(sectionName);
					}
					else if (currentSection != null)
					{
						int equalsIdx = trimLine.IndexOf('=', StringComparison.Ordinal);
						if (equalsIdx != -1)
						{
							string value = line.Substring(equalsIdx + 1).TrimStart();
							if (trimLine.StartsWith("+", StringComparison.Ordinal))
							{
								currentSection.AppendValue(trimLine.Substring(1, equalsIdx - 1).Trim(), value);
							}
							else
							{
								string key = trimLine.Substring(0, equalsIdx).TrimEnd();

								if (wildCardSection == false || // Non-wildcard settings always take priority
									currentSection.GetValue(key) == null || // If we don't have any value already set, allow a wildcard to be set
									currentSection.HasWildCardValue(key)) // If the current value was set from a wildcard, allow it to be overwritten
								{
									currentSection.SetValue(key, value, wildCardSection);
								}
							}
						}
					}
				}
			}
		}

		public void Save(FileReference fileName)
		{
			FileReference tempFileName = GetTempFileName(fileName);
			FileReference.Delete(tempFileName);

			using (StreamWriter writer = new StreamWriter(tempFileName.FullName))
			{
				for (int idx = 0; idx < _sections.Count; idx++)
				{
					writer.WriteLine("[{0}]", _sections[idx].Name);
					foreach (KeyValuePair<string, string> pair in _sections[idx].Pairs)
					{
						if (pair.Value.Contains('\n', StringComparison.Ordinal))
						{
							foreach (string line in pair.Value.Split('\n'))
							{
								writer.WriteLine("+{0}={1}", pair.Key, line);
							}
						}
						else
						{
							writer.WriteLine("{0}={1}", pair.Key, pair.Value);
						}
					}
					if (idx < _sections.Count - 1)
					{
						writer.WriteLine();
					}
				}
			}

			FileReference.Delete(fileName);
			FileReference.Move(tempFileName, fileName);
		}

		public ConfigSection? FindSection(string name)
		{
			return _sections.FirstOrDefault(x => String.Equals(x.Name, name, StringComparison.OrdinalIgnoreCase));
		}

		public ConfigSection FindOrAddSection(string name)
		{
			ConfigSection? section = FindSection(name);
			if (section == null)
			{
				section = new ConfigSection(name);
				_sections.Add(section);
			}
			return section;
		}

		public void RemoveSection(string name)
		{
			ConfigSection? section = FindSection(name);
			if (section != null)
			{
				_sections.Remove(section);
			}
		}

		public void SetValue(string key, int value)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			ConfigSection section = FindOrAddSection(key.Substring(0, dotIdx));
			section.SetValue(key.Substring(dotIdx + 1), value);
		}

		public void SetValue(string key, bool value)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			ConfigSection section = FindOrAddSection(key.Substring(0, dotIdx));
			section.SetValue(key.Substring(dotIdx + 1), value);
		}

		public void SetValue(string key, string value)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			ConfigSection section = FindOrAddSection(key.Substring(0, dotIdx));
			section.SetValue(key.Substring(dotIdx + 1), value);
		}

		public void SetValues(string key, string[] values)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			ConfigSection section = FindOrAddSection(key.Substring(0, dotIdx));
			section.SetValues(key.Substring(dotIdx + 1), values);
		}

		public bool GetValue(string key, bool defaultValue)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetValue(key.Substring(dotIdx + 1), defaultValue);
		}

		public bool GetValue(string key, bool defaultValue, string? projectIdentifier)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);

			if (!String.IsNullOrWhiteSpace(projectIdentifier))
			{
				string? sectionName = key.Substring(0, dotIdx);
				string? extension = Path.GetExtension(projectIdentifier);
				if (!String.IsNullOrWhiteSpace(sectionName) && !String.IsNullOrWhiteSpace(extension))
				{
					ConfigSection? projectSection = FindSection($"{sectionName}{extension}");
					if (projectSection != null)
					{
						return projectSection.GetValue(key.Substring(dotIdx + 1), defaultValue);
					}
				}
			}

			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetValue(key.Substring(dotIdx + 1), defaultValue);
		}

		public int GetValue(string key, int defaultValue)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);

			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetValue(key.Substring(dotIdx + 1), defaultValue);
		}
		
		public int GetValue(string key, int defaultValue, string? projectIdentifier)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			
			if (!String.IsNullOrWhiteSpace(projectIdentifier))
			{
				string? sectionName = key.Substring(0, dotIdx);
				string? extension = Path.GetExtension(projectIdentifier);
				if (!String.IsNullOrWhiteSpace(sectionName) && !String.IsNullOrWhiteSpace(extension))
				{
					ConfigSection? projectSection = FindSection($"{sectionName}{extension}");
					if (projectSection != null)
					{
						return projectSection.GetValue(key.Substring(dotIdx + 1), defaultValue);
					}
				}
			}

			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetValue(key.Substring(dotIdx + 1), defaultValue);
		}

		public int? GetOptionalIntValue(string key, int? defaultValue)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);

			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetOptionalIntValue(key.Substring(dotIdx + 1), defaultValue);
		}

		public int? GetOptionalIntValue(string key, int? defaultValue, string? projectIdentifier)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);

			if (!String.IsNullOrWhiteSpace(projectIdentifier))
			{
				string? sectionName = key.Substring(0, dotIdx);
				string? extension = Path.GetExtension(projectIdentifier);
				if (!String.IsNullOrWhiteSpace(sectionName) && !String.IsNullOrWhiteSpace(extension))
				{
					ConfigSection? projectSection = FindSection($"{sectionName}{extension}");
					if (projectSection != null)
					{
						return projectSection.GetOptionalIntValue(key.Substring(dotIdx + 1), defaultValue);
					}
				}
			}

			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetOptionalIntValue(key.Substring(dotIdx + 1), defaultValue);
		}
		

		[return: NotNullIfNotNull("defaultValue")]
		public string? GetValue(string key, string? defaultValue)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetValue(key.Substring(dotIdx + 1), defaultValue);
		}

		[return: NotNullIfNotNull("defaultValue")]
		public string? GetValue(string key, string? defaultValue, string? projectIdentifier)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);

			if (!String.IsNullOrWhiteSpace(projectIdentifier))
			{
				string? sectionName = key.Substring(0, dotIdx);
				string? extension = Path.GetExtension(projectIdentifier);
				if (!String.IsNullOrWhiteSpace(sectionName) && !String.IsNullOrWhiteSpace(extension))
				{
					ConfigSection? projectSection = FindSection($"{sectionName}{extension}");
					if (projectSection != null)
					{
						return projectSection.GetValue(key.Substring(dotIdx + 1), defaultValue);
					}
				}
			}

			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetValue(key.Substring(dotIdx + 1), defaultValue);
		}

		public TEnum GetEnumValue<TEnum>(string key, TEnum defaultValue) where TEnum : struct
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetEnumValue(key.Substring(dotIdx + 1), defaultValue);
		}

		public TEnum GetEnumValue<TEnum>(string key, TEnum defaultValue, string? projectIdentifier) where TEnum : struct
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);

			if (!String.IsNullOrWhiteSpace(projectIdentifier))
			{
				string? sectionName = key.Substring(0, dotIdx);
				string? extension = Path.GetExtension(projectIdentifier);
				if (!String.IsNullOrWhiteSpace(sectionName) && !String.IsNullOrWhiteSpace(extension))
				{
					ConfigSection? projectSection = FindSection($"{sectionName}{extension}");
					if (projectSection != null)
					{
						return projectSection.GetEnumValue(key.Substring(dotIdx + 1), defaultValue);
					}
				}
			}

			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetEnumValue(key.Substring(dotIdx + 1), defaultValue);
		}

		[return: NotNullIfNotNull("defaultValue")]
		public string[]? GetValues(string key, string[]? defaultValue)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetValues(key.Substring(dotIdx + 1), defaultValue);
		}

		[return: NotNullIfNotNull("defaultValue")]
		public string[]? GetValues(string key, string[]? defaultValue, string? projectIdentifier)
		{
			List<string> values = new List<string>();

			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			if (section != null)
			{
				values.AddRange(section.GetValues(key.Substring(dotIdx + 1), (string[]?)[]) ?? []);
			}

			if (!String.IsNullOrWhiteSpace(projectIdentifier))
			{
				string? sectionName = key.Substring(0, dotIdx);
				string? extension = Path.GetExtension(projectIdentifier);
				if (!String.IsNullOrWhiteSpace(sectionName) && !String.IsNullOrWhiteSpace(extension))
				{
					ConfigSection? projectSection = FindSection($"{sectionName}{extension}");
					if (projectSection != null)
					{
						values.AddRange(projectSection.GetValues(key.Substring(dotIdx + 1), (string[]?)[]) ?? []);
					}
				}
			}

			return values.Any()
				? values.ToArray()
				: defaultValue;
		}

		[return: NotNullIfNotNull("defaultValue")]
		public Guid[]? GetGuidValues(string key, Guid[]? defaultValue)
		{
			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			return (section == null) ? defaultValue : section.GetValues(key.Substring(dotIdx + 1), defaultValue);
		}

		[return: NotNullIfNotNull("defaultValue")]
		public Guid[]? GetGuidValues(string key, Guid[]? defaultValue, string? projectIdentifier)
		{
			List<Guid> values = new List<Guid>();

			int dotIdx = key.IndexOf('.', StringComparison.Ordinal);
			ConfigSection? section = FindSection(key.Substring(0, dotIdx));
			if (section != null)
			{
				values.AddRange(section.GetValues(key.Substring(dotIdx + 1), (Guid[]?)[]) ?? []);
			}

			if (!String.IsNullOrWhiteSpace(projectIdentifier))
			{
				string? sectionName = key.Substring(0, dotIdx);
				string? extension = Path.GetExtension(projectIdentifier);
				if (!String.IsNullOrWhiteSpace(sectionName) && !String.IsNullOrWhiteSpace(extension))
				{
					ConfigSection? projectSection = FindSection($"{sectionName}{extension}");
					if (projectSection != null)
					{
						values.AddRange(projectSection.GetValues(key.Substring(dotIdx + 1), (Guid[]?)[]) ?? []);
					}
				}
			}

			return values.Any()
				? values.ToArray()
				: defaultValue;
		}
	}
}
