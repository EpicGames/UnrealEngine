// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	public class TargetReceipt
	{
		public string? Configuration { get; set; }
		public string? Launch { get; set; }
		public string? LaunchCmd { get; set; }

		public static bool TryRead(FileReference location, DirectoryReference? engineDir, DirectoryReference? projectDir, [NotNullWhen(true)] out TargetReceipt? receipt)
		{
			if (Utility.TryLoadJson(location, out receipt))
			{
				receipt.Launch = ExpandReceiptVariables(receipt.Launch, engineDir, projectDir);
				receipt.LaunchCmd = ExpandReceiptVariables(receipt.LaunchCmd, engineDir, projectDir);
				return true;
			}
			return false;
		}

		[return: NotNullIfNotNull("line")]
		private static string? ExpandReceiptVariables(string? line, DirectoryReference? engineDir, DirectoryReference? projectDir)
		{
			string? expandedLine = line;
			if (expandedLine != null)
			{
				if (engineDir != null)
				{
					expandedLine = expandedLine.Replace("$(EngineDir)", engineDir.FullName, StringComparison.OrdinalIgnoreCase);
				}
				if (projectDir != null)
				{
					expandedLine = expandedLine.Replace("$(ProjectDir)", projectDir.FullName, StringComparison.OrdinalIgnoreCase);
				}
			}
			return expandedLine;
		}
	}

	public static class ConfigUtils
	{
		public static string HostPlatform { get; } = GetHostPlatform();

		public static string HostArchitectureSuffix { get; } = String.Empty;

		static string GetHostPlatform()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return "Win64";
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				return "Mac";
			}
			else
			{
				return "Linux";
			}
		}

		public static Task<ConfigFile> ReadProjectConfigFileAsync(IPerforceConnection perforce, ProjectInfo projectInfo, ILogger logger, CancellationToken cancellationToken)
		{
			return ReadProjectConfigFileAsync(perforce, projectInfo, new List<KeyValuePair<FileReference, DateTime>>(), logger, cancellationToken);
		}

		public static async Task<ConfigFile> ReadProjectConfigFileAsync(IPerforceConnection perforce, ProjectInfo projectInfo, List<KeyValuePair<FileReference, DateTime>> localConfigFiles, ILogger logger, CancellationToken cancellationToken)
		{
			return await ReadProjectConfigFileAsync(perforce, projectInfo.ClientRootPath, projectInfo.ClientFileName, projectInfo.ProjectIdentifier, projectInfo.CacheFolder, localConfigFiles, logger, cancellationToken);
		}

		public static async Task<ConfigFile> ReadProjectConfigFileAsync(IPerforceConnection perforce, string branchClientPath, string selectedClientFileName, string projectIdentifier, DirectoryReference cacheFolder, List<KeyValuePair<FileReference, DateTime>> localConfigFiles, ILogger logger, CancellationToken cancellationToken)
		{
			List<string> configFilePaths = Utility.GetDepotConfigPaths(branchClientPath + "/Engine", selectedClientFileName, branchClientPath);

			ConfigFile projectConfig = new ConfigFile();

			List<PerforceResponse<FStatRecord>> responses = await perforce.TryFStatAsync(FStatOptions.IncludeFileSizes, configFilePaths, cancellationToken).ToListAsync(cancellationToken);
			foreach (PerforceResponse<FStatRecord> response in responses)
			{
				if (response.Succeeded)
				{
					string[]? lines = null;

					// Skip file records which are still in the workspace, but were synced from a different branch. For these files, the action seems to be empty, so filter against that.
					FStatRecord fileRecord = response.Data;
					if (fileRecord.HeadAction == FileAction.None)
					{
						continue;
					}

					// If this file is open for edit, read the local version
					string? localFileName = fileRecord.ClientFile;
					if (localFileName != null && File.Exists(localFileName) && (File.GetAttributes(localFileName) & FileAttributes.ReadOnly) == 0)
					{
						try
						{
							DateTime lastModifiedTime = File.GetLastWriteTimeUtc(localFileName);
							localConfigFiles.Add(new KeyValuePair<FileReference, DateTime>(new FileReference(localFileName), lastModifiedTime));
							lines = await File.ReadAllLinesAsync(localFileName, cancellationToken);
						}
						catch (Exception ex)
						{
							logger.LogInformation(ex, "Failed to read local config file for {Path}", localFileName);
						}
					}

					// Otherwise try to get it from perforce
					if (lines == null && fileRecord.DepotFile != null)
					{
						lines = await Utility.TryPrintFileUsingCacheAsync(perforce, fileRecord.DepotFile, cacheFolder, fileRecord.Digest, logger, cancellationToken);
					}

					// Merge the text with the config file
					if (lines != null)
					{
						try
						{
							projectConfig.Parse(lines.ToArray(), projectIdentifier);
							logger.LogDebug("Read config file from {DepotFile}", fileRecord.DepotFile);
						}
						catch (Exception ex)
						{
							logger.LogInformation(ex, "Failed to read config file from {DepotFile}", fileRecord.DepotFile);
						}
					}
				}
			}

			// Execute dynamic config injection scripts
			await ExecuteConfigInjectionScriptsAsync(perforce, branchClientPath, selectedClientFileName, projectIdentifier, cacheFolder, projectConfig, localConfigFiles, logger, cancellationToken);

			return projectConfig;
		}

		public static async Task<List<string[]>> ReadConfigFiles(IPerforceConnection perforce, IEnumerable<string> depotPaths, List<KeyValuePair<FileReference, DateTime>> localFiles, DirectoryReference cacheFolder, ILogger logger, CancellationToken cancellationToken)
		{
			List<string[]> contents = new List<string[]>();

			List<PerforceResponse<FStatRecord>> responses = await perforce.TryFStatAsync(FStatOptions.IncludeFileSizes, depotPaths.ToArray(), cancellationToken).ToListAsync(cancellationToken);
			foreach (PerforceResponse<FStatRecord> response in responses)
			{
				if (response.Succeeded)
				{
					string[]? lines = null;

					// Skip file records which are still in the workspace, but were synced from a different branch. For these files, the action seems to be empty, so filter against that.
					FStatRecord fileRecord = response.Data;
					if (fileRecord.HeadAction == FileAction.None)
					{
						continue;
					}

					// If this file is open for edit, read the local version
					string? localFileName = fileRecord.ClientFile;
					if (localFileName != null && File.Exists(localFileName) && (File.GetAttributes(localFileName) & FileAttributes.ReadOnly) == 0)
					{
						try
						{
							DateTime lastModifiedTime = File.GetLastWriteTimeUtc(localFileName);
							localFiles.Add(new KeyValuePair<FileReference, DateTime>(new FileReference(localFileName), lastModifiedTime));
							lines = await File.ReadAllLinesAsync(localFileName, cancellationToken);
						}
						catch (Exception ex)
						{
							logger.LogInformation(ex, "Failed to read local config file for {Path}", localFileName);
						}
					}

					// Otherwise try to get it from perforce
					if (lines == null && fileRecord.DepotFile != null)
					{
						lines = await Utility.TryPrintFileUsingCacheAsync(perforce, fileRecord.DepotFile, cacheFolder, fileRecord.Digest, logger, cancellationToken);
					}

					// Merge the text with the config file
					if (lines != null)
					{
						contents.Add(lines);
					}
				}
			}

			return contents;
		}

		public static FileReference GetEditorTargetFile(ProjectInfo projectInfo, ConfigFile projectConfig)
		{
			if (projectInfo.ProjectPath.EndsWith(".uproject", StringComparison.OrdinalIgnoreCase))
			{
				List<FileReference> targetFiles = FindTargets(projectInfo.LocalFileName.Directory);

				FileReference? targetFile = targetFiles.OrderBy(x => x.FullName, StringComparer.OrdinalIgnoreCase).FirstOrDefault(x => x.FullName.EndsWith("Editor.target.cs", StringComparison.OrdinalIgnoreCase));
				if (targetFile != null)
				{
					return targetFile;
				}
			}

			string defaultEditorTargetName = GetDefaultEditorTargetName(projectInfo, projectConfig);
			return FileReference.Combine(projectInfo.LocalRootPath, "Engine", "Source", $"{defaultEditorTargetName}.Target.cs");
		}

		public static FileReference GetEditorReceiptFile(ProjectInfo projectInfo, ConfigFile projectConfig, BuildConfig config)
		{
			FileReference targetFile = GetEditorTargetFile(projectInfo, projectConfig);
			return GetReceiptFile(projectInfo, projectConfig, targetFile, config.ToString());
		}

		private static List<FileReference> FindTargets(DirectoryReference engineOrProjectDir)
		{
			List<FileReference> targets = new List<FileReference>();

			DirectoryReference sourceDir = DirectoryReference.Combine(engineOrProjectDir, "Source");
			if (DirectoryReference.Exists(sourceDir))
			{
				foreach (FileReference targetFile in DirectoryReference.EnumerateFiles(sourceDir))
				{
					const string extension = ".target.cs";
					if (targetFile.FullName.EndsWith(extension, StringComparison.OrdinalIgnoreCase))
					{
						targets.Add(targetFile);
					}
				}
			}

			return targets;
		}

		public static string GetDefaultEditorTargetName(ProjectInfo projectInfo, ConfigFile projectConfigFile)
		{
			string? editorTarget;
			if (!TryGetProjectSetting(projectConfigFile, projectInfo.ProjectIdentifier, "EditorTarget", out editorTarget))
			{
				if (projectInfo.IsEnterpriseProject)
				{
					editorTarget = "StudioEditor";
				}
				else
				{
					editorTarget = "UE4Editor";
				}
			}
			return editorTarget;
		}

		public static bool TryReadEditorReceipt(ProjectInfo projectInfo, FileReference receiptFile, [NotNullWhen(true)] out TargetReceipt? receipt)
		{
			DirectoryReference engineDir = DirectoryReference.Combine(projectInfo.LocalRootPath, "Engine");
			DirectoryReference projectDir = projectInfo.LocalFileName.Directory;

			if (receiptFile.IsUnderDirectory(projectDir))
			{
				return TargetReceipt.TryRead(receiptFile, engineDir, projectDir, out receipt);
			}
			else
			{
				return TargetReceipt.TryRead(receiptFile, engineDir, null, out receipt);
			}
		}

		public static TargetReceipt CreateDefaultEditorReceipt(ProjectInfo projectInfo, ConfigFile projectConfigFile, BuildConfig configuration)
		{
			string baseName = GetDefaultEditorTargetName(projectInfo, projectConfigFile);
			if (configuration != BuildConfig.Development || !String.IsNullOrEmpty(HostArchitectureSuffix))
			{
				if (configuration != BuildConfig.DebugGame || projectConfigFile.GetValue("Options.DebugGameHasSeparateExecutable", false))
				{
					baseName += $"-{HostPlatform}-{configuration}{HostArchitectureSuffix}";
				}
			}

			string extension = String.Empty;
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				extension = ".exe";
			}

			TargetReceipt receipt = new TargetReceipt();
			receipt.Configuration = configuration.ToString();
			receipt.Launch = FileReference.Combine(projectInfo.LocalRootPath, "Engine", "Binaries", HostPlatform, $"{baseName}{extension}").FullName;
			receipt.LaunchCmd = FileReference.Combine(projectInfo.LocalRootPath, "Engine", "Binaries", HostPlatform, $"{baseName}-Cmd{extension}").FullName;
			return receipt;
		}

		private static bool UseSharedEditorReceipt(ProjectInfo projectInfo, ConfigFile projectConfig)
		{
			string? setting;
			if (TryGetProjectSetting(projectConfig, projectInfo.ProjectIdentifier, "UseSharedEditor", out setting))
			{
				bool value;
				if (Boolean.TryParse(setting, out value))
				{
					return value;
				}
			}
			return false;
		}

		public static FileReference GetReceiptFile(ProjectInfo projectInfo, ConfigFile projectConfig, FileReference targetFile, string configuration)
		{
			string targetName = targetFile.GetFileNameWithoutAnyExtensions();

			DirectoryReference? projectDir = projectInfo.ProjectDir;
			if (projectDir != null && (targetFile.IsUnderDirectory(projectDir) || !UseSharedEditorReceipt(projectInfo, projectConfig)))
			{
				return GetReceiptFile(projectDir, targetName, configuration);
			}
			else
			{
				return GetReceiptFile(projectInfo.EngineDir, targetName, configuration);
			}
		}

		public static FileReference GetReceiptFile(DirectoryReference baseDir, string targetName, string configuration)
		{
			return GetReceiptFile(baseDir, targetName, HostPlatform, configuration, HostArchitectureSuffix);
		}

		public static FileReference GetReceiptFile(DirectoryReference baseDir, string targetName, string platform, string configuration, string architectureSuffix)
		{
			if (String.IsNullOrEmpty(architectureSuffix) && configuration.Equals("Development", StringComparison.OrdinalIgnoreCase))
			{
				return FileReference.Combine(baseDir, "Binaries", platform, $"{targetName}.target");
			}
			else
			{
				return FileReference.Combine(baseDir, "Binaries", platform, $"{targetName}-{platform}-{configuration}{architectureSuffix}.target");
			}
		}

		public static Dictionary<Guid, ConfigObject> GetDefaultBuildStepObjects(ProjectInfo projectInfo, string editorTarget, BuildConfig editorConfig, ConfigFile latestProjectConfigFile, bool shouldSyncPrecompiledEditor)
		{
			string projectArgument = "";
			if (projectInfo.LocalFileName.HasExtension(".uproject"))
			{
				projectArgument = String.Format("\"{0}\"", projectInfo.LocalFileName);
			}

			bool useCrashReportClientEditor = latestProjectConfigFile.GetValue("Options.UseCrashReportClientEditor", false);

			string hostPlatform;
			if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				hostPlatform = "Mac";
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				hostPlatform = "Linux";
			}
			else
			{
				hostPlatform = "Win64";
			}

			List<BuildStep> defaultBuildSteps = new List<BuildStep>();
			if (latestProjectConfigFile.GetValue("Options.BuildUnrealHeaderTool", true))
			{
				defaultBuildSteps.Add(new BuildStep(new Guid("{01F66060-73FA-4CC8-9CB3-E217FBBA954E}"), 0, "Compile UnrealHeaderTool", "Compiling UnrealHeaderTool...", 1, "UnrealHeaderTool", hostPlatform, "Development", "", !shouldSyncPrecompiledEditor));
			}
			defaultBuildSteps.Add(new BuildStep(new Guid("{F097FF61-C916-4058-8391-35B46C3173D5}"), 1, $"Compile {editorTarget}", $"Compiling {editorTarget}...", 10, editorTarget, hostPlatform, editorConfig.ToString(), projectArgument, !shouldSyncPrecompiledEditor));
			defaultBuildSteps.Add(new BuildStep(new Guid("{C6E633A1-956F-4AD3-BC95-6D06D131E7B4}"), 2, "Compile ShaderCompileWorker", "Compiling ShaderCompileWorker...", 1, "ShaderCompileWorker", hostPlatform, "Development", "", !shouldSyncPrecompiledEditor));
			defaultBuildSteps.Add(new BuildStep(new Guid("{24FFD88C-7901-4899-9696-AE1066B4B6E8}"), 3, "Compile UnrealLightmass", "Compiling UnrealLightmass...", 1, "UnrealLightmass", hostPlatform, "Development", "", !shouldSyncPrecompiledEditor));
			defaultBuildSteps.Add(new BuildStep(new Guid("{FFF20379-06BF-4205-8A3E-C53427736688}"), 4, "Compile CrashReportClient", "Compiling CrashReportClient...", 1, "CrashReportClient", hostPlatform, "Shipping", "", !shouldSyncPrecompiledEditor && !useCrashReportClientEditor));
			defaultBuildSteps.Add(new BuildStep(new Guid("{7143D861-58D3-4F83-BADC-BC5DCB2079F6}"), 5, "Compile CrashReportClientEditor", "Compiling CrashReportClientEditor...", 1, "CrashReportClientEditor", hostPlatform, "Shipping", "", !shouldSyncPrecompiledEditor && useCrashReportClientEditor));

			return defaultBuildSteps.ToDictionary(x => x.UniqueId, x => x.ToConfigObject());
		}

		public static Dictionary<string, string> GetWorkspaceVariables(ProjectInfo projectInfo, int changeNumber, int codeChangeNumber, TargetReceipt? editorTarget, ConfigFile? projectConfigFile, IPerforceSettings perforceSettings)
		{
			Dictionary<string, string> variables = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);

			if (projectInfo.StreamName != null)
			{
				variables.Add("Stream", projectInfo.StreamName);
			}

			variables.Add("Change", changeNumber.ToString());
			variables.Add("CodeChange", codeChangeNumber.ToString());

			variables.Add("ClientName", projectInfo.ClientName);
			variables.Add("BranchDir", projectInfo.LocalRootPath.FullName);
			variables.Add("ProjectDir", projectInfo.LocalFileName.Directory.FullName);
			variables.Add("ProjectFile", projectInfo.LocalFileName.FullName);
			variables.Add("UseIncrementalBuilds", "1");

			string editorConfig = editorTarget?.Configuration ?? String.Empty;
			variables.Add("EditorConfig", editorConfig);

			string editorLaunch = editorTarget?.Launch ?? String.Empty;
			variables.Add("EditorExe", editorLaunch);

			string editorLaunchCmd = editorTarget?.LaunchCmd ?? editorLaunch.Replace(".exe", "-Cmd.exe", StringComparison.OrdinalIgnoreCase);
			variables.Add("EditorCmdExe", editorLaunchCmd);

			// Legacy
			variables.Add("UE4EditorConfig", editorConfig);
			variables.Add("UE4EditorDebugArg", (editorConfig.Equals("Debug", StringComparison.Ordinal) || editorConfig.Equals("DebugGame", StringComparison.Ordinal)) ? " -debug" : "");
			variables.Add("UE4EditorExe", editorLaunch);
			variables.Add("UE4EditorCmdExe", editorLaunchCmd);

			if (projectConfigFile != null)
			{
				if (TryGetProjectSetting(projectConfigFile, projectInfo.ProjectIdentifier, "SdkInstallerDir", out string? sdkInstallerDir))
				{
					variables.Add("SdkInstallerDir", sdkInstallerDir);
				}
			}

			variables.Add("PerforceServerAndPort", perforceSettings.ServerAndPort);
			variables.Add("PerforceUserName", perforceSettings.UserName);
			if (perforceSettings.ClientName != null)
			{
				variables.Add("PerforceClientName", perforceSettings.ClientName);
			}

			return variables;
		}

		public static Dictionary<string, string> GetWorkspaceVariables(ProjectInfo projectInfo, int changeNumber, int codeChangeNumber, TargetReceipt? editorTarget, ConfigFile? projectConfigFile, IPerforceSettings perforceSettings, IEnumerable<KeyValuePair<string, string>> additionalVariables)
		{
			Dictionary<string, string> variables = GetWorkspaceVariables(projectInfo, changeNumber, codeChangeNumber, editorTarget, projectConfigFile, perforceSettings);
			foreach ((string key, string value) in additionalVariables)
			{
				variables[key] = value;
			}
			return variables;
		}

		public static bool TryGetProjectSetting(ConfigFile projectConfigFile, string selectedProjectIdentifier, string name, [NotNullWhen(true)] out string? value)
		{
			string path = selectedProjectIdentifier;
			for (; ; )
			{
				ConfigSection? projectSection = projectConfigFile.FindSection(path);
				if (projectSection != null)
				{
					string? newValue = projectSection.GetValue(name, null);
					if (newValue != null)
					{
						value = newValue;
						return true;
					}
				}

				int lastSlash = path.LastIndexOf('/');
				if (lastSlash < 2)
				{
					break;
				}

				path = path.Substring(0, lastSlash);
			}

			ConfigSection? defaultSection = projectConfigFile.FindSection("Default");
			if (defaultSection != null)
			{
				string? newValue = defaultSection.GetValue(name, null);
				if (newValue != null)
				{
					value = newValue;
					return true;
				}
			}

			value = null;
			return false;
		}

		public static void GetProjectSettings(ConfigFile projectConfigFile, string selectedProjectIdentifier, string name, List<string> values)
		{
			string path = selectedProjectIdentifier;
			for (; ; )
			{
				ConfigSection? projectSection = projectConfigFile.FindSection(path);
				if (projectSection != null)
				{
					values.AddRange(projectSection.GetValues(name, Array.Empty<string>()));
				}

				int lastSlash = path.LastIndexOf('/');
				if (lastSlash < 2)
				{
					break;
				}

				path = path.Substring(0, lastSlash);
			}

			ConfigSection? defaultSection = projectConfigFile.FindSection("Default");
			if (defaultSection != null)
			{
				values.AddRange(defaultSection.GetValues(name, Array.Empty<string>()));
			}
		}

		public static Dictionary<Guid, WorkspaceSyncCategory> GetSyncCategories(ConfigFile projectConfigFile)
		{
			Dictionary<Guid, WorkspaceSyncCategory> uniqueIdToCategory = new Dictionary<Guid, WorkspaceSyncCategory>();
			if (projectConfigFile != null)
			{
				string[] categoryLines = projectConfigFile.GetValues("Options.SyncCategory", Array.Empty<string>());
				foreach (string categoryLine in categoryLines)
				{
					ConfigObject obj = new ConfigObject(categoryLine);

					Guid uniqueId;
					if (Guid.TryParse(obj.GetValue("UniqueId", ""), out uniqueId))
					{
						WorkspaceSyncCategory? category;
						if (!uniqueIdToCategory.TryGetValue(uniqueId, out category))
						{
							category = new WorkspaceSyncCategory(uniqueId);
							uniqueIdToCategory.Add(uniqueId, category);
						}

						if (obj.GetValue("Clear", false))
						{
							category.Paths.Clear();
							category.Requires.Clear();
						}

						category.Name = obj.GetValue("Name", category.Name);
						category.Enable = obj.GetValue("Enable", category.Enable);

						string[] paths = Enumerable.Concat(category.Paths, obj.GetValue("Paths", "").Split(';').Select(x => x.Trim())).Where(x => x.Length > 0).Distinct().OrderBy(x => x).ToArray();
						category.Paths.Clear();
						category.Paths.AddRange(paths);

						category.Hidden = obj.GetValue("Hidden", category.Hidden);

						Guid[] requires = Enumerable.Concat(category.Requires, ParseGuids(obj.GetValue("Requires", "").Split(';'))).Distinct().OrderBy(x => x).ToArray();
						category.Requires.Clear();
						category.Requires.AddRange(requires);
					}
				}
			}
			return uniqueIdToCategory;
		}

		public static bool IsPresetAvailable(ConfigFile? projectConfigFile, string projectIdentifier, string preset)
		{
			if (projectConfigFile == null || String.IsNullOrWhiteSpace(projectIdentifier))
			{
				return false;
			}

			IDictionary<string, Preset> presets = GetPresets(projectConfigFile, projectIdentifier);

			return presets.ContainsKey(preset);
		}

		public static bool GetDefaultPreset(ConfigFile? projectConfigFile, string projectIdentifier, out string preset)
		{
			preset = String.Empty;

			if (projectConfigFile == null || String.IsNullOrWhiteSpace(projectIdentifier))
			{
				return false;
			}

			if (TryGetProjectSetting(projectConfigFile, projectIdentifier, "DefaultPreset", out string? projectDefaultPreset))
			{
				IDictionary<string, Preset> presets = GetPresets(projectConfigFile, projectIdentifier);

				using StringReader sr = new StringReader(projectDefaultPreset);
				while (sr.Peek() != -1)
				{
					string? line = sr.ReadLine();
					if (String.IsNullOrWhiteSpace(line))
					{
						continue;
					}
					ConfigObject obj = new ConfigObject(line);

					string configPreset = obj.GetValue("Name", String.Empty);
					if (!presets.ContainsKey(configPreset) || String.IsNullOrWhiteSpace(configPreset))
					{
						continue;
					}

					preset = configPreset;
					return true;
				}
			}

			return false;
		}

		public static bool GetForceDefaultPreset(ConfigFile? projectConfigFile, string projectIdentifier, out bool forceDefaultPreset)
		{
			forceDefaultPreset = false;

			if (projectConfigFile == null || String.IsNullOrWhiteSpace(projectIdentifier))
			{
				return false;
			}

			if (TryGetProjectSetting(projectConfigFile, projectIdentifier, "DefaultPreset", out string? projectDefaultPreset))
			{
				using StringReader sr = new StringReader(projectDefaultPreset);
				while (sr.Peek() != -1)
				{
					string? line = sr.ReadLine();
					if (String.IsNullOrWhiteSpace(line))
					{
						continue;
					}
					ConfigObject obj = new ConfigObject(line);

					bool locked = obj.GetValue("Locked",false);

					forceDefaultPreset = locked;
					return true;
				}
			}

			return false;
		}

		public static bool GetForceSyncBaseContent(ConfigFile? projectConfigFile, string projectIdentifier, out bool forceSyncBaseContent)
		{
			forceSyncBaseContent = false;

			if (projectConfigFile == null || String.IsNullOrWhiteSpace(projectIdentifier))
			{
				return false;
			}

			if (TryGetProjectSetting(projectConfigFile, projectIdentifier, "ForceSyncBaseContent", out string? setting))
			{
				if (Boolean.TryParse(setting, out bool value))
				{
					forceSyncBaseContent = value;
					return true;
				}
			}

			return false;
		}

		public static IDictionary<string, Preset> GetPresets(ConfigFile? projectConfigFile, string projectIdentifier)
		{
			Dictionary<string, Preset> presetsDictionary = new(StringComparer.OrdinalIgnoreCase);

			if (projectConfigFile == null)
			{
				return presetsDictionary;
			}
			List<string> presetDefinitions = new List<string>();

			presetDefinitions.AddRange(projectConfigFile.GetValues("Presets.Preset", []));

			if (TryGetProjectSetting(projectConfigFile, projectIdentifier, "Preset", out string? projectPresets))
			{
				using StringReader sr = new StringReader(projectPresets);
				while (sr.Peek() != -1)
				{
					string? line = sr.ReadLine();
					if (String.IsNullOrWhiteSpace(line))
					{
						continue;
					}
					presetDefinitions.Add(line);
				}
			}

			foreach (string presetDefinition in presetDefinitions)
			{
				ConfigObject obj = new ConfigObject(presetDefinition);

				Preset? preset = new Preset();

				preset.Name = obj.GetValue("Name", String.Empty);

				// do not allow empty role name
				if (String.IsNullOrWhiteSpace(preset.Name))
				{
					continue;
				}

				IEnumerable<string> categories = obj.GetValue("Categories", String.Empty)
					.Split(';')
					.Select(x => x.Trim())
					.Where(x => x.Length > 0)
					.Distinct()
					.OrderBy(x => x);

				foreach (string categoryLine in categories)
				{
					string[] values = categoryLine
						.Split(',')
						.Select(x => x.Trim())
						.ToArray();

					// all categories shall be completely defined
					if (values.Length != 2)
					{
						continue;
					}

					RoleCategory category = new();
					if (Guid.TryParse(values[0], out Guid guid))
					{
						category.Id = guid;
					}

					if (Boolean.TryParse(values[1], out bool enabled))
					{
						category.Enabled = enabled;
					}

					if (category.Id != Guid.Empty)
					{
						preset.Categories.TryAdd(category.Id, category);
					}
				}

				IEnumerable<string> views = obj.GetValue("Views", String.Empty)
						.Split(';')
						.Select(x => x.Trim())
						.Where(x => x.Length > 0)
						.Distinct()
					;

				foreach (string view in views)
				{
					preset.Views.Add(view);
				}

				if (presetsDictionary.ContainsKey(preset.Name))
				{
					presetsDictionary[preset.Name].Import(preset);
				}
				else
				{
					presetsDictionary.TryAdd(preset.Name, preset);
				}
			}

			return presetsDictionary;
		}

		public static ISet<string> GetAdditionalPathsToSync(ConfigFile latestProjectConfigFile)
		{
			ConfigSection? perforceConfigSection = latestProjectConfigFile.FindSection("Perforce");
			if (perforceConfigSection != null)
			{
				IEnumerable<string> additionalPaths = perforceConfigSection.GetValues("AdditionalPathsToSync", Array.Empty<string>());
				if (additionalPaths.Any())
				{
					return new HashSet<string>(additionalPaths);
				}
			}
			return new HashSet<string>();
		}

		public static ISet<string> GetBaseContentPaths(ConfigFile? projectConfigFile, string projectIdentifier)
		{
			ISet<string> output = new HashSet<string>();
			if (projectConfigFile == null)
			{
				return output;
			}

			if (TryGetProjectSetting(projectConfigFile, projectIdentifier, "BaseContent", out string? projectBaseContent))
			{
				using StringReader sr = new StringReader(projectBaseContent);
				while (sr.Peek() != -1)
				{
					string? line = sr.ReadLine();
					if (String.IsNullOrWhiteSpace(line))
					{
						continue;
					}
					output.Add(line);
				}
			}

			return output;
		}

		static IEnumerable<Guid> ParseGuids(IEnumerable<string> values)
		{
			foreach (string value in values)
			{
				Guid guid;
				if (Guid.TryParse(value, out guid))
				{
					yield return guid;
				}
			}
		}

		private static async Task<DirectoryReference?> GetLocalRootPathAsync(IPerforceConnection perforce, ILogger logger, CancellationToken cancellationToken)
		{
			try
			{
				string? clientName = perforce.Settings.ClientName;
				if (String.IsNullOrEmpty(clientName))
				{
					logger.LogDebug("No client name available in Perforce settings");
					return null;
				}

				ClientRecord client = await perforce.GetClientAsync(clientName, cancellationToken);
				if (client == null || String.IsNullOrEmpty(client.Root))
				{
					logger.LogDebug("Could not get client root from Perforce client {ClientName}", clientName);
					return null;
				}

				return new DirectoryReference(client.Root);
			}
			catch (Exception ex)
			{
				logger.LogDebug(ex, "Failed to get local root path from Perforce client");
				return null;
			}
		}

		public static async Task ExecuteConfigInjectionScriptsAsync(
			IPerforceConnection perforce,
			string branchClientPath,
			string selectedClientFileName,
			string projectIdentifier,
			DirectoryReference cacheFolder,
			ConfigFile configFile,
			List<KeyValuePair<FileReference, DateTime>> localConfigFiles,
			ILogger logger,
			CancellationToken cancellationToken = default)
		{
			// Get all Inject values
			string[] injectLines = configFile.GetValues("IniDynamicInjection.Inject", Array.Empty<string>(), projectIdentifier);
			if (injectLines.Length == 0)
			{
				return;
			}

			// Get local root path from Perforce client
			DirectoryReference? localRootPath = await GetLocalRootPathAsync(perforce, logger, cancellationToken);
			if (localRootPath == null)
			{
				logger.LogWarning("Could not determine local root path from Perforce client, skipping dynamic config injection");
				return;
			}

			logger.LogDebug("Found {Count} dynamic injection script(s) to execute", injectLines.Length);

			// Execute each script in order
			foreach (string injectLine in injectLines)
			{
				try
				{
					// Parse the ConfigObject format: (Platform="Win64", Script="Tools/Scripts/inject.py")
					ConfigObject injectObj = new ConfigObject(injectLine);

					string platform = injectObj.GetValue("Platform", "");
					string scriptPath = injectObj.GetValue("Script", "");
					string scriptArgs = injectObj.GetValue("Args", "");

					if (String.IsNullOrWhiteSpace(scriptPath))
					{
						logger.LogWarning("Inject entry missing Script value: {Line}", injectLine);
						continue;
					}

					// Check if platform matches (empty platform means all platforms)
					if (!String.IsNullOrEmpty(platform) && !platform.Equals(HostPlatform, StringComparison.OrdinalIgnoreCase))
					{
						logger.LogDebug("Skipping script {Script} (platform {Platform} does not match {HostPlatform})", scriptPath, platform, HostPlatform);
						continue;
					}

					await ExecuteSingleInjectionScriptAsync(perforce, branchClientPath, localRootPath, selectedClientFileName, projectIdentifier, cacheFolder, configFile, localConfigFiles, scriptPath, scriptArgs, logger, cancellationToken);
				}
				catch (Exception ex)
				{
					logger.LogWarning(ex, "Failed to execute injection script from line: {Line}", injectLine);
				}
			}
		}

		public static IDictionary<string, FeatureFlag> GetFeatureFlags(ConfigFile? projectConfigFile, string projectIdentifier)
		{
			Dictionary<string, FeatureFlag> output = new(StringComparer.OrdinalIgnoreCase);

			if (projectConfigFile == null)
			{
				return output;
			}
			List<string> features = new List<string>();

			features.AddRange(projectConfigFile.GetValues("Features.FeatureFlag", []));

			if (TryGetProjectSetting(projectConfigFile, projectIdentifier, "FeatureFlag", out string? projectFeatures))
			{
				using StringReader sr = new StringReader(projectFeatures);
				while (sr.Peek() != -1)
				{
					string? line = sr.ReadLine();
					if (String.IsNullOrWhiteSpace(line))
					{
						continue;
					}
					features.Add(line);
				}
			}

			foreach (string presetDefinition in features)
			{
				ConfigObject obj = new ConfigObject(presetDefinition);

				FeatureFlag? ff = new FeatureFlag();

				ff.Name = obj.GetValue("Name", String.Empty);

				// do not allow empty name
				if (String.IsNullOrWhiteSpace(ff.Name))
				{
					continue;
				}

				ff.Enabled = obj.GetValue("Enabled", ff.Enabled);

				if (output.ContainsKey(ff.Name))
				{
					output[ff.Name].Import(ff);
				}
				else
				{
					output.TryAdd(ff.Name, ff);
				}
			}

			return output;
		}

		public static bool IsFeatureEnabled(IDictionary<string, FeatureFlag> features, string featureName)
		{
			if (features.TryGetValue(featureName, out FeatureFlag? featureFlag))
			{
				return featureFlag.Enabled;
			}

			return false;
		}

		private static async Task ExecuteSingleInjectionScriptAsync(
			IPerforceConnection perforce,
			string branchClientPath,
			DirectoryReference localRootPath,
			string selectedClientFileName,
			string projectIdentifier,
			DirectoryReference cacheFolder,
			ConfigFile configFile,
			List<KeyValuePair<FileReference, DateTime>> localConfigFiles,
			string relativePath,
			string scriptArgs,
			ILogger logger,
			CancellationToken cancellationToken)
		{
			// Determine local path
			string localPath = Path.Combine(localRootPath.FullName, relativePath.Replace('/', Path.DirectorySeparatorChar));

			// Validate path doesn't escape workspace root (prevent path traversal)
			string normalizedPath = Path.GetFullPath(localPath);
			string rootPrefix = localRootPath.FullName.TrimEnd(Path.DirectorySeparatorChar) + Path.DirectorySeparatorChar;
			StringComparison pathComparison = RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? StringComparison.OrdinalIgnoreCase : StringComparison.Ordinal;
			if (!normalizedPath.StartsWith(rootPrefix, pathComparison))
			{
				logger.LogWarning("Script path {Script} attempts to escape workspace root, skipping", relativePath);
				return;
			}

			bool isLocalFile = false;
			bool needsTempFile = false;

			// Convert relative path to client path for Perforce (always use forward slashes)
			string clientPath = $"{branchClientPath}/{relativePath
				.Replace('\\', '/')
				.TrimStart('/')}";

			// Try to get file metadata from Perforce
			List<PerforceResponse<FStatRecord>> responses = await perforce.TryFStatAsync(FStatOptions.IncludeFileSizes, clientPath, cancellationToken).ToListAsync(cancellationToken);

			if (responses.Count > 0 && responses[0].Succeeded)
			{
				// File exists in Perforce
				FStatRecord fileRecord = responses[0].Data;

				// Skip if synced from different branch
				if (fileRecord.HeadAction == FileAction.None && !fileRecord.IsMapped)
				{
					logger.LogDebug("Skipping script {Script} (synced from different branch)", relativePath);
					return;
				}

				// Check if local file is writable (edited or opened for add)
				string? localFileName = fileRecord.ClientFile;
				if (localFileName != null && File.Exists(localFileName) && (File.GetAttributes(localFileName) & FileAttributes.ReadOnly) == 0)
				{
					try
					{
						DateTime lastModifiedTime = File.GetLastWriteTimeUtc(localFileName);
						localConfigFiles.Add(new KeyValuePair<FileReference, DateTime>(new FileReference(localFileName), lastModifiedTime));
						localPath = localFileName;
						isLocalFile = true;
						logger.LogDebug("Using locally edited script from Perforce workspace: {Script}", relativePath);
					}
					catch (Exception ex)
					{
						logger.LogWarning(ex, "Failed to read local script file {Path}, falling back to Perforce", localFileName);
					}
				}

				// If not using local edited version, get from Perforce cache
				if (!isLocalFile && !String.IsNullOrWhiteSpace(fileRecord.DepotFile))
				{
					string[]? scriptLines = await Utility.TryPrintFileUsingCacheAsync(perforce, fileRecord.DepotFile, cacheFolder, fileRecord.Digest, logger, cancellationToken);
					if (scriptLines == null)
					{
						logger.LogWarning("Failed to retrieve script from Perforce: {Script}", relativePath);
						return;
					}

					// Write cached script to temp location for execution
					localPath = Path.Combine(Path.GetTempPath(), $"ugs_inject_{Guid.NewGuid()}{Path.GetExtension(relativePath)}");
					await File.WriteAllLinesAsync(localPath, scriptLines, System.Text.Encoding.UTF8, cancellationToken);
					needsTempFile = true;
				}
				else if (!isLocalFile)
				{
					logger.LogWarning("Script {Script} exists in Perforce but has no depot path and no usable local file, skipping", relativePath);
					return;
				}
			}
			else
			{
				// File not in Perforce - check if it exists locally only
				if (File.Exists(localPath))
				{
					try
					{
						DateTime lastModifiedTime = File.GetLastWriteTimeUtc(localPath);
						localConfigFiles.Add(new KeyValuePair<FileReference, DateTime>(new FileReference(localPath), lastModifiedTime));
						isLocalFile = true;
						logger.LogInformation("Using local-only script (not in Perforce): {Script}", relativePath);
					}
					catch (Exception ex)
					{
						logger.LogWarning(ex, "Failed to read local-only script file {Path}", localPath);
						return;
					}
				}
				else
				{
					logger.LogWarning("Script not found in Perforce or locally: {Script}", relativePath);
					return;
				}
			}

			// Determine interpreter for script
			(string? interpreter, string? interpreterArgs) = GetInterpreterForScript(localPath, logger);
			if (interpreter == null)
			{
				logger.LogWarning("No interpreter found for script: {Script}", relativePath);
				if (needsTempFile)
				{
					try
					{
						File.Delete(localPath);
					}
					catch
					{
						// Ignore cleanup errors
					}
				}
				return;
			}

			// Make script executable on Unix platforms
			if (!RuntimeInformation.IsOSPlatform(OSPlatform.Windows) && interpreter == localPath)
			{
				MakeScriptExecutable(localPath, logger);
			}

			try
			{
				// Execute script
				List<string> capturedLines = new List<string>();
				string arguments;
				if (String.Equals(interpreter, localPath, StringComparison.OrdinalIgnoreCase))
				{
					// Direct execution (.exe or extensionless): the script is the
					// process itself, so only pass the workspace root as an argument.
					arguments = $"\"{localRootPath.FullName}\"";
				}
				else if (String.Equals(interpreter, "cmd", StringComparison.OrdinalIgnoreCase))
				{
					// cmd.exe /c strips the first and last quote characters when the
					// command string contains more than two quotes. Wrap the entire
					// command in an extra set of quotes so inner quoting is preserved
					// after stripping.
					arguments = $"{interpreterArgs} \"\"{localPath}\" \"{localRootPath.FullName}\"\"";
				}
				else if (String.IsNullOrEmpty(interpreterArgs))
				{
					// Interpreter without extra args (e.g. python, bash): pass the
					// script path followed by the workspace root.
					arguments = $"\"{localPath}\" \"{localRootPath.FullName}\"";
				}
				else
				{
					// Interpreter with args (e.g. pwsh -ExecutionPolicy Bypass -File):
					// pass interpreter args, then script path, then workspace root.
					arguments = $"{interpreterArgs} \"{localPath}\" \"{localRootPath.FullName}\"";
				}
				
				ClientRecord clientRecord = await perforce.GetClientAsync(null, cancellationToken);

				string clientName = clientRecord.Name;

				// Append additional script arguments if specified
				if (!String.IsNullOrEmpty(scriptArgs))
				{
					scriptArgs = scriptArgs.Replace("$(ClientName)", clientRecord.Name, StringComparison.OrdinalIgnoreCase);
					scriptArgs = scriptArgs.Replace("$(ClientRoot)", clientRecord.Root, StringComparison.OrdinalIgnoreCase);
					scriptArgs = scriptArgs.Replace("$(ProjectClientPath)", selectedClientFileName, StringComparison.OrdinalIgnoreCase);

					arguments = $"{arguments} {scriptArgs}";
				}

					int exitCode = await Utility.ExecuteProcessAsync(
					interpreter,
					localRootPath.FullName,
					arguments,
					line => capturedLines.Add(line),
					env: null,
					cancellationToken);

				// Parse output if successful
				if (exitCode == 0)
				{
					try
					{
						foreach (string line in capturedLines)
						{
							logger.LogDebug("Script '{Script}' output: {Line}", localPath, line);
						}

						configFile.Parse(capturedLines.ToArray(), projectIdentifier);
						logger.LogDebug("Injected config from script {Script} ({LineCount} lines)", relativePath, capturedLines.Count);
					}
					catch (Exception ex)
					{
						logger.LogWarning(ex, "Failed to parse output from script {Script}", relativePath);
					}
				}
				else
				{
					logger.LogWarning("Script {Script} exited with code {ExitCode}", relativePath, exitCode);
				}
			}
			finally
			{
				// Clean up temp file if needed (even on cancellation)
				if (needsTempFile)
				{
					try
					{
						File.Delete(localPath);
					}
					catch
					{
						// Ignore cleanup errors
					}
				}
			}
		}

		private static (string? interpreter, string? interpreterArgs) GetInterpreterForScript(string scriptPath, ILogger logger)
		{
			string extension = Path.GetExtension(scriptPath).ToLowerInvariant();

			switch (extension)
			{
				case ".exe":
				case "":
					// Direct execution
					return (scriptPath, null);

				case ".py":
					// Python script - check for python3 then python
					if (IsCommandAvailable("python3"))
					{
						return ("python3", null);
					}
					else if (IsCommandAvailable("python"))
					{
						return ("python", null);
					}
					else
					{
						logger.LogWarning("Python interpreter not found in PATH for script {Script}", scriptPath);
						return (null, null);
					}

				case ".ps1":
					// PowerShell script - check for pwsh then powershell
					if (IsCommandAvailable("pwsh"))
					{
						return ("pwsh", "-ExecutionPolicy Bypass -File");
					}
					else if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows) && IsCommandAvailable("powershell"))
					{
						return ("powershell", "-ExecutionPolicy Bypass -File");
					}
					else
					{
						logger.LogWarning("PowerShell interpreter not found in PATH for script {Script}", scriptPath);
						return (null, null);
					}

				case ".sh":
					// Shell script - check for bash then sh (Unix only)
					if (!RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
					{
						if (IsCommandAvailable("bash"))
						{
							return ("bash", null);
						}
						else if (IsCommandAvailable("sh"))
						{
							return ("sh", null);
						}
						else
						{
							logger.LogWarning("Shell interpreter not found in PATH for script {Script}", scriptPath);
							return (null, null);
						}
					}
					else
					{
						logger.LogWarning("Shell scripts (.sh) are not supported on Windows");
						return (null, null);
					}

				case ".bat":
				case ".cmd":
					// Batch file (Windows only)
					if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
					{
						return ("cmd", "/c");
					}
					else
					{
						logger.LogWarning("Batch files are not supported on non-Windows platforms");
						return (null, null);
					}

				default:
					logger.LogWarning("Unknown script extension: {Extension} for script {Script}", extension, scriptPath);
					return (null, null);
			}
		}

		private static bool IsCommandAvailable(string command)
		{
			try
			{
				string fileName = RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? "where" : "which";
				using Process process = new Process();
				process.StartInfo.FileName = fileName;
				process.StartInfo.Arguments = command;
				process.StartInfo.UseShellExecute = false;
				process.StartInfo.RedirectStandardOutput = true;
				process.StartInfo.RedirectStandardError = true;
				process.StartInfo.CreateNoWindow = true;
				process.Start();

				if (!process.WaitForExit(1000))
				{
					try
					{
						process.Kill();
					}
					catch (InvalidOperationException)
					{
						// Process already exited between WaitForExit and Kill
					}
					return false;
				}

				return process.ExitCode == 0;
			}
			catch
			{
				return false;
			}
		}

		private static void MakeScriptExecutable(string scriptPath, ILogger logger)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return;
			}

			try
			{
				using Process process = new Process();
				process.StartInfo.FileName = "chmod";
				process.StartInfo.Arguments = $"+x \"{scriptPath}\"";
				process.StartInfo.UseShellExecute = false;
				process.StartInfo.RedirectStandardOutput = true;
				process.StartInfo.RedirectStandardError = true;
				process.StartInfo.CreateNoWindow = true;
				process.Start();

				if (!process.WaitForExit(5000))
				{
					try
					{
						process.Kill();
					}
					catch (InvalidOperationException)
					{
						// Process already exited between WaitForExit and Kill
					}
					logger.LogDebug("chmod command timed out for {Script}", scriptPath);
				}
				else if (process.ExitCode != 0)
				{
					logger.LogDebug("chmod command failed for {Script} with exit code {ExitCode}", scriptPath, process.ExitCode);
				}
			}
			catch (Exception ex)
			{
				logger.LogDebug(ex, "Failed to make script executable: {Script}", scriptPath);
			}
		}
	}
}
