// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace UnrealGameSync
{
	public class FeatureFlag
	{
		public string Name { get; set; } = String.Empty;
		public bool Enabled { get; set; } = false;

		public void Import(FeatureFlag ff)
		{
			Name = ff.Name;
			Enabled = ff.Enabled;
		}
	}

	public class FeatureFlagCache
	{
		private IDictionary<string, FeatureFlag> Features { get; }

		public FeatureFlagCache(IDictionary<string, FeatureFlag> features)
		{
			Features = features;
		}

		public bool IsFeatureEnabled(string featureName)
		{
			if (Features.TryGetValue(featureName, out FeatureFlag? featureFlag))
			{
				return featureFlag.Enabled;
			}

			return false;
		}
	}
}
