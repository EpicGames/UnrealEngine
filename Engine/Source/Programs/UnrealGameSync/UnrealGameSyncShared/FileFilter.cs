// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;

namespace UnrealGameSync
{
	/// <summary>
	/// Indicates whether files which match a pattern should be included or excluded
	/// </summary>
	public enum FileFilterType
	{
		Include,
		Exclude,
	}

	/// <summary>
	/// Stores a set of rules, similar to p4 path specifications, which can be used to efficiently filter files in or out of a set.
	/// The rules are merged into a tree with nodes for each path fragments and rule numbers in each leaf, allowing tree traversal in an arbitrary order 
	/// to how they are applied.
	/// </summary>
	public class FileFilter
	{
		/// <summary>
		/// Root node for the tree.
		/// </summary>
		readonly FileFilterNode _rootNode;

		/// <summary>
		/// The default node, which will match any path
		/// </summary>
		readonly FileFilterNode _defaultNode;

		/// <summary>
		/// Terminating nodes for each rule added to the filter
		/// </summary>
		readonly List<FileFilterNode> _rules = new List<FileFilterNode>();

		/// <summary>
		/// Default constructor
		/// </summary>
		public FileFilter(FileFilterType defaultType = FileFilterType.Exclude)
		{
			_rootNode = new FileFilterNode(null, "");

			_defaultNode = new FileFilterNode(_rootNode, "...");
			_defaultNode.Type = defaultType;
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="other">Filter to copy from</param>
		public FileFilter(FileFilter other)
			: this(other._defaultNode.Type)
		{
			foreach (FileFilterNode otherRule in other._rules)
			{
				AddRule(otherRule.ToString(), otherRule.Type);
			}
		}

		/// <summary>
		/// Constructs a file filter from a p4-style filespec. Exclude lines are prefixed with a - character.
		/// </summary>
		/// <param name="lines">Lines to parse rules from</param>
		public FileFilter(IEnumerable<string> lines)
			: this()
		{
			foreach (string line in lines)
			{
				AddRule(line);
			}
		}

		/// <summary>
		/// Adds an include or exclude rule to the filter
		/// </summary>
		/// <param name="rule">Pattern to match. See CreateRegex() for details.</param>
		public void AddRule(string rule)
		{
			if (rule.StartsWith("-", StringComparison.Ordinal))
			{
				Exclude(rule.Substring(1).TrimStart());
			}
			else
			{
				Include(rule);
			}
		}

		/// <summary>
		/// Adds an include or exclude rule to the filter. The rule may be 
		/// </summary>
		/// <param name="rule">Pattern to match. See CreateRegex() for details.</param>
		/// <param name="allowTags"></param>
		public void AddRule(string rule, params string[] allowTags)
		{
			string cleanRule = rule.Trim();
			if (cleanRule.StartsWith("{", StringComparison.Ordinal))
			{
				// Find the end of the condition
				int conditionEnd = cleanRule.IndexOf('}', StringComparison.Ordinal);
				if (conditionEnd == -1)
				{
					throw new Exception(String.Format("Missing closing parenthesis in rule: {0}", cleanRule));
				}

				// Check there's a matching tag
				string[] ruleTags = cleanRule.Substring(1, conditionEnd - 1).Split(',').Select(x => x.Trim()).ToArray();
				if (!ruleTags.Any(x => allowTags.Contains(x)))
				{
					return;
				}

				// Strip the condition from the rule
				cleanRule = cleanRule.Substring(conditionEnd + 1).TrimStart();
			}
			AddRule(cleanRule);
		}

		/// <summary>
		/// Adds several rules to the filter
		/// </summary>
		/// <param name="rules">List of patterns to match.</param>
		public void AddRules(IEnumerable<string> rules)
		{
			foreach (string rule in rules)
			{
				AddRule(rule);
			}
		}

		/// <summary>
		/// Adds several rules in the given lines. Rules may be prefixed with conditions of the syntax {Key=Value, Key2=Value2}, which
		/// will be evaluated using variables in the given dictionary before being added.
		/// </summary>
		public void AddRules(IEnumerable<string> rules, params string[] tags)
		{
			foreach (string rule in rules)
			{
				AddRule(rule, tags);
			}
		}

		/// <summary>
		/// Reads a configuration file split into sections
		/// </summary>
		public void ReadRulesFromFile(string fileName, string sectionName, params string[] allowTags)
		{
			bool inSection = false;
			foreach (string line in File.ReadAllLines(fileName))
			{
				string trimLine = line.Trim();
				if (!trimLine.StartsWith(";", StringComparison.Ordinal) && trimLine.Length > 0)
				{
					if (trimLine.StartsWith("[", StringComparison.Ordinal))
					{
						inSection = (trimLine == "[" + sectionName + "]");
					}
					else if (inSection)
					{
						AddRule(line, allowTags);
					}
				}
			}
		}

		/// <summary>
		/// Adds an include rule to the filter
		/// </summary>
		/// <param name="pattern">Pattern to match. See CreateRegex() for details.</param>
		public void Include(string pattern)
		{
			AddRule(pattern, FileFilterType.Include);
		}

		/// <summary>
		/// Adds several exclude rules to the filter
		/// </summary>
		/// <param name="patterns">Patterns to match. See CreateRegex() for details.</param>
		public void Include(IEnumerable<string> patterns)
		{
			foreach (string pattern in patterns)
			{
				Include(pattern);
			}
		}

		/// <summary>
		/// Adds an exclude rule to the filter
		/// </summary>
		/// <param name="pattern">Mask to match. See CreateRegex() for details.</param>
		public void Exclude(string pattern)
		{
			AddRule(pattern, FileFilterType.Exclude);
		}

		/// <summary>
		/// Adds several exclude rules to the filter
		/// </summary>
		/// <param name="patterns">Patterns to match. See CreateRegex() for details.</param>
		public void Exclude(IEnumerable<string> patterns)
		{
			foreach (string pattern in patterns)
			{
				Exclude(pattern);
			}
		}

		/// <summary>
		/// Adds an include or exclude rule to the filter
		/// </summary>
		/// <param name="pattern">The pattern which the rule should match</param>
		/// <param name="type">Whether to include or exclude files matching this rule</param>
		public void AddRule(string pattern, FileFilterType type)
		{
			string normalizedPattern = pattern.Replace('\\', '/');

			// We don't want a slash at the start, but if there was not one specified, it's not anchored to the root of the tree.
			if (normalizedPattern.StartsWith("/", StringComparison.Ordinal))
			{
				normalizedPattern = normalizedPattern.Substring(1);
			}
			else if (!normalizedPattern.StartsWith("...", StringComparison.Ordinal))
			{
				normalizedPattern = ".../" + normalizedPattern;
			}

			// All directories indicate a wildcard match
			if (normalizedPattern.EndsWith("/", StringComparison.Ordinal))
			{
				normalizedPattern += "...";
			}

			// Replace any directory wildcards mid-string
			for (int idx = normalizedPattern.IndexOf("...", StringComparison.Ordinal); idx != -1; idx = normalizedPattern.IndexOf("...", idx, StringComparison.Ordinal))
			{
				if (idx > 0 && normalizedPattern[idx - 1] != '/')
				{
					normalizedPattern = normalizedPattern.Insert(idx, "*/");
					idx++;
				}

				idx += 3;

				if (idx < normalizedPattern.Length && normalizedPattern[idx] != '/')
				{
					normalizedPattern = normalizedPattern.Insert(idx, "/*");
					idx += 2;
				}
			}

			// Split the pattern into fragments
			string[] branchPatterns = normalizedPattern.Split('/');

			// Add it into the tree
			FileFilterNode lastNode = _rootNode;
			foreach (string branchPattern in branchPatterns)
			{
				FileFilterNode? nextNode = lastNode.Branches.FirstOrDefault(x => x.Pattern == branchPattern);
				if (nextNode == null)
				{
					nextNode = new FileFilterNode(lastNode, branchPattern);
					lastNode.Branches.Add(nextNode);
				}
				lastNode = nextNode;
			}

			// We've reached the end of the pattern, so mark it as a leaf node
			_rules.Add(lastNode);
			lastNode.RuleNumber = _rules.Count - 1;
			lastNode.Type = type;

			// Update the maximums along that path
			for (FileFilterNode? updateNode = lastNode; updateNode != null; updateNode = updateNode.Parent)
			{
				if (type == FileFilterType.Include)
				{
					updateNode.MaxIncludeRuleNumber = lastNode.RuleNumber;
				}
				else
				{
					updateNode.MaxExcludeRuleNumber = lastNode.RuleNumber;
				}
			}
		}

		/// <summary>
		/// Excludes all confidential folders from the filter
		/// </summary>
		public void ExcludeConfidentialFolders()
		{
			AddRule(".../EpicInternal/...", FileFilterType.Exclude);
			AddRule(".../CarefullyRedist/...", FileFilterType.Exclude);
			AddRule(".../NotForLicensees/...", FileFilterType.Exclude);
			AddRule(".../NoRedist/...", FileFilterType.Exclude);
		}

		/// <summary>
		/// Determines whether the given file matches the filter
		/// </summary>
		/// <param name="fileName">File to match</param>
		/// <returns>True if the file passes the filter</returns>
		public bool Matches(string fileName)
		{
			string[] tokens = fileName.TrimStart('/', '\\').Split('/', '\\');

			FileFilterNode matchingNode = FindMatchingNode(_rootNode, tokens, 0, _defaultNode);

			return matchingNode.Type == FileFilterType.Include;
		}

		/// <summary>
		/// Determines whether it's possible for anything within the given folder name to match the filter. Useful to early out of recursive file searches.
		/// </summary>
		/// <param name="folderName">File to match</param>
		/// <returns>True if the file passes the filter</returns>
		public bool PossiblyMatches(string folderName)
		{
			string[] tokens = folderName.Trim('/', '\\').Split('/', '\\');

			FileFilterNode matchingNode = FindMatchingNode(_rootNode, tokens.Union(new string[] { "" }).ToArray(), 0, _defaultNode);

			return matchingNode.Type == FileFilterType.Include || HighestPossibleIncludeMatch(_rootNode, tokens, 0, matchingNode.RuleNumber) > matchingNode.RuleNumber;
		}

		/// <summary>
		/// Applies the filter to each element in a sequence, and returns the list of files that match
		/// </summary>
		/// <param name="fileNames">List of filenames</param>
		/// <returns>List of filenames which match the filter</returns>
		public IEnumerable<string> ApplyTo(IEnumerable<string> fileNames)
		{
			return fileNames.Where(x => Matches(x));
		}

		/// <summary>
		/// Finds the node which matches a given list of tokens.
		/// </summary>
		/// <param name="currentNode"></param>
		/// <param name="tokens"></param>
		/// <param name="tokenIdx"></param>
		/// <param name="currentBestNode"></param>
		/// <returns></returns>
		static FileFilterNode FindMatchingNode(FileFilterNode currentNode, string[] tokens, int tokenIdx, FileFilterNode currentBestNode)
		{
			// If we've matched all the input tokens, check if this rule is better than any other we've seen
			if (tokenIdx == tokens.Length)
			{
				return (currentNode.RuleNumber > currentBestNode.RuleNumber) ? currentNode : currentBestNode;
			}

			// If there is no rule under the current node which is better than the current best node, early out
			if (currentNode.MaxIncludeRuleNumber <= currentBestNode.RuleNumber && currentNode.MaxExcludeRuleNumber <= currentBestNode.RuleNumber)
			{
				return currentBestNode;
			}

			// Test all the branches for one that matches
			FileFilterNode bestNode = currentBestNode;
			foreach (FileFilterNode branch in currentNode.Branches)
			{
				if (branch.Pattern == "...")
				{
					for (int nextTokenIdx = tokens.Length; nextTokenIdx >= tokenIdx; nextTokenIdx--)
					{
						bestNode = FindMatchingNode(branch, tokens, nextTokenIdx, bestNode);
					}
				}
				else
				{
					if (branch.IsMatch(tokens[tokenIdx]))
					{
						bestNode = FindMatchingNode(branch, tokens, tokenIdx + 1, bestNode);
					}
				}
			}
			return bestNode;
		}

		/// <summary>
		/// Returns the highest possible rule number which can match the given list of input tokens, assuming that the list of input tokens is incomplete.
		/// </summary>
		/// <param name="currentNode">The current node being checked</param>
		/// <param name="tokens">The tokens to match</param>
		/// <param name="tokenIdx">Current token index</param>
		/// <param name="currentBestRuleNumber">The highest rule number seen so far. Used to optimize tree traversals.</param>
		/// <returns>New highest rule number</returns>
		static int HighestPossibleIncludeMatch(FileFilterNode currentNode, string[] tokens, int tokenIdx, int currentBestRuleNumber)
		{
			// If we've matched all the input tokens, check if this rule is better than any other we've seen
			if (tokenIdx == tokens.Length)
			{
				return Math.Max(currentBestRuleNumber, currentNode.MaxIncludeRuleNumber);
			}

			// Test all the branches for one that matches
			int bestRuleNumber = currentBestRuleNumber;
			if (currentNode.MaxIncludeRuleNumber > bestRuleNumber)
			{
				foreach (FileFilterNode branch in currentNode.Branches)
				{
					if (branch.Pattern == "...")
					{
						if (branch.MaxIncludeRuleNumber > bestRuleNumber)
						{
							bestRuleNumber = branch.MaxIncludeRuleNumber;
						}
					}
					else
					{
						if (branch.IsMatch(tokens[tokenIdx]))
						{
							bestRuleNumber = HighestPossibleIncludeMatch(branch, tokens, tokenIdx + 1, bestRuleNumber);
						}
					}
				}
			}
			return bestRuleNumber;
		}
	}

	/// <summary>
	/// Node within a filter tree. Each node matches a single path fragment - a folder or file name, with an include or exclude rule consisting of a sequence of nodes. 
	/// </summary>
	class FileFilterNode
	{
		/// <summary>
		/// Node which this is parented to. Null for the root node.
		/// </summary>
		public readonly FileFilterNode? Parent;

		/// <summary>
		/// Pattern to match for this node. May contain * or ? wildcards.
		/// </summary>
		public readonly string Pattern;

		/// <summary>
		/// Highest include rule number matched by this node or any child nodes.
		/// </summary>
		public int MaxIncludeRuleNumber = -1;

		/// <summary>
		/// Highest exclude rule number matched by this node or any child nodes.
		/// </summary>
		public int MaxExcludeRuleNumber = -1;

		/// <summary>
		/// Child branches of this node, distinct by the the pattern for each.
		/// </summary>
		public List<FileFilterNode> Branches = new List<FileFilterNode>();

		/// <summary>
		/// If a path matches the sequence of nodes ending in this node, the number of the rule which added it. -1 if this was not the last node in a rule.
		/// </summary>
		public int RuleNumber;

		/// <summary>
		/// Whether a rule terminating in this node should include files or exclude files.
		/// </summary>
		public FileFilterType Type;

		/// <summary>
		/// Default constructor.
		/// </summary>
		public FileFilterNode(FileFilterNode? inParent, string inPattern)
		{
			Parent = inParent;
			Pattern = inPattern;
			RuleNumber = -1;
			Type = FileFilterType.Exclude;
		}

		/// <summary>
		/// Determine if the given token matches the pattern in this node
		/// </summary>
		public bool IsMatch(string token)
		{
			if (Pattern.EndsWith(".", StringComparison.Ordinal))
			{
				return !token.Contains('.', StringComparison.Ordinal) && IsMatch(token, 0, Pattern.Substring(0, Pattern.Length - 1), 0);
			}
			else
			{
				return IsMatch(token, 0, Pattern, 0);
			}
		}

		/// <summary>
		/// Determine if a given token matches a pattern, starting from the given positions within each string.
		/// </summary>
		static bool IsMatch(string token, int tokenIdx, string pattern, int patternIdx)
		{
			for (; ; )
			{
				if (patternIdx == pattern.Length)
				{
					return (tokenIdx == token.Length);
				}
				else if (pattern[patternIdx] == '*')
				{
					for (int nextTokenIdx = token.Length; nextTokenIdx >= tokenIdx; nextTokenIdx--)
					{
						if (IsMatch(token, nextTokenIdx, pattern, patternIdx + 1))
						{
							return true;
						}
					}
					return false;
				}
				else if (tokenIdx < token.Length && (Char.ToLower(token[tokenIdx]) == Char.ToLower(pattern[patternIdx]) || pattern[patternIdx] == '?'))
				{
					tokenIdx++;
					patternIdx++;
				}
				else
				{
					return false;
				}
			}
		}

		/// <summary>
		/// Debugger visualization
		/// </summary>
		/// <returns>Path to this node</returns>
		public override string ToString()
		{
			return (Parent == null) ? "/" : Parent.ToString().TrimEnd('/') + "/" + Pattern;
		}
	}
}
