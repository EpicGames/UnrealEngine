// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	public class FilterSettings
	{
		public List<Guid> IncludeCategories { get; init; } = new List<Guid>();
		public List<Guid> ExcludeCategories { get; init; } = new List<Guid>();
		public List<string> View { get; init; } = new List<string>();
		public bool? AllProjects { get; set; }
		public bool? LocalProjects { get; set; }
		public bool? AllProjectsInSln { get; set; }
		public bool? UprojectSpecificSln { get; set; }

		public void Reset()
		{
			IncludeCategories.Clear();
			ExcludeCategories.Clear();
			View.Clear();
			AllProjects = null;
			LocalProjects = null;
			AllProjectsInSln = null;
			UprojectSpecificSln = null;
		}

		public void SetCategories(Dictionary<Guid, bool> categories)
		{
			IncludeCategories.Clear();
			IncludeCategories.AddRange(categories.Where(x => x.Value).Select(x => x.Key));

			ExcludeCategories.Clear();
			ExcludeCategories.AddRange(categories.Where(x => !x.Value).Select(x => x.Key));
		}

		public Dictionary<Guid, bool> GetCategories()
		{
			Dictionary<Guid, bool> categories = new Dictionary<Guid, bool>();
			foreach (Guid includeCategory in IncludeCategories)
			{
				categories[includeCategory] = true;
			}
			foreach (Guid excludeCategory in ExcludeCategories)
			{
				categories[excludeCategory] = false;
			}
			return categories;
		}
	}

	public class GlobalSettings
	{
		public string? LatestVersion { get; set; }
		public DateTime LastVersionCheck { get; set; }
		public PerforceSyncOptions Perforce { get; set; } = new PerforceSyncOptions();
		public FilterSettings Filter { get; set; } = new FilterSettings();
		public bool AutoResolveConflicts { get; set; } = true;
		public bool AlwaysClobberFiles { get; set; } = false;
		public bool AlwaysDeleteFiles { get; set; } = false;
	}

	public class GlobalSettingsFile
	{
		public FileReference File { get; }
		public GlobalSettings Global { get; }

		public UserProjectSettings FindOrAddProjectSettings(ProjectInfo projectInfo, UserWorkspaceSettings settings, ILogger logger)
		{
			FileReference configFile;
			if (projectInfo.LocalFileName.HasExtension(".uprojectdirs"))
			{
				configFile = FileReference.Combine(UserSettings.GetConfigDir(settings.RootDir), "project.json");
			}
			else
			{
				configFile = FileReference.Combine(UserSettings.GetConfigDir(settings.RootDir), $"project_{projectInfo.LocalFileName.GetFileNameWithoutExtension()}.json");
			}

			UserSettings.CreateConfigDir(configFile.Directory);

			UserProjectSettings? projectSettings;
			if (!UserProjectSettings.TryLoad(configFile, out projectSettings))
			{
				projectSettings = new UserProjectSettings(configFile);
				ImportProjectSettings(projectInfo, projectSettings);
				projectSettings.Save(logger);
			}
			return projectSettings;
		}

		protected virtual void ImportProjectSettings(ProjectInfo projectInfo, UserProjectSettings projectSettings)
		{
		}

		protected virtual void ImportWorkspaceSettings(DirectoryReference rootDir, string clientName, string branchPath, UserWorkspaceSettings workspaceSettings)
		{
		}

		protected virtual void ImportWorkspaceState(DirectoryReference rootDir, string clientName, string branchPath, WorkspaceState workspaceState)
		{
		}

		public GlobalSettingsFile(FileReference file, GlobalSettings global)
		{
			File = file;
			Global = global;
		}

		public static GlobalSettingsFile Create(FileReference file)
		{
			GlobalSettings? data;
			if (!Utility.TryLoadJson(file, out data))
			{
				data = new GlobalSettings();
			}
			return new GlobalSettingsFile(file, data);
		}

		public virtual bool Save(ILogger logger)
		{
			try
			{
				Utility.SaveJson(File, Global);
				return true;
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Unable to save {File}: {Message}", File, ex.Message);
				return false;
			}
		}

		public bool Save(ILogger logger, FileReference globalSettingsJsonFile)
		{
			try
			{
				Utility.SaveJson(globalSettingsJsonFile, Global);
				return true;
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Unable to save {File}: {Message}", globalSettingsJsonFile, ex.Message);
				return false;
			}
		}

		public WorkspaceStateWrapper FindOrAddWorkspaceState(UserWorkspaceSettings settings)
		{
			return FindOrAddWorkspaceState(settings.RootDir, settings.ClientName, settings.BranchPath);
		}

		public WorkspaceStateWrapper FindOrAddWorkspaceState(DirectoryReference rootDir, string clientName, string branchPath)
		{
			return new WorkspaceStateWrapper(rootDir, () =>
			{
				WorkspaceState state = new WorkspaceState();
				ImportWorkspaceState(rootDir, clientName, branchPath, state);
				return state;
			});
		}

		public WorkspaceStateWrapper FindOrAddWorkspaceState(ProjectInfo projectInfo, UserWorkspaceSettings settings)
		{
			WorkspaceStateWrapper wrapper = FindOrAddWorkspaceState(projectInfo.LocalRootPath, projectInfo.ClientName, projectInfo.BranchPath);
			wrapper.Modify(x => x.UpdateCachedProjectInfo(projectInfo, settings.LastModifiedTimeUtc));
			return wrapper;
		}

		public UserWorkspaceSettings FindOrAddWorkspaceSettings(DirectoryReference rootDir, string? serverAndPort, string? userName, string clientName, string branchPath, string projectPath, string localPath, ILogger logger)
		{
			ProjectInfo.ValidateBranchPath(branchPath);
			ProjectInfo.ValidateProjectPath(projectPath);

			UserWorkspaceSettings? settings;
			if (!UserWorkspaceSettings.TryLoad(rootDir, out settings))
			{
				settings = new UserWorkspaceSettings();
				settings.RootDir = rootDir;
				ImportWorkspaceSettings(rootDir, clientName, branchPath, settings);
			}

			settings.Init(serverAndPort, userName, clientName, branchPath, projectPath, localPath);
			settings.Save(logger);

			return settings;
		}

		public static string[] GetCombinedSyncFilter(
			Dictionary<Guid, WorkspaceSyncCategory> uniqueIdToFilter,
			string roleName,
			IDictionary<string, Preset> roles,
			FilterSettings globalFilter,
			FilterSettings workspaceFilter,
			ConfigSection? perforceSection)
		{
			List<string> lines = new List<string>();
			foreach (string viewLine in Enumerable.Concat(globalFilter.View, workspaceFilter.View).Select(x => x.Trim()).Where(x => x.Length > 0 && !x.StartsWith(";", StringComparison.Ordinal)))
			{
				lines.Add(viewLine);
			}

			if (roles.TryGetValue(roleName, out Preset? role))
			{
				foreach (string roleView in role.Views)
				{
					lines.Add(roleView);
				}
			}

			Dictionary<Guid, bool> globalCategoryIdToSetting = globalFilter.GetCategories();
			Dictionary<Guid, bool> workspaceCategoryIdToSetting = workspaceFilter.GetCategories();

			HashSet<Guid> enabled = new HashSet<Guid>();
			foreach (WorkspaceSyncCategory filter in uniqueIdToFilter.Values)
			{
				bool enable = filter.Enable;

				if (globalCategoryIdToSetting.TryGetValue(filter.UniqueId, out bool globalEnable))
				{
					enable = globalEnable;
				}

				if (workspaceCategoryIdToSetting.TryGetValue(filter.UniqueId, out bool workspaceEnable))
				{
					enable = workspaceEnable;
				}	

				// role override anything set at the global or workspace level
				if (role != null)
				{
					if (role.Categories.TryGetValue(filter.UniqueId, out RoleCategory? roleCategory))
					{
						enable = roleCategory.Enabled;
					}
				}
				
				if (enable)
				{
					EnableFilter(filter.UniqueId, enabled, uniqueIdToFilter);
				}
			}

			foreach (WorkspaceSyncCategory filter in uniqueIdToFilter.Values.OrderBy(x => x.Name))
			{
				if (!enabled.Contains(filter.UniqueId))
				{
					foreach (string path in filter.Paths.Select(x => "-" + x.Trim()))
					{
						lines.Add(path);
					}
				}
			}

			if (perforceSection != null)
			{
				IEnumerable<string> additionalPaths = perforceSection.GetValues("AdditionalPathsToSync", Array.Empty<string>());
				lines.AddRange(additionalPaths);
			}

			return lines.ToArray();
		}

		static void EnableFilter(Guid uniqueId, HashSet<Guid> enabled, Dictionary<Guid, WorkspaceSyncCategory> uniqueIdToFilter)
		{
			if (enabled.Add(uniqueId))
			{
				WorkspaceSyncCategory? category;
				if (uniqueIdToFilter.TryGetValue(uniqueId, out category))
				{
					foreach (Guid requiresUniqueId in category.Requires)
					{
						EnableFilter(requiresUniqueId, enabled, uniqueIdToFilter);
					}
				}
			}
		}
	}
}
