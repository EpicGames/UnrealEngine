// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Text.RegularExpressions;
using EpicGames.Horde;
using Microsoft.Win32;

namespace UnrealGameSync
{
	public enum LauncherUpdateSource
	{
		Unknown = 0,
		Perforce = 1,
		Horde = 2,
		None = 3,
	}

	public class LauncherSettings
	{
		public LauncherUpdateSource UpdateSource { get; set; }

		public string? HordeServer { get; set; }

		public string? PerforceServerAndPort { get; set; }
		public string? PerforceUserName { get; set; }
		public string? PerforceDepotPath { get; set; }

		public bool PreviewBuild { get; set; }

		public LauncherSettings()
		{
			UpdateSource = DeploymentSettings.Instance.UpdateSource;
			HordeServer = DeploymentSettings.Instance.HordeUrl;
			PerforceServerAndPort = DeploymentSettings.Instance.DefaultPerforceServer;
			PerforceDepotPath = DeploymentSettings.Instance.DefaultDepotPath;
		}

		public LauncherSettings(LauncherSettings other)
		{
			UpdateSource = other.UpdateSource;
			HordeServer = other.HordeServer;
			PerforceServerAndPort = other.PerforceServerAndPort;
			PerforceUserName = other.PerforceUserName;
			PerforceDepotPath = other.PerforceDepotPath;

			PreviewBuild = other.PreviewBuild;
		}

		public void Read()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				ReadFromRegistry();
			}
		}

		[SupportedOSPlatform("windows")]
		void ReadFromRegistry()
		{
			Uri? defaultServerUrl = HordeOptions.GetDefaultServerUrl();
			if (defaultServerUrl != null)
			{
				HordeServer = defaultServerUrl.ToString();
			}

			using (RegistryKey? key = Registry.CurrentUser.OpenSubKey("SOFTWARE\\Epic Games\\UnrealGameSync", false))
			{
				if (key != null)
				{
					LauncherUpdateSource updateSource;
					if (Enum.TryParse(key.GetValue("Source", UpdateSource) as string, out updateSource))
					{
						UpdateSource = updateSource;
					}

					PerforceServerAndPort = key.GetValue("ServerAndPort", PerforceServerAndPort) as string;
					PerforceUserName = key.GetValue("UserName", PerforceUserName) as string;
					PerforceDepotPath = key.GetValue("DepotPath", PerforceDepotPath) as string;
					PreviewBuild = ((key.GetValue("Preview", PreviewBuild ? 1 : 0) as int?) ?? 0) != 0;

					// Fix corrupted depot path string
					if (PerforceDepotPath != null)
					{
						Match match = Regex.Match(PerforceDepotPath, "^(.*)/(Release|UnstableRelease)/\\.\\.\\.@.*$");
						if (match.Success)
						{
							PerforceDepotPath = match.Groups[1].Value;
							Save();
						}
					}
				}
			}
		}

		public bool Save()
		{
			try
			{
				if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
				{
					SaveToRegistry();
				}
				return true;
			}
			catch
			{
				return false;
			}
		}

		[SupportedOSPlatform("windows")]
		void SaveToRegistry()
		{
			if (!String.IsNullOrEmpty(HordeServer))
			{
				HordeOptions.SetDefaultServerUrl(new Uri(HordeServer));
			}

			using (RegistryKey key = Registry.CurrentUser.CreateSubKey("SOFTWARE\\Epic Games\\UnrealGameSync"))
			{
				// Delete this legacy setting
				Utility.DeleteRegistryKey(key, "Server");

				SaveRegistryValue(key, "Source", UpdateSource.ToString(), DeploymentSettings.Instance.UpdateSource.ToString());
				SaveRegistryValue(key, "ServerAndPort", PerforceServerAndPort, DeploymentSettings.Instance.DefaultPerforceServer);
				SaveRegistryValue(key, "UserName", PerforceUserName, null);
				SaveRegistryValue(key, "DepotPath", PerforceDepotPath, DeploymentSettings.Instance.DefaultDepotPath);

				if (PreviewBuild)
				{
					key.SetValue("Preview", 1);
				}
				else
				{
					Utility.DeleteRegistryKey(key, "Preview");
				}
			}
		}

		[SupportedOSPlatform("windows")]
		static void SaveRegistryValue(RegistryKey key, string name, string? value, string? defaultValue)
		{
			if (String.IsNullOrEmpty(value) || String.Equals(value, defaultValue, StringComparison.OrdinalIgnoreCase))
			{
				Utility.DeleteRegistryKey(key, name);
			}
			else
			{
				key.SetValue(name, value);
			}
		}
	}
}
