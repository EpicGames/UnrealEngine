// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	public class OpenProjectInfo
	{
		public UserSelectedProjectSettings SelectedProject { get; }

		public IPerforceSettings PerforceSettings { get; }
		public ProjectInfo ProjectInfo { get; }
		public UserWorkspaceSettings WorkspaceSettings { get; }
		public WorkspaceStateWrapper WorkspaceStateWrapper { get; }
		public ConfigFile LatestProjectConfigFile { get; }
		public ConfigFile WorkspaceProjectConfigFile { get; }
		public IReadOnlyList<string>? WorkspaceProjectStreamFilter { get; }
		public List<KeyValuePair<FileReference, DateTime>> LocalConfigFiles { get; }
		public bool GenerateP4Config { get; }

		public OpenProjectInfo(UserSelectedProjectSettings selectedProject, IPerforceSettings perforceSettings, ProjectInfo projectInfo, UserWorkspaceSettings workspaceSettings, WorkspaceStateWrapper workspaceStateWrapper, ConfigFile latestProjectConfigFile, ConfigFile workspaceProjectConfigFile, IReadOnlyList<string>? workspaceProjectStreamFilter, List<KeyValuePair<FileReference, DateTime>> localConfigFiles, bool generateP4Config)
		{
			SelectedProject = selectedProject;

			PerforceSettings = perforceSettings;
			ProjectInfo = projectInfo;
			WorkspaceSettings = workspaceSettings;
			WorkspaceStateWrapper = workspaceStateWrapper;
			LatestProjectConfigFile = latestProjectConfigFile;
			WorkspaceProjectConfigFile = workspaceProjectConfigFile;
			WorkspaceProjectStreamFilter = workspaceProjectStreamFilter;
			LocalConfigFiles = localConfigFiles;
			GenerateP4Config = generateP4Config;
		}

		public static async Task<OpenProjectInfo> CreateAsync(IPerforceSettings defaultPerforceSettings, UserSelectedProjectSettings selectedProject, UserSettings userSettings, bool generateP4Config, ILogger<OpenProjectInfo> logger, CancellationToken cancellationToken)
		{
			PerforceSettings perforceSettings = Utility.OverridePerforceSettings(defaultPerforceSettings, selectedProject.ServerAndPort, selectedProject.UserName);
			using IPerforceConnection perforce = await PerforceConnection.CreateAsync(perforceSettings, logger);

			// Make sure we're logged in
			PerforceResponse<LoginRecord> loginState = await perforce.TryGetLoginStateAsync(cancellationToken);
			if (!loginState.Succeeded)
			{
				throw new UserErrorException("User is not logged in to Perforce.");
			}

			// Execute like a regular task
			return await CreateAsync(perforce, selectedProject, userSettings, generateP4Config, logger, cancellationToken);
		}

		public static async Task<OpenProjectInfo> CreateAsync(IPerforceConnection defaultConnection, UserSelectedProjectSettings selectedProject, UserSettings userSettings, bool generateP4Config, ILogger<OpenProjectInfo> logger, CancellationToken cancellationToken)
		{
			using IDisposable? loggerScope = logger.BeginScope("Project {SelectedProject}", selectedProject.ToString());
			logger.LogInformation("Detecting settings for {Project}", selectedProject);

			// Use the cached client path to the file if it's available; it's much quicker than trying to find the correct workspace.
			IPerforceConnection? perforceClient = null;
			try
			{
				IPerforceSettings perforceSettings;

				FileReference newSelectedFileName;
				string newSelectedClientFileName;
				if (!String.IsNullOrEmpty(selectedProject.ClientPath))
				{
					// Get the client path
					newSelectedClientFileName = selectedProject.ClientPath;

					// Get the client name
					string? clientName;
					if (!PerforceUtils.TryGetClientName(newSelectedClientFileName, out clientName))
					{
						throw new UserErrorException($"Couldn't get client name from {newSelectedClientFileName}");
					}

					// Create the client
					perforceSettings = new PerforceSettings(defaultConnection.Settings) { ClientName = clientName };
					perforceClient = await PerforceConnection.CreateAsync(perforceSettings, logger);

					// Figure out the path on the client. Use the cached location if it's valid.
					string? localPath = selectedProject.LocalPath;
					if (localPath == null || !File.Exists(localPath))
					{
						List<WhereRecord> records = await perforceClient.WhereAsync(newSelectedClientFileName, cancellationToken).Where(x => !x.Unmap).ToListAsync(cancellationToken);
						if (records.Count != 1)
						{
							throw new UserErrorException($"Couldn't get client path for {newSelectedClientFileName}");
						}
						localPath = Path.GetFullPath(records[0].Path);
					}
					newSelectedFileName = new FileReference(localPath);
				}
				else
				{
					// Get the perforce server settings
					InfoRecord perforceInfo = await defaultConnection.GetInfoAsync(InfoOptions.ShortOutput, cancellationToken);

					// Use the path as the selected filename
					newSelectedFileName = new FileReference(selectedProject.LocalPath!);

					// Make sure the project exists
					if (!FileReference.Exists(newSelectedFileName))
					{
						throw new UserErrorException($"{selectedProject.LocalPath} does not exist.");
					}

					// Find all the clients for this user
					logger.LogInformation("Enumerating clients for {UserName}...", perforceInfo.UserName);

					List<ClientsRecord> clients = await defaultConnection.GetClientsAsync(ClientsOptions.None, defaultConnection.Settings.UserName, cancellationToken);

					List<IPerforceSettings> candidateClients = await FilterClients(clients, newSelectedFileName, defaultConnection.Settings, perforceInfo.ClientHost, logger, cancellationToken);
					if (candidateClients.Count == 0)
					{
						// Search through all workspaces. We may find a suitable workspace which is for any user.
						logger.LogInformation("Enumerating shared clients...");
						clients = await defaultConnection.GetClientsAsync(ClientsOptions.None, "", cancellationToken);

						// Filter this list of clients
						candidateClients = await FilterClients(clients, newSelectedFileName, defaultConnection.Settings, perforceInfo.ClientHost, logger, cancellationToken);

#pragma warning disable CA1508 // False positive? "warning CA1508: 'newCandidateClients.Count == 0' is always 'true'. Remove or refactor the condition(s) to avoid dead code."
						// If we still couldn't find any, fail.
						if (candidateClients.Count == 0)
						{
							throw new UserErrorException($"Couldn't find any Perforce workspace containing {newSelectedFileName}. Check your connection settings.");
						}
#pragma warning restore CA1508
					}

					// Check there's only one client
					if (candidateClients.Count > 1)
					{
						throw new UserErrorException(String.Format("Found multiple workspaces containing {0}:\n\n{1}\n\nCannot determine which to use.", Path.GetFileName(newSelectedFileName.GetFileName()), String.Join("\n", candidateClients.Select(x => x.ClientName))));
					}

					// Take the client we've chosen
					perforceSettings = candidateClients[0];
					perforceClient = await PerforceConnection.CreateAsync(perforceSettings, logger);

					// Get the client path for the project file
					List<WhereRecord> records = await perforceClient.WhereAsync(newSelectedFileName.FullName, cancellationToken).Where(x => !x.Unmap).ToListAsync(cancellationToken);
					if (records.Count == 0)
					{
						throw new UserErrorException("File is not mapped to any client");
					}
					else if (records.Count > 1)
					{
						throw new UserErrorException($"File is mapped to {records.Count} locations: {String.Join(", ", records.Select(x => x.Path))}");
					}

					newSelectedClientFileName = records[0].ClientFile;
				}

				// Make sure the drive containing the project exists, to prevent other errors down the line
				string pathRoot = Path.GetPathRoot(newSelectedFileName.FullName)!;
				if (!Directory.Exists(pathRoot))
				{
					throw new UserErrorException($"Path '{newSelectedFileName}' is invalid");
				}

				// Make sure the path case is correct. This can cause UBT intermediates to be out of date if the case mismatches.
				newSelectedFileName = FileReference.FindCorrectCase(newSelectedFileName);

				// Update the selected project with all the data we've found
				selectedProject = new UserSelectedProjectSettings(selectedProject.ServerAndPort, selectedProject.UserName, selectedProject.Type, newSelectedClientFileName, newSelectedFileName.FullName);

				// Get the local branch root
				string? branchClientPath = null;
				DirectoryReference? branchDirectoryName = null;

				// Figure out where the engine is in relation to it
				int endIdx = newSelectedClientFileName.Length - 1;
				if (endIdx != -1 && newSelectedClientFileName.EndsWith(".uproject", StringComparison.InvariantCultureIgnoreCase))
				{
					endIdx = newSelectedClientFileName.LastIndexOf('/') - 1;
				}
				for (; endIdx >= 2; endIdx--)
				{
					if (newSelectedClientFileName[endIdx] == '/')
					{
						List<PerforceResponse<FStatRecord>> fileRecords = await perforceClient.TryFStatAsync(FStatOptions.None, newSelectedClientFileName.Substring(0, endIdx) + "/Engine/Build/Build.version", cancellationToken).ToListAsync(cancellationToken);
						if (fileRecords.Succeeded() && fileRecords.Count > 0)
						{
							FStatRecord fileRecord = fileRecords[0].Data;
							if (fileRecord.ClientFile == null)
							{
								throw new UserErrorException($"Missing client path for {fileRecord.DepotFile}");
							}

							branchClientPath = newSelectedClientFileName.Substring(0, endIdx);
							branchDirectoryName = new FileReference(fileRecord.ClientFile).Directory.ParentDirectory?.ParentDirectory;
							break;
						}
					}
				}
				if (branchClientPath == null || branchDirectoryName == null)
				{
					throw new UserErrorException($"Could not find engine in Perforce relative to project path ({newSelectedClientFileName})");
				}

				logger.LogInformation("Found branch root at {RootPath}", branchClientPath);

				// Read the existing workspace settings from disk, and update them with any info computed here
				int branchIdx = branchClientPath.IndexOf('/', 2);
				string branchPath = (branchIdx == -1) ? String.Empty : branchClientPath.Substring(branchIdx);
				string projectPath = newSelectedClientFileName.Substring(branchClientPath.Length);

				UserWorkspaceSettings userWorkspaceSettings = userSettings.FindOrAddWorkspaceSettings(branchDirectoryName, perforceSettings.ServerAndPort, perforceSettings.UserName, perforceSettings.ClientName!, branchPath, projectPath, selectedProject.LocalPath ?? String.Empty, logger);

				// Now compute the updated project info
				ProjectInfo projectInfo = await ProjectInfo.CreateAsync(perforceClient, userWorkspaceSettings, cancellationToken);

				// Update the cached workspace state
				WorkspaceStateWrapper? workspaceStateWrapper = null;
				try
				{
#pragma warning disable CA2000
					workspaceStateWrapper = userSettings.FindOrAddWorkspaceState(projectInfo, userWorkspaceSettings);
#pragma warning restore CA2000

					// Read the initial config file
					List<KeyValuePair<FileReference, DateTime>> localConfigFiles = new List<KeyValuePair<FileReference, DateTime>>();
					ConfigFile latestProjectConfigFile = await ConfigUtils.ReadProjectConfigFileAsync(perforceClient, projectInfo, localConfigFiles, logger, cancellationToken);

					// Get the local config file and stream filter
					ConfigFile workspaceProjectConfigFile = await WorkspaceUpdate.ReadProjectConfigFile(perforceClient, projectInfo, logger);
					IReadOnlyList<string>? workspaceProjectStreamFilter = await WorkspaceUpdate.ReadProjectStreamFilter(perforceClient, workspaceProjectConfigFile, cancellationToken);

					OpenProjectInfo workspaceSettings = new OpenProjectInfo(selectedProject, perforceSettings, projectInfo, userWorkspaceSettings, workspaceStateWrapper, latestProjectConfigFile, workspaceProjectConfigFile, workspaceProjectStreamFilter, localConfigFiles, generateP4Config);

					if (generateP4Config)
					{
						GenerateP4ConfigFile(perforceSettings, projectInfo, logger);
					}

					return workspaceSettings;
				}
				catch
				{
					workspaceStateWrapper?.Dispose();
					throw;
				}
			}
			finally
			{
				perforceClient?.Dispose();
			}
		}

		static async Task<List<IPerforceSettings>> FilterClients(List<ClientsRecord> clients, FileReference newSelectedFileName, IPerforceSettings defaultPerforceSettings, string? hostName, ILogger logger, CancellationToken cancellationToken)
		{
			List<IPerforceSettings> candidateClients = new List<IPerforceSettings>();
			foreach (ClientsRecord client in clients)
			{
				// Make sure the client is well formed
				if (!String.IsNullOrEmpty(client.Name) && (!String.IsNullOrEmpty(client.Host) || !String.IsNullOrEmpty(client.Owner)) && !String.IsNullOrEmpty(client.Root))
				{
					// Require either a username or host name match
					if ((String.IsNullOrEmpty(client.Host) || String.Equals(client.Host, hostName, StringComparison.OrdinalIgnoreCase)) && (String.IsNullOrEmpty(client.Owner) || String.Equals(client.Owner, defaultPerforceSettings.UserName, StringComparison.OrdinalIgnoreCase)))
					{
						if (!Utility.SafeIsFileUnderDirectory(newSelectedFileName.FullName, client.Root))
						{
							logger.LogInformation("Rejecting {ClientName} due to root mismatch ({RootPath})", client.Name, client.Root);
							continue;
						}

						PerforceSettings candidateSettings = new PerforceSettings(defaultPerforceSettings) { ClientName = client.Name };
						using IPerforceConnection candidateClient = await PerforceConnection.CreateAsync(candidateSettings, logger);

						List<PerforceResponse<WhereRecord>> whereRecords = await candidateClient.TryWhereAsync(newSelectedFileName.FullName, cancellationToken).Where(x => x.Failed || !x.Data.Unmap).ToListAsync(cancellationToken);
						if (!whereRecords.Succeeded() || whereRecords.Count != 1)
						{
							logger.LogInformation("Rejecting {ClientName} due to file not existing in workspace", client.Name);
							continue;
						}

						List<PerforceResponse<FStatRecord>> records = await candidateClient.TryFStatAsync(FStatOptions.None, newSelectedFileName.FullName, cancellationToken).ToListAsync(cancellationToken);
						if (!records.Succeeded())
						{
							logger.LogInformation("Rejecting {ClientName} due to {FileName} not in depot", client.Name, newSelectedFileName);
							continue;
						}

						records.RemoveAll(x => !x.Data.IsMapped);
						if (records.Count == 0)
						{
							logger.LogInformation("Rejecting {ClientName} due to {NumRecords} matching records", client.Name, records.Count);
							continue;
						}

						logger.LogInformation("Found valid client {ClientName}", client.Name);
						candidateClients.Add(candidateSettings);
					}
				}
			}
			return candidateClients;
		}

		static void GenerateP4ConfigFile(IPerforceSettings perforceSettings, ProjectInfo projectInfo, ILogger<OpenProjectInfo> logger)
		{
			string? p4ConfigName = PerforceEnvironment.Default.GetValue("P4CONFIG");

			// We only create a p4config if the user has opted into the system by setting P4CONFIG manually.
			if (String.IsNullOrEmpty(p4ConfigName))
			{
				logger.LogError("Could not generate a p4config file as the envvar P4CONFIG is not set!");
				return;
			}

			string configFilePath = Path.Combine(projectInfo.LocalRootPath.ToString(), p4ConfigName);

			// Do not update or replace the file if it already exists 
			if (File.Exists(configFilePath))
			{
				logger.LogDebug("P4CONFIG file '{Path}' already exists.", configFilePath);
				return;
			}

			Dictionary<string, string> settings = new Dictionary<string, string>();
			settings.Add("P4PORT", perforceSettings.ServerAndPort);
			settings.Add("P4USER", perforceSettings.UserName);
			settings.Add("P4CLIENT", perforceSettings.ClientName ?? "");

			List<string> Lines = new List<string>();
			foreach (KeyValuePair<string, string> setting in settings)
			{
				Lines.Add(String.Format("{0}={1}", setting.Key, setting.Value));

			}

			try
			{
				File.WriteAllLines(configFilePath, Lines);
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Failed to write {Path} with exception - {Msg}", p4ConfigName, ex.Message);
				return;
			}

			logger.LogInformation("Successfully wrote a P4CONFIG file to '{Path}'.", configFilePath);
		}
	}
}
