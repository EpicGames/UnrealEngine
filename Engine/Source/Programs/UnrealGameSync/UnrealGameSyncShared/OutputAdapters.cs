// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using Microsoft.Extensions.Logging;

#nullable enable

namespace UnrealGameSync
{
	class PrefixedTextWriter : ILogger
	{
		readonly string _prefix;
		readonly ILogger _inner;

		public PrefixedTextWriter(string inPrefix, ILogger inInner)
		{
			_prefix = inPrefix;
			_inner = inInner;
		}

		public IDisposable? BeginScope<TState>(TState state) where TState : notnull => _inner.BeginScope(state);

		public bool IsEnabled(LogLevel logLevel) => _inner.IsEnabled(logLevel);

		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			_inner.Log(logLevel, eventId, state, exception, (state, exception) => _prefix + formatter(state, exception));
		}
	}
}

