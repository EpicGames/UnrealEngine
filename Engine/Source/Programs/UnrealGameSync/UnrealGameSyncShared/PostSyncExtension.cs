// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	internal class PostSyncExtension : IDisposable
	{
		private readonly WorkspaceUpdateContext Context;

		private readonly IPerforceConnection Perforce;

		private readonly ProjectInfo Project;

		private readonly FileReference ConfigTempFile ;
		private readonly FileReference GlobalSettingsJsonFile;
		private readonly FileReference GlobalSettingsIniFile;
		private readonly FileReference WorkspaceSettingsFile;
		private readonly FileReference ProjectSettingsFile;

		private readonly List<FileReference> TempFiles = new List<FileReference>();

		public PostSyncExtension(ILogger logger, WorkspaceUpdateContext context, IPerforceConnection perforce, ProjectInfo project) 
		{
			Context = context;
			Perforce = perforce;
			Project = project;

			ConfigTempFile = new FileReference(Path.GetTempFileName());
			Context.ProjectConfigFile?.Save(ConfigTempFile);
			TempFiles.Add(ConfigTempFile);

			GlobalSettingsJsonFile = new FileReference(Path.GetTempFileName());
			GlobalSettingsIniFile = new FileReference(Path.GetTempFileName());

			// todo: check what this is doing!
			Context.Usersettings?.Save(logger, GlobalSettingsJsonFile, GlobalSettingsIniFile);
			TempFiles.Add(GlobalSettingsJsonFile);
			TempFiles.Add(GlobalSettingsIniFile);

			WorkspaceSettingsFile = new FileReference(Path.GetTempFileName());
			Context.UserWorkspaceSettings?.Save(logger, WorkspaceSettingsFile);
			TempFiles.Add(WorkspaceSettingsFile);

			ProjectSettingsFile = new FileReference(Path.GetTempFileName());
			Context.UserProjectSettings?.Save(logger, ProjectSettingsFile);
			TempFiles.Add(ProjectSettingsFile);
		}

		public async Task<Dictionary<string, string>> ExpandAsync(string step, Dictionary<string, string> postSyncVariables, CancellationToken cancellationToken)
		{
			Dictionary<string, string> output = new Dictionary<string, string>(postSyncVariables);
			
			if (step.Contains("$(StreamPrefix)", StringComparison.Ordinal))
			{
				if (!String.IsNullOrWhiteSpace(Project.StreamName))
				{
					string? streamPrefix = await ProjectInfo.TryGetStreamPrefixAsync(Perforce, Project.StreamName, cancellationToken);
					if (!String.IsNullOrWhiteSpace(streamPrefix))
					{
						output.TryAdd("StreamPrefix", streamPrefix);
					}
					else
					{
						output.TryAdd("StreamPrefix", String.Empty);
					}
				}
			}

			if (step.Contains("$(ConfigFile)", StringComparison.Ordinal))
			{
				output.TryAdd("ConfigFile", ConfigTempFile.FullName);
			}

			if (step.Contains("$(GlobalJsonFile)", StringComparison.Ordinal))
			{
				output.TryAdd("GlobalJsonFile", GlobalSettingsJsonFile.FullName);
			}

			if (step.Contains("$(GlobalIniFile)", StringComparison.Ordinal))
			{
				output.TryAdd("GlobalIniFile", GlobalSettingsIniFile.FullName);
			}

			if (step.Contains("$(WorkspaceSettings)", StringComparison.Ordinal))
			{
				output.TryAdd("WorkspaceSettings", WorkspaceSettingsFile.FullName);
			}

			if (step.Contains("$(ProjectSettings)", StringComparison.Ordinal))
			{
				output.TryAdd("ProjectSettings", ProjectSettingsFile.FullName);
			}

			return output;
		}

		public void Dispose()
		{
			foreach (FileReference fileReference in TempFiles)
			{
				try
				{
					File.Delete(fileReference.FullName);
				}
				catch
				{
					// ignored
				}
			}
		}
	}
}
