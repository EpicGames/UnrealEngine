// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace UnrealGameSync
{
	public class RoleCategory
	{
		public Guid Id { get; set; } = Guid.Empty;
		public bool Enabled { get; set; } = false;
	}
	
	public class Preset
	{
		public string Name { get; set; } = String.Empty;
		public IDictionary<Guid, RoleCategory> Categories { get; } = new Dictionary<Guid, RoleCategory>();
		public ISet<string> Views { get; } = new HashSet<string>();

		public void Import(Preset preset)
		{
			Name = preset.Name;
			foreach (KeyValuePair<Guid, RoleCategory> rhs in preset.Categories)
			{
				if (Categories.TryGetValue(rhs.Key, out RoleCategory? lhs))
				{
					lhs.Enabled = rhs.Value.Enabled;
				}
				else
				{
					Categories.Add(rhs.Key, rhs.Value);
				}
			}

			foreach (string rhs in preset.Views)
			{
				Views.Add(rhs);
			}
		}
	}
}

