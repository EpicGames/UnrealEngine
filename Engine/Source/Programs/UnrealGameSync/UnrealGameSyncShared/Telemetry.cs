// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Reflection;
using System.Text;
using System.Text.Json;
using System.Threading.Channels;
using System.Threading.Tasks;
using System.Web;
using Microsoft.Extensions.Logging;

namespace UnrealGameSync
{
	/// <summary>
	/// Interface for a telemetry sink
	/// </summary>
	public interface ITelemetrySink : IDisposable
	{
		/// <summary>
		/// Sends a telemetry event with the given information
		/// </summary>
		/// <param name="eventName">Name of the event</param>
		/// <param name="attributes">Arbitrary object to include in the payload</param>
		void SendEvent(string eventName, object attributes);
	}

	/// <summary>
	/// Telemetry sink that discards all events
	/// </summary>
	public sealed class NullTelemetrySink : ITelemetrySink
	{
		/// <inheritdoc/>
		public void Dispose()
		{
		}

		/// <inheritdoc/>
		public void SendEvent(string eventName, object attributes)
		{
		}
	}

	/// <summary>
	/// Epic internal telemetry sink using the data router
	/// </summary>
	public sealed class EpicTelemetrySink : ITelemetrySink
	{
		readonly string _url;
		readonly ILogger _logger;
		readonly HttpClient _httpClient = new HttpClient();
		readonly Channel<string> _channel = Channel.CreateUnbounded<string>(new UnboundedChannelOptions());
		readonly Task _backgroundTask;

		/// <summary>
		/// Constructor
		/// </summary>
		public EpicTelemetrySink(string url, ILogger logger)
		{
			_url = url;
			_logger = logger;

			logger.LogInformation("Posting to URL: {Url}", url);

			_backgroundTask = Task.Run(WriteEventsAsync);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_channel.Writer.TryComplete();
			try
			{
				_backgroundTask.Wait();
			}
			catch (OperationCanceledException)
			{
			}
			_httpClient.Dispose();
		}

		/// <inheritdoc/>
		public void SendEvent(string eventName, object attributes)
		{
			string attributesText = JsonSerializer.Serialize(attributes);
			if (attributesText[0] != '{')
			{
				throw new Exception("Expected event data with named properties");
			}

			string eventText = attributesText.Insert(1, String.Format("\"EventName\":\"{0}\",", HttpUtility.JavaScriptStringEncode(eventName)));
			_channel.Writer.TryWrite(eventText);
		}

		/// <summary>
		/// Synchronously sends a telemetry event
		/// </summary>
		async Task WriteEventsAsync()
		{
			string version = Assembly.GetExecutingAssembly().GetName().Version?.ToString() ?? "0.0";
			while (await _channel.Reader.WaitToReadAsync())
			{
				try
				{
					// Generate the content for this event
					List<string> events = await _channel.Reader.ReadAllAsync().ToListAsync();
					if (events.Count == 0)
					{
						continue;
					}

					// Print all the events we're sending
					foreach (string evt in events)
					{
						_logger.LogInformation("Sending Event: {Event}", evt);
					}

					// Convert the content to UTF8
					string contentText = String.Format("{{\"Events\":[{0}]}}", String.Join(",", events));
					byte[] content = Encoding.UTF8.GetBytes(contentText);

					// Post the event data
					using (HttpRequestMessage request = new HttpRequestMessage(HttpMethod.Post, _url))
					{
						request.Headers.UserAgent.Add(new ProductInfoHeaderValue("UnrealGameSync", version));
						request.Content = new ByteArrayContent(content);
						request.Content.Headers.ContentType = new MediaTypeHeaderValue("application/json");

						using (HttpResponseMessage response = await _httpClient.SendAsync(request))
						{
							if (response.IsSuccessStatusCode)
							{
								_logger.LogInformation("Response: {StatusCode}", (int)response.StatusCode);
							}
							else
							{
								string responseContent = await response.Content.ReadAsStringAsync();
								_logger.LogError("Unable to send telemetry data to server ({Code}): {Message}", response.StatusCode, responseContent);
							}
						}
					}
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception while attempting to send event");
				}
			}
		}
	}

	/// <summary>
	/// Global telemetry static class
	/// </summary>
	public static class UgsTelemetry
	{
		/// <summary>
		/// The current telemetry provider
		/// </summary>
		public static ITelemetrySink? ActiveSink
		{
			get; set;
		}

		/// <summary>
		/// Sends a telemetry event with the given information
		/// </summary>
		/// <param name="eventName">Name of the event</param>
		/// <param name="attributes">Arbitrary object to include in the payload</param>
		public static void SendEvent(string eventName, object attributes)
		{
			ActiveSink?.SendEvent(eventName, attributes);
		}
	}
}
