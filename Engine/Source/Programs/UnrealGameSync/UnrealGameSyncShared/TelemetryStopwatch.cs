// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Reflection;

namespace UnrealGameSync
{
	class TelemetryStopwatch : IDisposable
	{
		readonly string _eventName;
		readonly Dictionary<string, object?> _eventData;
		readonly Stopwatch _timer;

		public TelemetryStopwatch(string eventName, string project)
		{
			_eventName = eventName;

			_eventData = new Dictionary<string, object?>();
			_eventData["Project"] = project;

			_timer = Stopwatch.StartNew();
		}

		public void AddData(object data)
		{
			foreach (PropertyInfo property in data.GetType().GetProperties())
			{
				_eventData[property.Name] = property.GetValue(data);
			}
		}

		public TimeSpan Stop(string inResult)
		{
			if (_timer.IsRunning)
			{
				_timer.Stop();

				_eventData["Result"] = inResult;
				_eventData["TimeSeconds"] = _timer.Elapsed.TotalSeconds;
				_eventData["TimeMinutes"] = _timer.Elapsed.TotalSeconds / 60.0f;
			}
			return Elapsed;
		}

		public void Dispose()
		{
			if (_timer.IsRunning)
			{
				Stop("Aborted");
			}
			UgsTelemetry.SendEvent(_eventName, _eventData);
		}

		public TimeSpan Elapsed => _timer.Elapsed;
	}
}
