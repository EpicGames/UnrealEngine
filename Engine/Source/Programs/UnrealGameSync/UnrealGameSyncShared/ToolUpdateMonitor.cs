// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Tools;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace UnrealGameSync
{
	[DebuggerDisplay("{Label}")]
	public class ToolLink
	{
		public string Label { get; set; }
		public string FileName { get; set; }
		public string? Arguments { get; set; }
		public string? WorkingDir { get; set; }

		public ToolLink(string label, string fileName)
		{
			Label = label;
			FileName = fileName;
		}
	}

	// Settings in the UgsTool.json file in the root of a Horde tool
	public class ToolSettings
	{
		public string? InstallCommand { get; set; }
		public string? UninstallCommand { get; set; }

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", 
			Justification = "Set property accessed through reflection")]
		public List<ToolLink> StatusPanelLinks { get; set; } = new List<ToolLink>();

		public bool SafeWhenBusy { get; set; }
	}

	[DebuggerDisplay("{Name}")]
	public class ToolInfo
	{
		public Guid Id { get; set; }
		public string Name { get; set; }
		public string Description { get; set; }

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", 
			Justification = "Set property accessed through reflection")]
		public HashSet<Guid> DependsOnToolIds { get; set; } = new HashSet<Guid>();

		public ToolSettings Settings { get; set; } = new ToolSettings();
		public string Revision { get; set; }

		public ToolInfo(Guid id, string name, string description, string revision)
		{
			Id = id;
			Name = name;
			Description = description;
			Revision = revision;
		}

		public ToolInfo Clone()
		{
			return (ToolInfo)MemberwiseClone();
		}
	}

	public sealed class ToolUpdateMonitor : IDisposable
	{
#pragma warning disable CA2213 // warning CA2213: 'ToolUpdateMonitor' contains field '_cancellationSource' that is of IDisposable type 'CancellationTokenSource', but it is never disposed. Change the Dispose method on 'ToolUpdateMonitor' to call Close or Dispose on this field.
		readonly CancellationTokenSource _cancellationSource;
#pragma warning restore CA2213
		readonly SynchronizationContext _synchronizationContext;
		Task? _workerTask;
		readonly AsyncEvent _wakeEvent;
		readonly ILogger _logger;
		readonly IAsyncDisposer _asyncDisposer;
		readonly FileReference _enabledToolsFile;
		readonly IServiceProvider _serviceProvider;

		bool _readLegacyConfig;

		IReadOnlyDictionary<string, ToolInfo> _perforceTools = new Dictionary<string, ToolInfo>(StringComparer.Ordinal);
		int _perforceToolsChange = -1;

		IReadOnlyList<ToolInfo> _tools = Array.Empty<ToolInfo>();
		IReadOnlyList<ToolInfo> _enabledTools = Array.Empty<ToolInfo>();

		IPerforceSettings PerforceSettings { get; }
		DirectoryReference ToolsDir { get; }
		UserSettings Settings { get; }

		public Action? OnChange { get; set; }

		public Tuple<bool, string> LastStatus { get; private set; } = Tuple.Create(false, "Starting...");

		public ToolUpdateMonitor(IPerforceSettings perforceSettings, DirectoryReference dataDir, UserSettings settings, ILogger logger, IServiceProvider serviceProvider)
		{
			_cancellationSource = new CancellationTokenSource();
			_synchronizationContext = SynchronizationContext.Current!;
			ToolsDir = DirectoryReference.Combine(dataDir, "Tools");
			PerforceSettings = perforceSettings;
			Settings = settings;
			_logger = logger;
			_asyncDisposer = serviceProvider.GetRequiredService<IAsyncDisposer>();
			_serviceProvider = serviceProvider;

			DirectoryReference.CreateDirectory(ToolsDir);
			_enabledToolsFile = FileReference.Combine(ToolsDir, "tools.json");

			if (FileReference.Exists(_enabledToolsFile))
			{
				try
				{
					Load();
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Unable to read previous tools list: {Message}", ex.Message);
				}
			}
			else
			{
				_readLegacyConfig = true;
			}

			_wakeEvent = new AsyncEvent();
		}

		public void Start()
		{
			_workerTask = Task.Run(() => PollForUpdatesAsync(_cancellationSource.Token));
		}

		public Task GetDataFromBackendAsync()
		{
			return Task.Run(() => PollForUpdatesOnce(_cancellationSource.Token, false));
		}

		public Task UpdateToolsAsync()
		{
			return Task.Run(() => PollForUpdatesOnce(_cancellationSource.Token, true));
		}

		public void Dispose()
		{
			LastStatus = Tuple.Create(false, "Stopped");
			OnChange = null;

			if (_workerTask != null)
			{
				_cancellationSource.Cancel();
				_asyncDisposer.Add(_workerTask.ContinueWith(_ => _cancellationSource.Dispose(), TaskScheduler.Default));
				_workerTask = null;
			}
		}

		public IReadOnlyList<ToolInfo> GetTools()
			=> _tools;

		public IReadOnlyList<ToolInfo> GetEnabledTools()
			=> _enabledTools;

		DirectoryReference GetToolPathInternal(string toolName)
		{
			return DirectoryReference.Combine(ToolsDir, toolName, "Current");
		}

		public string? GetToolName(Guid toolId)
		{
			foreach (ToolInfo tool in _tools)
			{
				if (tool.Id == toolId)
				{
					return tool.Name;
				}
			}
			return null;
		}

		public DirectoryReference? GetToolPath(string toolName)
		{
			IReadOnlyList<ToolInfo> enabledTools = _enabledTools;
			if (enabledTools.Any(x => String.Equals(x.Name, toolName, StringComparison.OrdinalIgnoreCase)))
			{
				return GetToolPathInternal(toolName);
			}
			else
			{
				return null;
			}
		}

		public void UpdateNow()
		{
			_wakeEvent.Set();
		}

		async Task PollForUpdatesAsync(CancellationToken cancellationToken)
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				Task wakeTask = _wakeEvent.Task;

				try
				{
					await PollForUpdatesOnce(cancellationToken);
				}
				catch (Exception ex)
				{
					LastStatus = Tuple.Create(false, $"Exception while checking for tool updates: {ex.Message}");
					_logger.LogError(ex, "Exception while checking for tool updates: {Message}", ex.Message);
				}

				Task delayTask = Task.Delay(TimeSpan.FromMinutes(60.0), cancellationToken);
				await Task.WhenAny(delayTask, wakeTask);
			}
		}

		async Task PollForUpdatesOnce(CancellationToken cancellationToken, bool update = true)
		{
			IPerforceConnection? perforce = null;
			try
			{
				Stopwatch timer = Stopwatch.StartNew();
				// Update all the available tools
				List<ToolInfo> tools = new List<ToolInfo>();
				if (!String.IsNullOrEmpty(DeploymentSettings.Instance.ToolsDepotPath))
				{
					try
					{
						perforce = await PerforceConnection.CreateAsync(PerforceSettings, _logger);
						await ReadPerforceToolsAsync(perforce, tools, cancellationToken);
					}
					catch (Exception ex) when (ex is not OperationCanceledException)
					{
						LastStatus = Tuple.Create(false, $"Error while polling Perforce for available tools: {ex.Message}");
						_logger.LogWarning(ex, "Error while polling Perforce for available tools: {Message}", ex.Message);
						return;
					}
				}
				IHordeClient? hordeClient = _serviceProvider.GetService<IHordeClient>();
				if (hordeClient != null)
				{
					using HordeHttpClient hordeHttpClient = hordeClient.CreateHttpClient();
					try
					{
						await ReadHordeToolsAsync(hordeHttpClient, tools, cancellationToken);
					}
					catch (Exception ex) when (ex is not OperationCanceledException)
					{
						LastStatus = Tuple.Create(false, $"Error while polling Horde for available tools: {ex.Message}");
						_logger.LogWarning(ex, "Error while polling Horde for available tools: {Message}", ex.Message);
						return;
					}
				}

				bool hasChanged = false;
				int previousToolsCount = _tools.Count;

				_tools = tools;

				if (previousToolsCount != _tools.Count)
				{
					hasChanged = true;
				}

				// When upgrading from older UGS versions, read the legacy sync CL from plain-text config files
				if (_readLegacyConfig)
				{
					await ReadLegacyConfigAsync(tools, cancellationToken);
					_readLegacyConfig = false;
				}

				// Find all the tools which are enabled, including those enabled due to dependencies from other tools
				HashSet<Guid> enabledToolIds = new HashSet<Guid>();
				FindEnabledTools(Settings.EnabledTools, tools, enabledToolIds);

				// Install or update any new tools
				if (update)
				{
					foreach (ToolInfo toolInfo in _tools)
					{
						if (enabledToolIds.Contains(toolInfo.Id))
						{
							ToolInfo? existingTool = _enabledTools.FirstOrDefault(x => x.Id == toolInfo.Id);
							if (existingTool == null || !String.Equals(existingTool.Revision, toolInfo.Revision, StringComparison.OrdinalIgnoreCase))
							{
								await UpdateToolAsync(perforce, toolInfo, cancellationToken);
								hasChanged = true;
							}
						}
					}

					// Remove any tools which we no longer need
					for (int idx = _enabledTools.Count - 1; idx >= 0; idx--)
					{
						ToolInfo tool = _enabledTools[idx];
						if (!enabledToolIds.Contains(tool.Id))
						{
							await RemoveToolAsync(tool, cancellationToken);
							hasChanged = true;
						}
					}

					// Notify the main window if anything changed
					if (hasChanged)
					{
						_synchronizationContext?.Post(_ => OnChange?.Invoke(), null);
					}
				}

				LastStatus = Tuple.Create(true, $"Last update took {timer.ElapsedMilliseconds}ms (completed at {DateTime.Now.ToShortTimeString()})");
			}
			finally
			{
				perforce?.Dispose();
			}
		}

		static void FindEnabledTools(HashSet<Guid> inputToolIds, List<ToolInfo> tools, HashSet<Guid> enabledToolIds)
		{
			if (inputToolIds.Count > 0)
			{
				foreach (ToolInfo tool in tools)
				{
					if (inputToolIds.Contains(tool.Id) && enabledToolIds.Add(tool.Id))
					{
						FindEnabledTools(tool.DependsOnToolIds, tools, enabledToolIds);
					}
				}
			}
		}

		async Task ReadPerforceToolsAsync(IPerforceConnection perforce, List<ToolInfo> tools, CancellationToken cancellationToken)
		{
			List<ChangesRecord> changes = await perforce.GetChangesAsync(ChangesOptions.None, 1, ChangeStatus.Submitted, $"{DeploymentSettings.Instance.ToolsDepotPath}/...", cancellationToken);
			if (changes.Count > 0 && changes[0].Number != _perforceToolsChange)
			{
				ConcurrentDictionary<string, ToolInfo> newPerforceTools = new ConcurrentDictionary<string, ToolInfo>(StringComparer.Ordinal);

				List<FilesRecord> fileRecords = await perforce.FilesAsync(FilesOptions.ExcludeDeleted, $"{DeploymentSettings.Instance.ToolsDepotPath}/...", cancellationToken);

				await Parallel.ForEachAsync(fileRecords, cancellationToken, async (fileRecord, ct) =>
				{
					if (fileRecord.DepotFile != null && fileRecord.DepotFile.EndsWith(".ini", StringComparison.OrdinalIgnoreCase))
					{
						string zipFile = fileRecord.DepotFile.Substring(0, fileRecord.DepotFile.Length - 4) + ".zip";
						FilesRecord? zipRecord = fileRecords.FirstOrDefault(x => String.Equals(x.DepotFile, zipFile, StringComparison.OrdinalIgnoreCase));
						if (zipRecord != null)
						{
							string toolRevision = $"{zipFile}@{Math.Max(fileRecord.Change, zipRecord.Change)}";
							ToolInfo? toolInfo;
							if (!_perforceTools.TryGetValue(toolRevision, out toolInfo))
							{
								toolInfo = await ReadToolDefinitionAsync(perforce, $"{fileRecord.DepotFile}@{fileRecord.Change}", toolRevision, ct);
							}
							if (toolInfo != null)
							{
								newPerforceTools.TryAdd(toolRevision, toolInfo);
							}
						}
					}
				});

				_perforceTools = newPerforceTools;
				_perforceToolsChange = changes[0].Number;
			}

			tools.AddRange(_perforceTools.Values);
		}

		static async Task ReadHordeToolsAsync(HordeHttpClient hordeHttpClient, List<ToolInfo> tools, CancellationToken cancellationToken)
		{
			GetToolsSummaryResponse toolsResponse = await hordeHttpClient.GetToolsAsync(cancellationToken);
			foreach (GetToolSummaryResponse toolResponse in toolsResponse.Tools)
			{
				if (toolResponse.ShowInUgs)
				{
					IoHash hash = IoHash.Compute(Encoding.UTF8.GetBytes($"horde:{toolResponse.Id}"));
					Guid guid = new Guid(hash.ToByteArray().AsSpan(0, 16));
					ToolInfo toolInfo = new ToolInfo(guid, toolResponse.Id.ToString(), toolResponse.Name, $"{hordeHttpClient},{toolResponse.Id},{toolResponse.DeploymentId}");
					tools.Add(toolInfo);
				}
			}
		}

		static async Task<ToolInfo?> ReadToolDefinitionAsync(IPerforceConnection perforce, string iniRevision, string toolRevision, CancellationToken cancellationToken)
		{
			PerforceResponse<PrintRecord<string[]>> response = await perforce.TryPrintLinesAsync(iniRevision, cancellationToken);
			if (!response.Succeeded || response.Data.Contents == null)
			{
				return null;
			}

			int nameIdx = iniRevision.LastIndexOf('/') + 1;
			int extensionIdx = iniRevision.LastIndexOf('.');
			string defaultToolName = iniRevision.Substring(nameIdx, extensionIdx - nameIdx);

			return ParseToolInfo(response.Data.Contents, defaultToolName, toolRevision);
		}

		static ToolInfo? ParseToolInfo(string[] contents, string defaultToolName, string revision)
		{
			ConfigFile configFile = new ConfigFile();
			configFile.Parse(contents);

			string? id = configFile.GetValue("Settings.Id", null);
			if (id == null || !Guid.TryParse(id, out Guid toolId))
			{
				return null;
			}

			string toolName = configFile.GetValue("Settings.Name", defaultToolName);
			string toolDescription = configFile.GetValue("Settings.Description", toolName);

			ToolInfo tool = new ToolInfo(toolId, toolName, toolDescription, revision);
			tool.Settings = new ToolSettings();
			tool.Settings.InstallCommand = configFile.GetValue("Settings.InstallCommand", null);
			tool.Settings.UninstallCommand = configFile.GetValue("Settings.UninstallCommand", null);
			tool.Settings.SafeWhenBusy = configFile.GetValue("Settings.SafeWhenBusy", false);

			foreach (string line in configFile.GetValues("Settings.DependsOnTool", Array.Empty<string>()))
			{
				if (Guid.TryParse(line.Trim(), out Guid requiredToolId))
				{
					tool.DependsOnToolIds.Add(requiredToolId);
				}
			}

			string[] statusPanelLinks = configFile.GetValues("Settings.StatusPanelLinks", Array.Empty<string>());
			foreach (string statusPanelLink in statusPanelLinks)
			{
				ConfigObject obj = new ConfigObject(statusPanelLink);

				string? label = obj.GetValue("Label", null);
				string? fileName = obj.GetValue("FileName", null);

				if (label != null && fileName != null)
				{
					ToolLink link = new ToolLink(label, fileName);
					link.Arguments = obj.GetValue("Arguments", null);
					link.WorkingDir = obj.GetValue("WorkingDir", null);
					tool.Settings.StatusPanelLinks.Add(link);
				}
			}

			return tool;
		}

		public async Task RunCommandAsync(string toolName, string command, CancellationToken cancellationToken)
		{
			DirectoryReference toolPath = GetToolPathInternal(toolName);

			string commandExe = command;
			string commandArgs = String.Empty;

			int spaceIdx = command.IndexOf(' ', StringComparison.Ordinal);
			if (spaceIdx != -1)
			{
				commandExe = command.Substring(0, spaceIdx);
				commandArgs = command.Substring(spaceIdx + 1);
			}

			int exitCode = await Utility.ExecuteProcessAsync(FileReference.Combine(toolPath, commandExe).FullName, toolPath.FullName, commandArgs, line => _logger.LogInformation("{ToolName}> {Line}", toolName, line), cancellationToken);
			_logger.LogInformation("{ToolName}> Exit code {ExitCode})", toolName, exitCode);
		}

		async Task RemoveToolAsync(ToolInfo tool, CancellationToken cancellationToken)
		{
			try
			{
				_logger.LogInformation("Removing {ToolName}", tool.Name);
				DirectoryReference? toolPath = GetToolPath(tool.Name);

				if (!String.IsNullOrEmpty(tool.Settings?.UninstallCommand))
				{
					_logger.LogInformation("Running unininstall action: {Command}", tool.Settings.UninstallCommand);
					await RunCommandAsync(tool.Name, tool.Settings.UninstallCommand, cancellationToken);
				}

				await SetToolRevisionAsync(tool.Name, null, cancellationToken);

				if (toolPath != null)
				{
					_logger.LogInformation("Removing {ToolPath}", toolPath);
					TryDeleteDirectory(toolPath);
				}

				_logger.LogInformation("{ToolName} has been removed successfully", tool.Name);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception while removing tool {ToolId}: {Message}", tool.Id, ex.Message);
			}
		}

		static void ForceDeleteDirectory(DirectoryReference directoryName)
		{
			DirectoryInfo baseDir = directoryName.ToDirectoryInfo();
			if (baseDir.Exists)
			{
				foreach (FileInfo file in baseDir.EnumerateFiles("*", SearchOption.AllDirectories))
				{
					file.Attributes = FileAttributes.Normal;
				}
				baseDir.Delete(true);
			}
		}

		bool TryDeleteDirectory(DirectoryReference directoryName)
		{
			try
			{
				ForceDeleteDirectory(directoryName);
				return true;
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Unable to delete directory {DirectoryName}", directoryName);
				return false;
			}
		}

		async Task<bool> UpdateToolAsync(IPerforceConnection? perforce, ToolInfo tool, CancellationToken cancellationToken)
		{
			try
			{
				DirectoryReference toolDir = DirectoryReference.Combine(ToolsDir, tool.Name);
				DirectoryReference.CreateDirectory(toolDir);

				foreach (DirectoryReference existingDir in DirectoryReference.EnumerateDirectories(toolDir, "Prev-*"))
				{
					TryDeleteDirectory(existingDir);
				}

				DirectoryReference nextToolDir = DirectoryReference.Combine(toolDir, "Next");
				ForceDeleteDirectory(nextToolDir);
				DirectoryReference.CreateDirectory(nextToolDir);

				DirectoryReference nextToolZipsDir = DirectoryReference.Combine(nextToolDir, ".zips");
				DirectoryReference.CreateDirectory(nextToolZipsDir);

				FileReference zipFile = FileReference.Combine(nextToolZipsDir, $"{tool.Name}.zip");
				if (perforce != null && tool.Revision.StartsWith("//", StringComparison.Ordinal))
				{
					// Read it from Perforce
					PerforceResponseList<PrintRecord> response = await perforce.TryPrintAsync(zipFile.FullName, tool.Revision, cancellationToken);
					if (!response.Succeeded || !FileReference.Exists(zipFile))
					{
						_logger.LogError("Unable to print {DepotFile}", tool.Revision);
						return false;
					}
					ArchiveUtils.ExtractFiles(zipFile, nextToolDir, null, new ProgressValue(), _logger);
				}
				else
				{
					IHordeClient? hordeClient = _serviceProvider.GetService<IHordeClient>();
					if (hordeClient != null)
					{
						using HordeHttpClient hordeHttpClient = hordeClient.CreateHttpClient();

						string[] fields = tool.Revision.Split(',');
						if (fields.Length != 3)
						{
							_logger.LogError("Unexpected format for Horde revision ('{Revision}')", tool.Revision);
							return false;
						}

						using (FileStream stream = FileReference.Open(zipFile, FileMode.Create, FileAccess.Write, FileShare.None))
						{
							await using Stream sourceStream = await hordeHttpClient.GetToolDeploymentZipAsync(new ToolId(fields[1]), new ToolDeploymentId(BinaryId.Parse(fields[2])), cancellationToken);
							await sourceStream.CopyToAsync(stream, cancellationToken);
						}

						ArchiveUtils.ExtractFiles(zipFile, nextToolDir, null, new ProgressValue(), _logger);

						FileReference settingsFile = FileReference.Combine(nextToolDir, "UgsTool.json");
						if (FileReference.Exists(settingsFile))
						{
							byte[] data = await FileReference.ReadAllBytesAsync(settingsFile, cancellationToken);
							tool.Settings = JsonSerializer.Deserialize<ToolSettings>(data, new JsonSerializerOptions { PropertyNameCaseInsensitive = true }) ?? tool.Settings;
						}
					}
					else
					{
						_logger.LogError("Unknown source for {Revision}", tool.Revision);
						return false;
					}
				}

				DirectoryReference currentToolDir = DirectoryReference.Combine(toolDir, "Current");
				if (DirectoryReference.Exists(currentToolDir))
				{
					DirectoryReference prevDirectoryName = DirectoryReference.Combine(toolDir, String.Format("Prev-{0:X16}", Stopwatch.GetTimestamp()));
					Directory.Move(currentToolDir.FullName, prevDirectoryName.FullName);
					await SetToolRevisionAsync(tool.Name, null, cancellationToken);
					TryDeleteDirectory(prevDirectoryName);
				}

				Directory.Move(nextToolDir.FullName, currentToolDir.FullName);

				if (!String.IsNullOrEmpty(tool.Settings?.InstallCommand))
				{
					_logger.LogInformation("Running install action: {Command}", tool.Settings.InstallCommand);
					await RunCommandAsync(tool.Name, tool.Settings.InstallCommand, cancellationToken);
				}

				await SetToolRevisionAsync(tool.Name, tool, cancellationToken);
				_logger.LogInformation("Updated {ToolName} to {Revision}", tool.Name, tool.Revision);
				return true;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception while updating tool: {Message}", ex.Message);
				return false;
			}
		}

		FileReference GetConfigFilePath(string toolName)
		{
			return FileReference.Combine(ToolsDir, toolName, toolName + ".ini");
		}

		async Task ReadLegacyConfigAsync(IReadOnlyList<ToolInfo> tools, CancellationToken cancellationToken)
		{
			try
			{
				List<FileReference> deleteFiles = new List<FileReference>();

				List<ToolInfo> enabledTools = new List<ToolInfo>();
				foreach (ToolInfo tool in tools)
				{
					int change = GetToolChange(tool.Name);
					if (change != 0)
					{
						int atIdx = tool.Revision.LastIndexOf('@');
						if (atIdx != -1)
						{
							ToolInfo clone = tool.Clone();
							clone.Revision = tool.Revision.Substring(0, atIdx + 1) + change.ToString();
							enabledTools.Add(clone);

							deleteFiles.Add(GetConfigFilePath(tool.Name));
						}
					}
				}

				await SaveAsync(enabledTools, cancellationToken);

				foreach (FileReference deleteFile in deleteFiles)
				{
					FileReference.Delete(deleteFile);
				}
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Unable to upgrade tools from legacy config format: {Message}", ex.Message);
			}
		}

		int GetToolChange(string toolName)
		{
			try
			{
				FileReference configFilePath = GetConfigFilePath(toolName);
				if (FileReference.Exists(configFilePath))
				{
					ConfigFile configFile = new ConfigFile();
					configFile.Load(configFilePath);
					return configFile.GetValue("Settings.Change", 0);
				}
			}
			catch
			{
			}
			return 0;
		}

		async Task SetToolRevisionAsync(string toolName, ToolInfo? toolInfo, CancellationToken cancellationToken)
		{
			List<ToolInfo> enabledTools = new List<ToolInfo>(_enabledTools);

			int toolIndex = enabledTools.FindIndex(x => String.Equals(x.Name, toolName, StringComparison.OrdinalIgnoreCase));
			if (toolIndex != -1)
			{
				if (toolInfo == null)
				{
					enabledTools.RemoveAt(toolIndex);
				}
			}
			else
			{
				if (toolInfo != null)
				{
					enabledTools.Add(toolInfo);
				}
			}

			await SaveAsync(enabledTools, cancellationToken);
			_enabledTools = enabledTools;
		}

		class State
		{
			public List<ToolInfo> Tools { get; set; } = new List<ToolInfo>();
		}

		void Load()
		{
			if (FileReference.Exists(_enabledToolsFile))
			{
				byte[]? data = FileTransaction.ReadAllBytes(_enabledToolsFile);
				if (data != null)
				{
					State? state = JsonSerializer.Deserialize<State>(data, new JsonSerializerOptions { PropertyNameCaseInsensitive = true });
					if (state != null)
					{
						_enabledTools = state.Tools;
					}
				}
			}
		}

		async Task SaveAsync(List<ToolInfo> enabledTools, CancellationToken cancellationToken)
		{
			JsonSerializerOptions options = new JsonSerializerOptions();
			options.DefaultIgnoreCondition = System.Text.Json.Serialization.JsonIgnoreCondition.WhenWritingNull;
			options.PropertyNamingPolicy = JsonNamingPolicy.CamelCase;
			options.WriteIndented = true;

			State state = new State { Tools = enabledTools };
			byte[] data = JsonSerializer.SerializeToUtf8Bytes(state, options);
			await FileTransaction.WriteAllBytesAsync(_enabledToolsFile, data, cancellationToken);
		}
	}
}
