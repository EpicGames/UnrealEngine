<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <RootNamespace>UnrealGameSync</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <None Remove="Deployment.json" />
  </ItemGroup>
  
  <Choose>
    <When Condition="Exists('..\..\..\..\Restricted\NotForLicensees\Source\Programs\UnrealGameSync\UnrealGameSyncShared\Deployment.json')">
      <ItemGroup>
        <Content Include="..\..\..\..\Restricted\NotForLicensees\Source\Programs\UnrealGameSync\UnrealGameSyncShared\Deployment.json">
          <CopyToOutputDirectory>Always</CopyToOutputDirectory>
        </Content>
      </ItemGroup>
    </When>
    <Otherwise>
      <ItemGroup>
        <Content Include="Deployment.json">
          <CopyToOutputDirectory>Always</CopyToOutputDirectory>
        </Content>
      </ItemGroup>
    </Otherwise>
  </Choose>
  
  <ItemGroup>
    <PackageReference Include="Microsoft.Extensions.Caching.Memory" Version="10.0.6" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.OIDC\EpicGames.OIDC.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Perforce\EpicGames.Perforce.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.ProjectStore\EpicGames.ProjectStore.csproj" />
  </ItemGroup>

</Project>
