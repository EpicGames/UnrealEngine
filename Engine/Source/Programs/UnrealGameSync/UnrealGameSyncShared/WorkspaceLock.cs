// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Security.Cryptography;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace UnrealGameSync
{
	/// <summary>
	/// Encapsulates the state of cross-process workspace lock
	/// </summary>
	public sealed class WorkspaceLock : IDisposable
	{
		const string Prefix = @"Global\ugs-workspace";

		readonly Mutex _mutex;

		readonly object _lockObject = new object();
		readonly string _objectName;
		int _acquireCount;

		bool _locked;
		EventWaitHandle? _lockedEvent;

		Thread? _acquireThread;
		readonly BlockingCollection<Action> _acquireActions = new BlockingCollection<Action>();
		readonly CancellationTokenSource _acquireCancellationSource = new CancellationTokenSource();

		Thread? _monitorThread;
		readonly ManualResetEvent _cancelMonitorEvent = new ManualResetEvent(false);

		/// <summary>
		/// Callback for the lock state changing
		/// </summary>
		public event Action<bool>? OnChange;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="rootDir">Root directory for the workspace</param>
		public WorkspaceLock(DirectoryReference rootDir)
		{
#pragma warning disable CA5351 // Do Not Use Broken Cryptographic Algorithms
			using (MD5 md5 = MD5.Create())
			{
				byte[] idBytes = Encoding.UTF8.GetBytes(rootDir.FullName.ToUpperInvariant());
				_objectName = StringUtils.FormatHexString(md5.ComputeHash(idBytes));
			}
#pragma warning restore CA5351 // Do Not Use Broken Cryptographic Algorithms

			_mutex = new Mutex(false, $"{Prefix}.{_objectName}.mutex");

			_acquireThread = new Thread(AcquireThread);
			_acquireThread.Name = nameof(AcquireThread);
			_acquireThread.IsBackground = true;
			_acquireThread.Start();

			_monitorThread = new Thread(MonitorThread);
			_monitorThread.Name = nameof(MonitorThread);
			_monitorThread.IsBackground = true;
			_monitorThread.Start();
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_acquireThread != null)
			{
				_acquireCancellationSource.Cancel();

				_acquireThread.Join();
				_acquireThread = null;
			}

			if (_monitorThread != null)
			{
				_cancelMonitorEvent.Set();

				_monitorThread.Join();
				_monitorThread = null;
			}

			_lockedEvent?.Dispose();

			_acquireCancellationSource.Dispose();
			_acquireActions.Dispose();
			_cancelMonitorEvent.Dispose();
			_mutex.Dispose();
		}

		/// <summary>
		/// Determines if the lock is held by any
		/// </summary>
		/// <returns>True if the lock is held by any process</returns>
		public bool IsLocked() => _locked;

		/// <summary>
		/// Determines if the lock is held by another process
		/// </summary>
		/// <returns>True if the lock is held by another process</returns>
		public bool IsLockedByOtherProcess() => _acquireCount == 0 && IsLocked();

		/// <summary>
		/// Attempt to acquire the mutext
		/// </summary>
		/// <returns></returns>
		public async Task<bool> TryAcquireAsync()
		{
			TaskCompletionSource<bool> result = new TaskCompletionSource<bool>();
			_acquireActions.Add(() => TryAcquireInternal(result));
			return await result.Task;
		}

		void TryAcquireInternal(TaskCompletionSource<bool> resultTcs)
		{
			bool result;
			lock (_lockObject)
			{
				try
				{
					result = _mutex.WaitOne(0);
				}
				catch (AbandonedMutexException)
				{
					result = true;
				}

				if (result && ++_acquireCount == 1)
				{
					_lockedEvent = CreateLockedEvent();
					_lockedEvent.Set();
				}
			}
			Task.Run(() => resultTcs.TrySetResult(result));
		}

		/// <summary>
		/// Release the current mutext
		/// </summary>
		public async Task ReleaseAsync()
		{
			TaskCompletionSource<bool> resultTcs = new TaskCompletionSource<bool>();
			_acquireActions.Add(() => ReleaseInternal(resultTcs));
			await resultTcs.Task;
		}

		private void ReleaseInternal(TaskCompletionSource<bool> resultTcs)
		{
			lock (_lockObject)
			{
				if (_acquireCount > 0)
				{
					_mutex.ReleaseMutex();
					if (--_acquireCount == 0)
					{
						ReleaseLockedEvent();
					}
				}
			}
			Task.Run(() => resultTcs.TrySetResult(true));
		}

		private void ReleaseLockedEvent()
		{
			if (_lockedEvent != null)
			{
				_lockedEvent.Reset();
				_lockedEvent.Dispose();
				_lockedEvent = null;
			}
		}

		void AcquireThread()
		{
			for (; ; )
			{
				try
				{
					_acquireActions.Take(_acquireCancellationSource.Token)();
				}
				catch (OperationCanceledException)
				{
					break;
				}
			}

			for (; _acquireCount > 0; _acquireCount--)
			{
				_mutex.ReleaseMutex();
			}

			ReleaseLockedEvent();
		}

		void MonitorThread()
		{
			_locked = IsLocked();
			for (; ; )
			{
				if (_locked)
				{
					try
					{
						int idx = WaitHandle.WaitAny(new WaitHandle[] { _mutex, _cancelMonitorEvent });
						if (idx == 1)
						{
							break;
						}
					}
					catch (AbandonedMutexException)
					{
					}

					_mutex.ReleaseMutex();
				}
				else
				{
					using EventWaitHandle lockedEvent = CreateLockedEvent();

					int idx = WaitHandle.WaitAny(new WaitHandle[] { lockedEvent, _cancelMonitorEvent });
					if (idx == 1)
					{
						break;
					}
				}

				_locked ^= true;
				OnChange?.Invoke(!_locked);
			}
		}

		EventWaitHandle CreateLockedEvent() => new EventWaitHandle(false, EventResetMode.ManualReset, $"{Prefix}.{_objectName}.locked");
	}
}
