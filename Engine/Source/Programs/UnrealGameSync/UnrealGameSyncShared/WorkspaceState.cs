// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using EpicGames.Core;

namespace UnrealGameSync
{
	/// <summary>
	/// The current workspace state. Use <see cref="WorkspaceStateWrapper"/> to access state for a workspace.
	/// </summary>
	public class WorkspaceState
	{
		// **********************************************************************
		//
		// NOTE: UPDATE CopyFrom() BELOW WHEN ADDING ANY PROPERTIES TO THIS CLASS
		//
		// **********************************************************************

		// Cached state about the project, configured using UserWorkspaceSettings and taken from computed values in ProjectInfo. Assumed valid unless manually updated.
		public long SettingsTimeUtc { get; set; }
		public string ClientName { get; set; } = String.Empty;
		public string BranchPath { get; set; } = String.Empty;
		public string ProjectPath { get; set; } = String.Empty;
		public string? StreamName { get; set; }

		public bool StreamSwitched { get; set; } = false; // Whether the stream has been switched since the last sync

		public string ProjectIdentifier { get; set; } = String.Empty; // Should be fully reset if this changes
		public bool IsEnterpriseProject { get; set; }

		// Settings for the currently synced project in this workspace. CurrentChangeNumber is only valid for this workspace if CurrentProjectPath is the current project.
		public int CurrentChangeNumber { get; set; } = -1;
		public int CurrentCodeChangeNumber { get; set; } = -1;
		public string? CurrentSyncFilterHash { get; set; }
		public List<int> AdditionalChangeNumbers { get; init; } = new List<int>();

		// Settings for the last attempted sync. These values are set to persist error messages between runs.
		public int LastSyncChangeNumber { get; set; }
		public WorkspaceUpdateResult LastSyncResult { get; set; }
		public string? LastSyncResultMessage { get; set; }
		public DateTime? LastSyncTime { get; set; }
		public int LastSyncDurationSeconds { get; set; }

		// The last successful build, regardless of whether a failed sync has happened in the meantime. Used to determine whether to force a clean due to entries in the project config file.
		public int LastBuiltChangeNumber { get; set; }

		// The path of the last synced editor archive
		public string LastSyncEditorArchive { get; set; } = "0";

		// Expanded archives in the workspace
		public HashSet<string> ExpandedArchiveTypes { get; init; } = new HashSet<string>(StringComparer.Ordinal);

		// The changes that we're regressing at the moment
		public List<BisectEntry> BisectChanges { get; init; } = new List<BisectEntry>();

		public void UpdateCachedProjectInfo(ProjectInfo projectInfo, long settingsTimeUtc)
		{
			SettingsTimeUtc = settingsTimeUtc;

			ClientName = projectInfo.ClientName;
			BranchPath = projectInfo.BranchPath;
			ProjectPath = projectInfo.ProjectPath;
			StreamName = projectInfo.StreamName;
			ProjectIdentifier = projectInfo.ProjectIdentifier;
			IsEnterpriseProject = projectInfo.IsEnterpriseProject;
		}

		public bool IsValid(ProjectInfo projectInfo)
		{
			return ProjectIdentifier.Equals(projectInfo.ProjectIdentifier, StringComparison.OrdinalIgnoreCase);
		}

		public void ResetForProject(ProjectInfo projectInfo)
		{
			if (!IsValid(projectInfo))
			{
				CopyFrom(new WorkspaceState());
				UpdateCachedProjectInfo(projectInfo, 0);
			}
		}

		public void CopyFrom(WorkspaceState other)
		{
			SettingsTimeUtc = other.SettingsTimeUtc;
			ClientName = other.ClientName;
			BranchPath = other.BranchPath;
			ProjectPath = other.ProjectPath;
			StreamName = other.StreamName;

			ProjectIdentifier = other.ProjectIdentifier;
			IsEnterpriseProject = other.IsEnterpriseProject;

			CurrentChangeNumber = other.CurrentChangeNumber;
			CurrentCodeChangeNumber = other.CurrentCodeChangeNumber;
			CurrentSyncFilterHash = other.CurrentSyncFilterHash;
			AdditionalChangeNumbers.Clear();
			AdditionalChangeNumbers.AddRange(other.AdditionalChangeNumbers);

			StreamSwitched = other.StreamSwitched;

			LastSyncChangeNumber = other.LastSyncChangeNumber;
			LastSyncResult = other.LastSyncResult;
			LastSyncResultMessage = other.LastSyncResultMessage;
			LastSyncTime = other.LastSyncTime;
			LastSyncDurationSeconds = other.LastSyncDurationSeconds;

			LastBuiltChangeNumber = other.LastBuiltChangeNumber;

			LastSyncEditorArchive = other.LastSyncEditorArchive;

			ExpandedArchiveTypes.Clear();
			ExpandedArchiveTypes.UnionWith(other.ExpandedArchiveTypes);

			BisectChanges.Clear();
			BisectChanges.AddRange(other.BisectChanges);
		}

		public void SetBisectState(int change, BisectState state)
		{
			BisectEntry? entry = BisectChanges.FirstOrDefault(x => x.Change == change);
			if (entry == null)
			{
				entry = new BisectEntry();
				entry.Change = change;
				BisectChanges.Add(entry);
			}
			entry.State = state;
		}

		public void SetLastSyncState(WorkspaceUpdateResult result, WorkspaceUpdateContext context, string statusMessage)
		{
			LastSyncChangeNumber = context.ChangeNumber;
			LastSyncResult = result;
			LastSyncResultMessage = statusMessage;
			LastSyncTime = DateTime.UtcNow;
			LastSyncDurationSeconds = (int)(LastSyncTime.Value - context.StartTime).TotalSeconds;
		}
	}

	/// <summary>
	/// Read-only wrapper around <see cref="WorkspaceState"/>. To modify state, use <see cref="WorkspaceStateWrapper.Modify(Action{WorkspaceState})"/>.
	/// </summary>
	public class ReadOnlyWorkspaceState
	{
		readonly WorkspaceState _inner;

		public long SettingsTimeUtc => _inner.SettingsTimeUtc;
		public string ClientName => _inner.ClientName;
		public string BranchPath => _inner.BranchPath;
		public string ProjectPath => _inner.ProjectPath;
		public string? StreamName => _inner.StreamName;
		public bool StreamSwitched => _inner.StreamSwitched;
		public string ProjectIdentifier => _inner.ProjectIdentifier;
		public bool IsEnterpriseProject => _inner.IsEnterpriseProject;

		// Settings for the currently synced project in this workspace. CurrentChangeNumber is only valid for this workspace if CurrentProjectPath is the current project.
		public int CurrentChangeNumber => _inner.CurrentChangeNumber;
		public int CurrentCodeChangeNumber => _inner.CurrentCodeChangeNumber;
		public string? CurrentSyncFilterHash => _inner.CurrentSyncFilterHash;
		public IReadOnlyList<int> AdditionalChangeNumbers => _inner.AdditionalChangeNumbers;

		// Settings for the last attempted sync. These values are set to persist error messages between runs.
		public int LastSyncChangeNumber => _inner.LastSyncChangeNumber;
		public WorkspaceUpdateResult LastSyncResult => _inner.LastSyncResult;
		public string? LastSyncResultMessage => _inner.LastSyncResultMessage;
		public DateTime? LastSyncTime => _inner.LastSyncTime;
		public int LastSyncDurationSeconds => _inner.LastSyncDurationSeconds;

		// The last successful build, regardless of whether a failed sync has happened in the meantime. Used to determine whether to force a clean due to entries in the project config file.
		public int LastBuiltChangeNumber => _inner.LastBuiltChangeNumber;

		// The path of the last synced editor archive
		public string LastSyncEditorArchive => _inner.LastSyncEditorArchive;

		// Expanded archives in the workspace
		public IReadOnlySet<string> ExpandedArchiveTypes { get; }

		// The changes that we're regressing at the moment
		public IReadOnlyList<BisectEntry> BisectChanges => _inner.BisectChanges;

		internal ReadOnlyWorkspaceState(WorkspaceState inner)
		{
			_inner = inner;
			ExpandedArchiveTypes = new HashSet<string>(_inner.ExpandedArchiveTypes);
		}

		public ReadOnlyWorkspaceState ResetForProject(ProjectInfo projectInfo)
		{
			if (_inner.IsValid(projectInfo))
			{
				return this;
			}
			else
			{
				WorkspaceState state = new WorkspaceState();
				state.ResetForProject(projectInfo);
				return new ReadOnlyWorkspaceState(state);
			}
		}

		public WorkspaceState MutableCopy()
		{
			WorkspaceState state = new WorkspaceState();
			state.CopyFrom(_inner);
			return state;
		}
	}

	/// <summary>
	/// Monitors state of a workspace state file.
	/// </summary>
	public sealed class WorkspaceStateWrapper : IDisposable
	{
		/// <summary>
		/// Directory containing the config file
		/// </summary>
		public DirectoryReference RootDir { get; }

		/// <summary>
		/// Path to the modified file
		/// </summary>
		public FileReference File { get; }

		/// <summary>
		/// Current workspace state
		/// </summary>
		public ReadOnlyWorkspaceState Current { get; private set; }

		/// <summary>
		/// Callback for modifications to the workspace state
		/// </summary>
		public event Action<ReadOnlyWorkspaceState>? OnModified;

		readonly FileSystemWatcher _watcher;
		readonly Func<WorkspaceState> _createNew;

		/// <summary>
		/// Constructor
		/// </summary>
		public WorkspaceStateWrapper(DirectoryReference rootDir, Func<WorkspaceState> createNew)
		{
			RootDir = rootDir;

			DirectoryReference configDir = UserSettings.GetConfigDir(rootDir);
			File = FileReference.Combine(configDir, "state.json");

			UserSettings.CreateConfigDir(RootDir);

			_watcher = new FileSystemWatcher(configDir.FullName, File.GetFileName());
			_watcher.NotifyFilter = NotifyFilters.LastWrite;
			_watcher.Changed += OnModifiedInternal;
			_watcher.Created += OnModifiedInternal;
			_watcher.Deleted += OnModifiedInternal;
			_watcher.EnableRaisingEvents = true;

			_createNew = createNew;

			Current = GetSnapshot();
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_watcher.Dispose();
		}

		/// <summary>
		/// Modify the current workspace state
		/// </summary>
		/// <param name="action">Action to be called with the state to be modified</param>
		public ReadOnlyWorkspaceState Modify(Action<WorkspaceState> action)
		{
			for (; ; )
			{
				try
				{
					using (FileStream stream = FileReference.Open(File, FileMode.OpenOrCreate, FileAccess.ReadWrite, FileShare.None))
					{
						stream.Position = 0;
						WorkspaceState state = ReadState(stream);
						action(state);

						stream.Position = 0;
						stream.Write(Utility.SerializeJson(state));
						stream.SetLength(stream.Position);

						Current = new ReadOnlyWorkspaceState(state);

						return Current;
					}
				}
				catch (IOException ex) when ((ex.HResult & 0xffff) == ERROR_SHARING_VIOLATION)
				{
					Thread.Sleep(10);
				}
			}
		}

		void OnModifiedInternal(object sender, FileSystemEventArgs e)
		{
			Current = GetSnapshot();
			OnModified?.Invoke(Current);
		}

		const int ERROR_SHARING_VIOLATION = 32;

		private ReadOnlyWorkspaceState GetSnapshot()
		{
			for (; ; )
			{
				try
				{
					using (FileStream stream = FileReference.Open(File, FileMode.OpenOrCreate, FileAccess.Read, FileShare.Read))
					{
						WorkspaceState state = ReadState(stream);
						return new ReadOnlyWorkspaceState(state);
					}
				}
				catch (IOException ex) when ((ex.HResult & 0xffff) == ERROR_SHARING_VIOLATION)
				{
					Thread.Sleep(10);
				}
			}
		}

		WorkspaceState ReadState(FileStream stream)
		{
			if (stream.Length == 0)
			{
				return new WorkspaceState();
			}

			byte[] buffer = new byte[stream.Length];
			try
			{
				stream.ReadExactly(buffer);
			}
			catch (EndOfStreamException)
			{
				return _createNew();
			}
			return Utility.TryDeserializeJson<WorkspaceState>(buffer) ?? _createNew();
		}
	}
}
